{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"Modules": [
		{
			"Name": "ElectraDecoders",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "WinGDK" ]
		},
		{
			"Name": "ElectraCodecFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "WinGDK" ]
		}
	]
}
