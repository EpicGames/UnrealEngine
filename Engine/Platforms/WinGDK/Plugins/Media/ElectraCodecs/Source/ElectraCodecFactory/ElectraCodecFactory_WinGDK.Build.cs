// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraCodecFactory_WinGDK : ElectraCodecFactory
	{
		public ElectraCodecFactory_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
