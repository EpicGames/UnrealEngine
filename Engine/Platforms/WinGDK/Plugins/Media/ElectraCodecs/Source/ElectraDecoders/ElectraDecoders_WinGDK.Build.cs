// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraDecoders_WinGDK : ElectraDecoders
	{
		protected override bool bPlatformHasDefaultDecoders
		{
			get
			{
				return true;
			}
		}

		public ElectraDecoders_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
