// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "WindowsPlatformHeaders_Video_DX.h"

#endif
