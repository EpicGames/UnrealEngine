{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Electra Codecs",
	"Description": "Codecs for use with Electra player.",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Engine/MediaFramework/Overview",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ElectraDecoders",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64", "Android", "Mac", "IOS", "TVOS", "Linux" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "ElectraCodecFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Android", "Mac", "IOS", "TVOS", "Linux" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "ElectraCodecFactory",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "ElectraUtil",
			"Enabled": true
		},
		{
			"Name": "MP4Utilities",
			"Enabled": true
		}
	]
}
