// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraCodecFactory : ModuleRules
	{
		public ElectraCodecFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			bLegalToDistributeObjectCode = true;

			bool bSupportedPlatform = IsSupportedPlatform(Target);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ElectraDecoders",
                    "ElectraBase"
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
			}

			if (bSupportedPlatform)
			{
				// ...
			}
		}

		protected virtual bool IsSupportedPlatform(ReadOnlyTargetRules Target)
		{
			return Target.IsInPlatformGroup(UnrealPlatformGroup.Windows)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Android)
				|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix)
		        || Target.IsInPlatformGroup(UnrealPlatformGroup.Apple)
				|| false
				;
		}
	}
}
