// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraCodecFactoryModule.h"

#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"

#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraCodecRegistry.h"

#include "IElectraDecodersModule.h"

#define LOCTEXT_NAMESPACE "ElectraCodecFactoryModule"

DEFINE_LOG_CATEGORY(LogElectraCodecFactory);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraCodecRegistry : public IElectraCodecModularFeature, public IElectraCodecRegistry
{
public:
	virtual ~FElectraCodecRegistry()
	{ }

	//-------------------------------------------------------------------------
	// Methods from IElectraCodecModularFeature
	//
	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override;

	//-------------------------------------------------------------------------
	// Methods from IElectraCodecRegistry
	//
	void AddCodecFactory(TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InCodecFactory) override;

private:
	mutable FCriticalSection Lock;
	TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> RegisteredCodecFactories;
};

// -----------------------------------------------------------------------------------------------------------------------------------


class FElectraCodecFactoryModule : public IElectraCodecFactoryModule
{
public:
	void StartupModule() override
	{
		// Load the ElectraDecoders module that provides the standard decoders.
		IElectraDecodersModule* ElectraDecoders = static_cast<IElectraDecodersModule*>(FModuleManager::Get().LoadModule(TEXT("ElectraDecoders")));
		if (ElectraDecoders)
		{
			// Have the standard decoders register themselves with us.
			ElectraDecoders->RegisterDecodersWithCodecFactory(&StandardDecoders);

			// Add our factory as a modular feature.
			IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), &StandardDecoders);
		}
		else
		{
			UE_LOGF(LogElectraCodecFactory, Warning, "The standard ElectraDecoders module was not loaded correctly. Decoders may not be available.");
		}
	}

	void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), &StandardDecoders);
	}

	bool SupportsDynamicReloading() override
	{
		return false;
	}

	TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> GetBestDecoderFactoryForFormat(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override;

private:
	FElectraCodecRegistry StandardDecoders;
};

IMPLEMENT_MODULE(FElectraCodecFactoryModule, ElectraCodecFactory);


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FElectraCodecFactoryModule::GetBestDecoderFactoryForFormat(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.Type == Electra::FCodecTypeFormat::EType::Video || InCodecFormat.Type == Electra::FCodecTypeFormat::EType::Audio)
	{
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		// Get the list of registered factories
		TArray<IElectraCodecModularFeature*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IElectraCodecModularFeature>(GetModularFeatureName());
		TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> BestFactory;
		int32 BestPriority = 0;
		for(int32 i=0; i<Factories.Num(); ++i)
		{
			TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> FormatFactories;
			Factories[i]->GetListOfFactories(FormatFactories);
			for(int32 j=0; j<FormatFactories.Num(); ++j)
			{
				TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FormatFactory = FormatFactories[j].Pin();
				TMap<FString, FVariant> FormatInfo;
				int32 SupportedPriority = FormatFactory.IsValid() ? FormatFactory->SupportsDecoding(FormatInfo, InCodecFormat, InAdditionalOptions) : 0;
				if (SupportedPriority > BestPriority)
				{
					BestPriority = SupportedPriority;
					BestFactory = FormatFactory;
					OutFormatInfo = MoveTemp(FormatInfo);
				}
			}
		}
		return BestFactory;
	}
	else
	{
		UE_LOGF(LogElectraCodecFactory, Warning, "GetBestDecoderFactoryForFormat() handles only video and audio at the moment.");
		return nullptr;
	}
}


// -----------------------------------------------------------------------------------------------------------------------------------
void FElectraCodecRegistry::AddCodecFactory(TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InCodecFactory)
{
	if (InCodecFactory.IsValid())
	{
		FScopeLock lock(&Lock);
		RegisteredCodecFactories.Emplace(MoveTemp(InCodecFactory));
	}
}

void FElectraCodecRegistry::GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories)
{
	FScopeLock lock(&Lock);
	OutCodecFactories.Append(RegisteredCodecFactories);
}

#undef LOCTEXT_NAMESPACE
