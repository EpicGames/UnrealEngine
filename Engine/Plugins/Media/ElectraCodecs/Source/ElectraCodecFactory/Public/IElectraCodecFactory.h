// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Containers/Map.h"
#include "Misc/Variant.h"
#include "CodecTypeFormat.h"

class IElectraDecoder;


class IElectraCodecFactory
{
public:
	virtual ~IElectraCodecFactory() = default;

	/**
	 * Populates the provided map with decoder configuration options (see IElectraDecoderFeature).
	 * Some required options must be available through the factory prior to creating a decoder instance.
	 */
	virtual void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const = 0;

	/**
	 * Queries whether or not this codec factory can create a decoder capable of decoding the specified format.
	 *
	 * The following FCodecTypeFormat members need to be set up
	 *	.Type
	 *  .FourCC
	 *  .RFC6381
	 *  .Properties
	 *     to the appropriate type like FVideo, FAudio, etc.
	 *
	 *  For video decoders:
	 *    Required:
	 *       .Properties.FVideo.Profile
	 *    Optional, but recommended:
	 *       .Properties.FVideo.Width
	 *       .Properties.FVideo.Height
	 *       .Properties.FVideo.FrameRate
	 *
	 * For convenience you can just set .RFC6381 and .DCR and have ElectraDecodersUtil::PrepareCodecTypeFormat()
	 * from the ElectraCodecs plugin fill in all other parameters.
	 *
	 * Additional options may be provided to specify the format more closely, which may help selecting
	 * the best suited implementation. See below.
	 *
	 * Returns 0 if not supported.
	 * If supported the return value indicates a priority value. If multiple factories are registered
	 * that claim support for the format the one with the highest priority is chosen.
	 */
	virtual int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const = 0;

	/**
	 * Called to create a decoder for the given format.
	 */
	virtual TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) = 0;


	class IProviderInformation
	{
	public:
		virtual ~IProviderInformation() = default;
		/**
		 * Returns the name of the implementation, like `VideoToolbox` or similar.
		 * Could be an empty string if information is not provided.
		 */
		virtual FString GetName() const = 0;

		/**
		 * Returns a version number of the implementation.
		 * Could be an empty string if information is not provided.
		 */
		virtual FString GetVersion() const = 0;

		/**
		 * Returns implementation, which could be platform specific, or a version specific value.
		 * Could be an empty string if information is not provided.
		 */
		virtual FString GetImplementation() const = 0;

		/**
		 * Returns the name of the vendor, if the codec is provided by the OS manufacturer.
		 * Could be an empty string if information is not provided.
		 */
		virtual FString GetVendor() const = 0;
	};
	/**
	 * Returns an interface to query decoder provider information.
	 */
	virtual const IProviderInformation& GetProviderInformation() const = 0;
};

/*
	Additional decoder options:

	General:
		Any option starting with a `$` character are those provided by the container (eg. the mp4 file)
		and are not exactly standardized.

	Audio decoder:

		"$chan_box" (ByteArray)
		    - The raw contents of the mp4 'chan' box if it exists without the first 8 bytes (size and atom)

		"$enda_box" (ByteArray)
		    - The raw contents of the mp4 'enda' box if it exists without the first 8 bytes (size and atom)

		"$FormatSpecificFlags" (int64)
			- The "FormatSpecificFlags" from a version 2 AudioSampleEntry


	Video decoder:

		"max_width"
		"max_height"
		"max_fps" or "max_fps_n" and "max_fps_d"
			The max values that are expected to be used.
			When decoding an adaptive stream the max values are the ones of the highest stream
			whereas the non-max ones are that of the current stream.

		"max_codecprofile" (FString)
			A codec profile string like `avc1.4d002a` indicating the maximum decoder profile and level
			that is expected to be handled.
*/
