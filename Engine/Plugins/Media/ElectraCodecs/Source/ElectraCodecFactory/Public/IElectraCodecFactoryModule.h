// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Containers/Map.h"
#include "Misc/Variant.h"
#include "CodecTypeFormat.h"

class IElectraCodecFactory;


/**
 * Interface for the `ElectraCodecFactory` module.
 */
class IElectraCodecFactoryModule : public IModuleInterface
{
public:
	virtual ~IElectraCodecFactoryModule() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ElectraCodecFactory"));
		return FeatureName;
	}

	virtual TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> GetBestDecoderFactoryForFormat(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) = 0;
};
