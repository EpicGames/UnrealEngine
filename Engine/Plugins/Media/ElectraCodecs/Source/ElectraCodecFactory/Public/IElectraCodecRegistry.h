// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeature.h"

class IElectraCodecFactory;

class IElectraCodecRegistry
{
public:
	virtual ~IElectraCodecRegistry() = default;

	virtual void AddCodecFactory(TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InCodecFactory) = 0;
};


class IElectraCodecModularFeature : public IModularFeature
{
public:
	virtual ~IElectraCodecModularFeature() = default;
	virtual void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) = 0;
};
