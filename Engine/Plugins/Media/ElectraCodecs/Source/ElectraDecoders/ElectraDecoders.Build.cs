// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class ElectraDecoders: ModuleRules
	{
		protected virtual bool bPlatformHasDefaultDecoders
		{
			get
			{
				return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows)
					|| Target.Platform.IsInGroup(UnrealPlatformGroup.Android)
					|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix)
					|| Target.IsInPlatformGroup(UnrealPlatformGroup.Apple);
			}
		}

		public ElectraDecoders(ReadOnlyTargetRules Target) : base(Target)
		{
			//
			// Common setup...
			//
			bLegalToDistributeObjectCode = true;

			bRequiresPlatformSDK = true;

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "RHI",
                    "Media",
					"MediaUtils",
                    "ElectraSamples",
					"ElectraBase",
                    "MP4Utilities",
                    "MP4Boxes",
                    // Common codecs
					"dr_libs"					// MPEG audio layer 3
				});
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"ElectraCodecFactory"
				});

			//
			// Common platform setup...
			//
			PublicDefinitions.Add("ELECTRA_DECODERS_HAVE_PLATFORM_DEFAULTS=" + (bPlatformHasDefaultDecoders ? "1" : "0"));

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
    			PrivateDependencyModuleNames.Add("SignalProcessing");

				PublicDependencyModuleNames.Add("DirectX");

				PrivateDependencyModuleNames.AddAll("D3D11RHI", "D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");


				PrivateDefinitions.Add("_CRT_SECURE_NO_WARNINGS=1");

				PrivateDefinitions.Add("ELECTRA_DECODERS_ENABLE_DX=1");
				//PrivateDefinitions.Add("ELECTRACODECS_ENABLE_MF_SWDECODE_H264=1");

				if (WinSupportsDX11())
				{
					PrivateDefinitions.Add("ELECTRA_DECODERS_HAVE_DX11");	// video decoding for DX11 enabled (Win8+)
				}

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}

				PublicSystemLibraries.AddRange(new string[] {
					"strmiids.lib",
					"legacy_stdio_definitions.lib",
					"Dxva2.lib",
				});

				// Delay-load all MF DLLs to be able to check Windows version for compatibility in `StartupModule` before loading them manually
				PublicSystemLibraries.Add("mfplat.lib");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicSystemLibraries.Add("mfuuid.lib");

				PublicIncludePaths.Add("$(ModuleDir)/Public/Windows");
				PublicIncludePaths.Add("$(ModuleDir)/Private/Windows");
			}
			else if (Target.Platform.IsInGroup(UnrealPlatformGroup.Android))
			{
				PrivateIncludePaths.Add("ElectraDecoders/Private/Android");
				PublicIncludePaths.Add("$(ModuleDir)/Public/Android");

				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "ElectraDecoders_UPL.xml"));
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
			{
				PrivateDefinitions.Add("ELECTRA_DECODERS_ENABLE_LINUX=1");
				PrivateIncludePaths.Add("ElectraDecoders/Private/Linux");
				PublicIncludePaths.Add("$(ModuleDir)/Public/Linux");

				AddEngineThirdPartyPrivateStaticDependencies(Target, "libav");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				PublicFrameworks.AddRange(
				new string[] {
								"CoreMedia",
								"CoreVideo",
								"AVFoundation",
								"AudioToolbox",
								"VideoToolbox",
								"QuartzCore"
				});

				PrivateIncludePathModuleNames.Add("MetalRHI");

				PrivateDefinitions.Add("ELECTRA_DECODERS_ENABLE_APPLE=1");
				PrivateIncludePaths.Add("ElectraDecoders/Private/Apple");
			}
		}

		protected virtual bool WinSupportsDX11()
		{
			return true;
		}
	}
}
