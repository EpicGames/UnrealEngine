<?xml version="1.0" encoding="utf-8"?>
<!--ElectraDecoders plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="ElectraDecoders init"/>
		<!-- currently supports all architectures -->
		<setBool result="bSupported" value="true"/>
		<if condition="bSupported">
			<true>
			</true>
			<false>
				<log text="ElectraDecoders not supported for this architecture, disabled."/>
			</false>
		</if>
	</init>

	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<if condition="bSupported">
			<true>
			</true>
		</if>
	</androidManifestUpdates>

	<!-- optional additions to proguard -->
	<proguardAdditions>
		<insert>
-keep class com.epicgames.unreal.ElectraDecoderVideoH264 {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH264$FCreateParameters {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH264$FDecoderInformation {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH264$FOutputFormatInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH264$FOutputBufferInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH265 {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH265$FCreateParameters {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH265$FDecoderInformation {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH265$FOutputFormatInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoH265$FOutputBufferInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoVPx {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoVPx$FCreateParameters {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoVPx$FDecoderInformation {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoVPx$FOutputFormatInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderVideoVPx$FOutputBufferInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderAudioAAC {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderAudioAAC$FOutputFormatInfo {
	public *;
}
-keep class com.epicgames.unreal.ElectraDecoderAudioAAC$FOutputBufferInfo {
	public *;
}
		</insert>
	</proguardAdditions>

	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
		<if condition="bSupported">
			<true>
				<copyDir src="$S(PluginDir)/Private/Android/Java" dst="$S(BuildDir)" />
			</true>
		</if>
	</resourceCopies>

	<!-- optional additions to the GameActivity imports in GameActivity.java -->
	<gameActivityImportAdditions>
		<if condition="bSupported">
			<true>
			</true>
		</if>
	</gameActivityImportAdditions>

	<!-- optional additions to the GameActivity class in GameActivity.java -->
	<gameActivityClassAdditions>
		<if condition="bSupported">
			<true>
			</true>
		</if>
	</gameActivityClassAdditions>

	<!-- optional additions to GameActivity onCreate in GameActivity.java
	<gameActivityOnCreateAdditions>
	</gameActivityOnCreateAdditions>
	-->

	<!-- optional additions to GameActivity onDestroy in GameActivity.java
	<gameActivityOnDestroyAdditions>
	</gameActivityOnDestroyAdditions>
	-->

	<!-- optional additions to GameActivity onStop in GameActivity.java -->
	<gameActivityOnStopAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnStopAdditions>


	<!-- optional additions to GameActivity onPause in GameActivity.java -->
	<gameActivityOnPauseAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnPauseAdditions>

	<!-- optional additions to GameActivity onResume in GameActivity.java -->
	<gameActivityOnResumeAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnResumeAdditions>

	<!-- optional additions to GameActivity onActivityResult in GameActivity.java
	<gameActivityOnActivityResultAdditions>
	</gameActivityOnActivityResultAdditions>
	-->

	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	<soLoadLibrary>
	</soLoadLibrary>
</root>
