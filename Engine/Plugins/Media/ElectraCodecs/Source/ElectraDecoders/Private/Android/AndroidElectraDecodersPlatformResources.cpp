// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidElectraDecodersPlatformResources.h"
#include "Misc/ScopeLock.h"

#include "ElectraDecodersModule.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MediaDecoderOutput.h"
#include "MediaVideoDecoderOutput.h"
#include "ElectraTextureSample.h"

#include <atomic>



namespace ElectraCodecResourcesAndroid
{
	static FCriticalSection Lock;
	static TMap<int64, TWeakPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe>> HandlerMap;
	std::atomic<int64> NextPoolID { 0 };


	class FDecoderResourceInstance : public FElectraDecodersPlatformResourcesAndroid::IDecoderResourceInstance
	{
	public:
		FDecoderResourceInstance(int64 InResourceID)
		{
			FScopeLock lock(&ElectraCodecResourcesAndroid::Lock);
			if (ElectraCodecResourcesAndroid::HandlerMap.Contains(InResourceID))
			{
				TexturePool = ElectraCodecResourcesAndroid::HandlerMap[InResourceID];
			}
		}

		virtual ~FDecoderResourceInstance()
		{
		}

		void RequestSurface(TWeakPtr<ISurfaceRequestCallback, ESPMode::ThreadSafe> InRequestCallback) override
		{
			TSharedPtr<ISurfaceRequestCallback, ESPMode::ThreadSafe> ReqCB = InRequestCallback.Pin();
			if (ReqCB)
			{
				TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> Pool = TexturePool.Pin();
				if (Pool)
				{
					jobject Surface = (jobject) Pool->GetCodecSurface();
					ReqCB->OnNewSurface(Surface ? ISurfaceRequestCallback::ESurfaceType::Surface : ISurfaceRequestCallback::ESurfaceType::Error, Surface);
				}
				else
				{
					ReqCB->OnNewSurface(ISurfaceRequestCallback::ESurfaceType::Error, nullptr);
				}
			}
		}

	private:
		TWeakPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> TexturePool;
	};


}


int64 FElectraDecodersPlatformResourcesAndroid::RegisterOutputTexturePool(TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> InTexturePool)
{
	FScopeLock lock(&ElectraCodecResourcesAndroid::Lock);
	int64 PoolID = ++ElectraCodecResourcesAndroid::NextPoolID;
	ElectraCodecResourcesAndroid::HandlerMap.Add(PoolID, InTexturePool);
	return PoolID;
}

void FElectraDecodersPlatformResourcesAndroid::UnregisterOutputTexturePool(int64 InTexturePoolID)
{
	FScopeLock lock(&ElectraCodecResourcesAndroid::Lock);
	ElectraCodecResourcesAndroid::HandlerMap.Remove(InTexturePoolID);
}



FElectraDecodersPlatformResourcesAndroid::IDecoderResourceInstance* FElectraDecodersPlatformResourcesAndroid::CreateResourceInstance(int64 InResourceID)
{
	return new ElectraCodecResourcesAndroid::FDecoderResourceInstance(InResourceID);
}

void FElectraDecodersPlatformResourcesAndroid::ReleaseResourceInstance(IDecoderResourceInstance* InInstanceToRelease)
{
	ElectraCodecResourcesAndroid::FDecoderResourceInstance* Inst = static_cast<ElectraCodecResourcesAndroid::FDecoderResourceInstance*>(InInstanceToRelease);
	delete Inst;
}



FElectraDecodersPlatformResourcesAndroid::IDecoderPlatformResource* FElectraDecodersPlatformResourcesAndroid::CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions)
{
	return nullptr;
}

void FElectraDecodersPlatformResourcesAndroid::ReleasePlatformVideoResource(void* InOwnerHandle, IDecoderPlatformResource* InHandleToRelease)
{
}

bool FElectraDecodersPlatformResourcesAndroid::SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, IDecoderPlatformResource* InPlatformResource)
{
	if (!InOutTextureSampleToSetup.IsValid() || !InDecoderOutput.IsValid() || !InOutBufferPropertes.IsValid())// || !InPlatformResource)
	{
		OutErrorMessage = TEXT("Bad parameter");
		return false;
	}

	TMap<FString, FVariant> ExtraValues;
	InDecoderOutput->GetExtraValues(ExtraValues);

	FElectraVideoDecoderOutputCropValues Crop = InDecoderOutput->GetCropValues();
// TBD: Zero out the crop values for now since the output only consists of active pixels
//      here and we don't have to crop ourselves.
	Crop.Top = Crop.Left = Crop.Bottom = Crop.Right = 0;
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Width, Electra::FVariantValue((int64)InDecoderOutput->GetWidth()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Height, Electra::FVariantValue((int64)InDecoderOutput->GetHeight()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropLeft, Electra::FVariantValue((int64)Crop.Left));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropRight, Electra::FVariantValue((int64)Crop.Right));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropTop, Electra::FVariantValue((int64)Crop.Top));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropBottom, Electra::FVariantValue((int64)Crop.Bottom));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectRatio, Electra::FVariantValue((double)InDecoderOutput->GetAspectRatioW() / (double)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectW, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioW()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectH, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSNumerator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateNumerator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSDenominator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateDenominator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelEncoding, Electra::FVariantValue((int64)EElectraTextureSamplePixelEncoding::Native));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelDataScale, Electra::FVariantValue((double)ElectraDecodersUtil::GetVariantValueSafeDouble(ExtraValues, TEXT("pix_datascale"), 1.0)));

	int32 NumBits = InDecoderOutput->GetNumberOfBits();
	EPixelFormat PixFmt = static_cast<EPixelFormat>(ElectraDecodersUtil::GetVariantValueSafeI64(ExtraValues, TEXT("pixfmt"), (int64)((NumBits > 8)  ? EPixelFormat::PF_A2B10G10R10 : EPixelFormat::PF_B8G8R8A8)));
	int32 Pitch = static_cast<int32>(ElectraDecodersUtil::GetVariantValueSafeI64(ExtraValues, TEXT("pitch"), (int64)InDecoderOutput->GetDecodedWidth() * ((NumBits > 8) ? 2 : 1)));

	InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)PixFmt));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64)NumBits));


	InOutTextureSampleToSetup->InitializeCommon(*InOutBufferPropertes);
	if (!InOutTextureSampleToSetup->Initialize())
	{
		OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Failed to initialize texture sample"));
		return false;
	}
	if (!InOutTextureSampleToSetup->FinishInitialization())
	{
		OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Failed to finalize texture sample"));
		return false;
	}
	return true;
}

