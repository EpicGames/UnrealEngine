// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformElectraDecoders.h"
#include "IElectraCodecRegistry.h"

#include "h264/H264_VideoDecoder_Android.h"
#include "h265/H265_VideoDecoder_Android.h"
#include "vpx/VPx_VideoDecoder_Android.h"
#include "aac/AAC_AudioDecoder_Android.h"

#include "HAL/IConsoleManager.h"


namespace PlatformElectraDecodersAndroid
{
	static TArray<TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> Factories;

	TAutoConsoleVariable<bool> CVarElectraDecoderAndroidRebasePTS(
		TEXT("ElectraDecoders.AndroidRebasePTS"),
		true,
		TEXT("Rebase PTS to zero for Android decoders.\n")
		TEXT(" true: rebase so values start at 0, false use original values"),
		ECVF_Default);
}


void FPlatformElectraDecodersAndroid::Startup()
{
}

void FPlatformElectraDecodersAndroid::Shutdown()
{
	PlatformElectraDecodersAndroid::Factories.Empty();
}

void FPlatformElectraDecodersAndroid::RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith)
{
	check(InCodecFactoryToRegisterWith);
	if (InCodecFactoryToRegisterWith)
	{
		auto RegisterFactory = [&](TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InFactory) -> void
		{
			if (InFactory.IsValid())
			{
				PlatformElectraDecodersAndroid::Factories.Emplace(InFactory);
				InCodecFactoryToRegisterWith->AddCodecFactory(InFactory);
			}
		};

		// H.264 video decoder
		RegisterFactory(FH264VideoDecoderAndroid::CreateFactory());
		// H.265 video decoder
		RegisterFactory(FH265VideoDecoderAndroid::CreateFactory());
		// VP8/VP9 video decoder
		RegisterFactory(FVPxVideoDecoderAndroid::CreateFactory());
		// AAC audio decoder
		RegisterFactory(FAACAudioDecoderAndroid::CreateFactory());

	}
}
