// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class IElectraCodecRegistry;

class FPlatformElectraDecodersAndroid
{
public:
	static void Startup();
	static void Shutdown();
	static void RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith);
};

using FPlatformElectraDecoders = FPlatformElectraDecodersAndroid;
