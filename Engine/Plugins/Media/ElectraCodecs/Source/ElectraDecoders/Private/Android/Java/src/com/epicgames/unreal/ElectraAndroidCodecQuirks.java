// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.annotation.SuppressLint;
import android.annotation.TargetApi;

import android.os.Build;
import java.io.IOException;
import java.util.ArrayList;
import android.util.Log;

import android.media.AudioFormat;
import android.media.MediaCodecList;
import android.media.MediaCodec;
import android.media.MediaCodecInfo;
import android.media.MediaCodecInfo.CodecCapabilities;
import android.media.MediaCodecInfo.CodecProfileLevel;
import android.media.MediaFormat;


public class ElectraAndroidCodecQuirks
{
	public static class FDecoderQuirks
	{
		public int ApiLevel = 0;
		public boolean bCanUse_SetOutputSurface = false;
	}

	private static FDecoderQuirks DecoderQuirks;
	private static final ElectraAndroidCodecQuirks Singleton = new ElectraAndroidCodecQuirks();

	private ElectraAndroidCodecQuirks()
	{
		DecoderQuirks = new FDecoderQuirks();
		DecoderQuirks.ApiLevel = Build.VERSION.SDK_INT;

		// setOutputSurface() exists from API level 23 onward.
		// There are however several devices that have issues with it, but apparently throw an IllegalArgumentException
		// which we catch and then fall back to handle this as if setOutputSurface() did not exist in the first place.
		DecoderQuirks.bCanUse_SetOutputSurface = DecoderQuirks.ApiLevel >= 23;
	}

	private static ElectraAndroidCodecQuirks GetInstance()
	{
		return Singleton;
	}

	public static FDecoderQuirks GetDecoderQuirks()
	{
		return GetInstance().DecoderQuirks;
	}
}

