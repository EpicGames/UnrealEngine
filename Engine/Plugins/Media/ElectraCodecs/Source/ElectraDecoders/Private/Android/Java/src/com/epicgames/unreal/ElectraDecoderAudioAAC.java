// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.os.Build;
import java.io.IOException;
import java.util.ArrayList;
import android.util.Log;
import android.util.SparseArray;

import android.media.AudioFormat;
import android.media.MediaCodec;
import android.media.MediaCodecInfo;
import android.media.MediaFormat;
import java.nio.ByteBuffer;


/**
 * AAC audio decoder
 *
 */
public class ElectraDecoderAudioAAC
{
	private static final String TAG = "[ElectraDecoders]";

	public class FOutputFormatInfo
	{
		public int SampleRate = 0;
		public int NumChannels = 0;
		public int BytesPerSample = 0;
	}

	public class FOutputBufferInfo
	{
		public int BufferIndex = -1;
		public long PresentationTimestamp = -1;
		public int Size = 0;
		public boolean bIsEOS = false;
		public boolean bIsConfig = false;
	}

	private MediaCodec DecoderHandle = null;
	private boolean bIsInitialized = false;

	private SparseArray<MediaCodec.BufferInfo> OutputBufferInfos = null;
	private FOutputFormatInfo CurrentOutputFormatInfo = null;



	public ElectraDecoderAudioAAC()
	{
		DecoderHandle = null;
		bIsInitialized = false;
		OutputBufferInfos = new SparseArray<>();
		CurrentOutputFormatInfo = new FOutputFormatInfo();
	}


	public int CreateDecoder(int InNumChannels, int InSampleRate, byte[] InCSD)
	{
		try
		{
			// Create decoder configuration object
			MediaFormat Format = MediaFormat.createAudioFormat(MediaFormat.MIMETYPE_AUDIO_AAC, InSampleRate, InNumChannels);

			// When decoding RAW AAC we need to have codec specific data.
			ByteBuffer Csd = ByteBuffer.wrap(InCSD);
			Format.setByteBuffer("csd-0", Csd);
			// Try to get float output. This may or may not work.
			Format.setInteger(MediaFormat.KEY_PCM_ENCODING, AudioFormat.ENCODING_PCM_FLOAT);

			// Create and configure the decoder
			DecoderHandle = MediaCodec.createDecoderByType(MediaFormat.MIMETYPE_AUDIO_AAC);
			// Configure the codec (no surface, no crypto, as decoder)
			if (DecoderHandle != null)
			{
				DecoderHandle.configure(Format, null, null, 0);
				bIsInitialized = true;
			}
		}
		catch(Exception e)
		{
			Log.w(TAG, "ElectraDecoderAudioAAC: Failed to create decoder");
			e.printStackTrace();
			return 1;
		}
		return 0;
	}


	public int ReleaseDecoder()
	{
		if (bIsInitialized)
		{
			bIsInitialized = false;
			try
			{
				DecoderHandle.release();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to release decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Start()
	{
		if (bIsInitialized)
		{
			try
			{
				// Start the decoder
				DecoderHandle.start();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to start decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Stop()
	{
		if (bIsInitialized)
		{
			try
			{
				// Stop the decoder
				DecoderHandle.stop();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to stop decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Flush()
	{
		if (bIsInitialized)
		{
			try
			{
				// Flush the decoder
				DecoderHandle.flush();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to flush decoder");
				e.printStackTrace();
				return 1;
			}

			// Clear the current infos by creating new empty structures.
			OutputBufferInfos = new SparseArray<>();
			// NOTE: Flushing does not invalidate the current output configuration. Do not clear it.
			// CurrentOutputFormatInfo = new FOutputFormatInfo();
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int DequeueInputBuffer(int InTimeoutUsec)
	{
		if (bIsInitialized)
		{
			int BufferIndex = -1;
			try
			{
				BufferIndex = DecoderHandle.dequeueInputBuffer(InTimeoutUsec);
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to dequeue input buffer");
				e.printStackTrace();
				return -10000;
			}
			return BufferIndex;
		}
		else
		{
			return -10001;
		}
	}


	public int QueueInputBuffer(int InBufferIndex, long InTimestampUSec, byte[] InAccessUnit)
	{
		if (bIsInitialized)
		{
			if (InBufferIndex >= 0)
			{
				try
				{
					int SizeOfAU = InAccessUnit.length;
					ByteBuffer Buffer = DecoderHandle.getInputBuffer(InBufferIndex);
					Buffer.put(InAccessUnit);
					DecoderHandle.queueInputBuffer(InBufferIndex, 0, SizeOfAU, InTimestampUSec, 0);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderAudioAAC: Failed to queue input buffer");
					e.printStackTrace();
					return -10000;
				}
				return 0;
			}
			else
			{
				return -10002;
			}
		}
		else
		{
			return -10001;
		}
	}


	public int QueueEOSInputBuffer(int InBufferIndex, long InTimestampUSec)
	{
		if (bIsInitialized)
		{
			if (InBufferIndex >= 0)
			{
				try
				{
					ByteBuffer Buffer = DecoderHandle.getInputBuffer(InBufferIndex);
					Buffer.clear();
					DecoderHandle.queueInputBuffer(InBufferIndex, 0, 0, InTimestampUSec, MediaCodec.BUFFER_FLAG_END_OF_STREAM);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderAudioAAC: Failed to queue EOS input buffer");
					e.printStackTrace();
					return -10000;
				}
				return 0;
			}
			else
			{
				return -10002;
			}
		}
		else
		{
			return -10001;
		}
	}


	public FOutputFormatInfo GetOutputFormatInfo(int InOutputBufferIndex)
	{
		if (bIsInitialized)
		{
			if (InOutputBufferIndex < 0)
			{
				return CurrentOutputFormatInfo;
			}
			else
			{
				MediaFormat Format = DecoderHandle.getOutputFormat(InOutputBufferIndex);
				if (Format != null)
				{
					FOutputFormatInfo NewFormatInfo = new FOutputFormatInfo();
					NewFormatInfo.SampleRate	 = Format.getInteger(MediaFormat.KEY_SAMPLE_RATE);
					NewFormatInfo.NumChannels    = Format.getInteger(MediaFormat.KEY_CHANNEL_COUNT);
					NewFormatInfo.BytesPerSample = 2;
					if (Format.containsKey(MediaFormat.KEY_PCM_ENCODING))
					{
						boolean bIsFloat = Format.getInteger(MediaFormat.KEY_PCM_ENCODING) == AudioFormat.ENCODING_PCM_FLOAT;
						if (bIsFloat)
						{
							NewFormatInfo.BytesPerSample = 4;
						}
					}
					return NewFormatInfo;
				}
			}
		}
		return null;
	}


	public FOutputBufferInfo DequeueOutputBuffer(int InTimeoutUsec)
	{
		if (bIsInitialized)
		{
			FOutputBufferInfo OutputBufferInfo = new FOutputBufferInfo();
			MediaCodec.BufferInfo Info = new MediaCodec.BufferInfo();
			try
			{
				OutputBufferInfo.BufferIndex = DecoderHandle.dequeueOutputBuffer(Info, InTimeoutUsec);
				// If there is output we need to remember the buffer metadata for later.
				if (OutputBufferInfo.BufferIndex >= 0)
				{
					OutputBufferInfo.PresentationTimestamp = Info.presentationTimeUs;
					OutputBufferInfo.Size = Info.size;
					OutputBufferInfo.bIsEOS = (Info.flags & MediaCodec.BUFFER_FLAG_END_OF_STREAM) != 0;
					OutputBufferInfo.bIsConfig = (Info.flags & MediaCodec.BUFFER_FLAG_CODEC_CONFIG) != 0;
					OutputBufferInfos.put(OutputBufferInfo.BufferIndex, Info);
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_TRY_AGAIN_LATER)				// -1
				{
					// Have no output yet.
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_OUTPUT_FORMAT_CHANGED)			// -2
				{
					MediaFormat Format = DecoderHandle.getOutputFormat();
					GameActivity.Log.debug("Android Audio Decoder: New format " + Format);

					// Mwahahaaa... this only works from API level 29 onward...
					//		boolean bIsFloat = Format.getInteger(MediaFormat.KEY_PCM_ENCODING, AudioFormat.ENCODING_PCM_16BIT) == AudioFormat.ENCODING_PCM_FLOAT;

					FOutputFormatInfo NewFormatInfo = new FOutputFormatInfo();
					NewFormatInfo.SampleRate	 = Format.getInteger(MediaFormat.KEY_SAMPLE_RATE);
					NewFormatInfo.NumChannels    = Format.getInteger(MediaFormat.KEY_CHANNEL_COUNT);
					//NewFormatInfo.BytesPerSample = bIsFloat ? 4 : 2;
					NewFormatInfo.BytesPerSample = 2;
					if (Format.containsKey(MediaFormat.KEY_PCM_ENCODING))
					{
						boolean bIsFloat = Format.getInteger(MediaFormat.KEY_PCM_ENCODING) == AudioFormat.ENCODING_PCM_FLOAT;
						if (bIsFloat)
						{
							NewFormatInfo.BytesPerSample = 4;
						}
					}
					CurrentOutputFormatInfo = NewFormatInfo;
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_OUTPUT_BUFFERS_CHANGED)		// -3
				{
					// This is deprecated with API level 21 and of no real value to us.
				}
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderAudioAAC: Failed to dequeue output buffer");
				e.printStackTrace();
				return null;
			}
			return OutputBufferInfo;
		}
		return null;
	}


	public byte[] GetOutputBufferAndRelease(int InOutputBufferIndex)
	{
		byte[] Data = null;
		if (bIsInitialized && InOutputBufferIndex >= 0)
		{
			MediaCodec.BufferInfo Info = OutputBufferInfos.get(InOutputBufferIndex);
			if (Info != null)
			{
				OutputBufferInfos.delete(InOutputBufferIndex);

				try
				{
					ByteBuffer outBuffer = DecoderHandle.getOutputBuffer(InOutputBufferIndex);
					Data = new byte[Info.size];
					outBuffer.get(Data);
					DecoderHandle.releaseOutputBuffer(InOutputBufferIndex, false);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderAudioAAC: Failed to get output buffer");
					e.printStackTrace();
					return null;
				}
			}
		}
		return Data;
	}
}

