// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.annotation.SuppressLint;
import android.annotation.TargetApi;

import android.os.Build;
import java.io.IOException;
import java.util.ArrayList;
import android.util.Log;
import android.util.SparseArray;

import android.view.Surface;
import android.media.AudioFormat;
import android.media.MediaCodecList;
import android.media.MediaCodec;
import android.media.MediaCodecInfo;
import android.media.MediaCodecInfo.CodecCapabilities;
import android.media.MediaCodecInfo.CodecProfileLevel;
import android.media.MediaFormat;
import java.nio.ByteBuffer;

import com.epicgames.unreal.ElectraAndroidCodecQuirks;



/**
 * VP8/VP9 video decoder
 *
 */
public class ElectraDecoderVideoVPx
{
	private static final String TAG = "[ElectraDecoders]";
	/**
	 * Parameters for creating a new decoder instance.
	 * NOTE: Make sure this is a static inner class so it can be
	 *       created from C++ through JNI.
	 */
	public static class FCreateParameters
	{
		public int MaxWidth;
		public int MaxHeight;
		public int MaxFPS;
		public int Width;
		public int Height;
		public boolean bNeedSecure;
		public boolean bNeedTunneling;
		public byte[] CSD0;
		public int NativeDecoderID;
		public android.view.Surface VideoCodecSurface;
		public boolean bSurfaceIsView;
		public FCreateParameters()
		{
			MaxWidth = 0;
			MaxHeight = 0;
			Width = 0;
			Height = 0;
			MaxFPS = 0;
			bNeedSecure = false;
			bNeedTunneling = false;
			NativeDecoderID = 0;
			VideoCodecSurface = null;
		}
	}

	public static class FDecoderInformation
	{
		public int ApiLevel = 0;
		public boolean bIsAdaptive = false;
		public boolean bCanUse_SetOutputSurface = false;
	}

	public class FOutputFormatInfo
	{
		public int Width = 0;
		public int Height = 0;
		public int CropTop = 0;
		public int CropBottom = 0;
		public int CropLeft = 0;
		public int CropRight = 0;
		public int Stride = 0;
		public int SliceHeight = 0;
		public int ColorFormat = 0;
	}

	public class FOutputBufferInfo
	{
		public int BufferIndex = -1;
		public long PresentationTimestamp = -1;
		public int Size = 0;
		public boolean bIsEOS = false;
		public boolean bIsConfig = false;
	}

	private static class FCodecInformation
	{
		MediaCodecInfo Info;
		MediaCodecInfo.CodecCapabilities Caps;
	}


	private static class FVPxDecoderInfo
	{
		private static final FVPxDecoderInfo Singleton = new FVPxDecoderInfo();
		private FCodecInformation CodecInfoVP8;
		private FCodecInformation CodecInfoVP9;
		private FVPxDecoderInfo()
		{
			CodecInfoVP8 = selectCodec(MediaFormat.MIMETYPE_VIDEO_VP8);
			CodecInfoVP9 = selectCodec(MediaFormat.MIMETYPE_VIDEO_VP9);
		}

		private static final boolean DoesDecoderSupportsLowLatency(MediaCodecInfo decoderInfo, String mimeType)
		{
			if (Build.VERSION.SDK_INT >= 30)
			{
				try
				{
					if (decoderInfo.getCapabilitiesForType(mimeType).isFeatureSupported("low-latency"))
					{
						return true;
					}
				}
				catch (Exception e)
				{
				}
			}
			return false;
		}

		private static final FCodecInformation selectCodec(String mimeType)
		{
			/*
			If we knew what to put into the media format exactly we could perhaps use findDecoderForFormat().
				@see: https://developer.android.com/reference/android/media/MediaCodec.html#creation
			Since we do not have an extractor to get the track format from, after all we are decoding an elementary stream here,
			we have no idea what would normally be found and what properties are _required_ in the format.

				MediaFormat testFmt = new MediaFormat();
				String thisOne = cl.findDecoderForFormat(testFmt);
			*/

			MediaCodecList cl = new MediaCodecList(MediaCodecList.ALL_CODECS);
			MediaCodecInfo[] codecInfos = cl.getCodecInfos();
			int numCodecs = codecInfos.length;
//			for(int pass=0; pass<2; ++pass)
			for(int pass=0; pass<1; ++pass)
			{
				for(int i=0; i<numCodecs; ++i)
				{
					if (codecInfos[i].isEncoder())
					{
						continue;
					}

					// If this is an alias for another decoder (starting with API level 29, aka "Q") we skip it.
					if (Build.VERSION.SDK_INT >= 29)
					{
						if (codecInfos[i].isAlias())
						{
							continue;
						}
					}

					String[] types = codecInfos[i].getSupportedTypes();
					for(int j=0; j<types.length; ++j)
					{
						if (types[j].equalsIgnoreCase(mimeType))
						{
							// Apparently decoders may exists twice, one that doesn't support low latency and another that does.
							// And the one that does is not placed in front of the one that doesn't, so we need to check twice.
							boolean bSupportsLowLatency = DoesDecoderSupportsLowLatency(codecInfos[i], mimeType);

							//Log.w(TAG, "["+pass+"] ElectraDecoderVideoVPx: " + codecInfos[i].getCanonicalName() + "; ll="+bSupportsLowLatency);
/*
							if (pass==0 && !bSupportsLowLatency)
							{
								continue;
							}
*/
							// Rumor has it that the codec list is sorted from "best" to "worst".
							// Assuming this to be true we just return the first one matching the MIME type.
							FCodecInformation Info = new FCodecInformation();
							Info.Info = codecInfos[i];
							Info.Caps = codecInfos[i].getCapabilitiesForType(types[j]);
							return Info;
						}
					}
				}
			}
			return null;
		}

		public static FCodecInformation GetDecoderInfo(boolean bForVP9)
		{
			if (bForVP9 == true)
			{
				return Singleton.CodecInfoVP9;
			}
			else
			{
				return Singleton.CodecInfoVP8;
			}
		}
	}


	private FCodecInformation PlatformCodecInfo;
	private ElectraAndroidCodecQuirks.FDecoderQuirks PlatformCodecQuirks;

	private FCreateParameters CreationParameters;
	private FDecoderInformation DecoderInformation;
	private MediaCodec DecoderHandle;
	private boolean bIsVP9;
	private boolean bIsInitialized;

	private SparseArray<MediaCodec.BufferInfo> OutputBufferInfos;
	private FOutputFormatInfo CurrentOutputFormatInfo;
	private boolean bDecoderHasSurface;


	public ElectraDecoderVideoVPx()
	{
		PlatformCodecInfo = null;
		PlatformCodecQuirks =  ElectraAndroidCodecQuirks.GetDecoderQuirks();
		CreationParameters = null;
		DecoderHandle = null;
		OutputBufferInfos = new SparseArray<>();
		CurrentOutputFormatInfo = new FOutputFormatInfo();
		bDecoderHasSurface = false;
		bIsVP9 = true;
		bIsInitialized = false;

		// Create the initial decoder information, which at this point contains the quirks only.
		DecoderInformation = new FDecoderInformation();
		SetupDecoderQuirks();
	}

	private void SetupDecoderQuirks()
	{
		if (DecoderInformation != null && PlatformCodecQuirks != null)
		{
			DecoderInformation.ApiLevel = Build.VERSION.SDK_INT;
			DecoderInformation.bCanUse_SetOutputSurface = PlatformCodecQuirks.bCanUse_SetOutputSurface;
		}
	}

	public int CreateDecoder(boolean bCreateVP9)
	{
		// Clear out everything in case this is called repeatedly.
		DecoderHandle = null;
		DecoderInformation = null;
		bIsVP9 = bCreateVP9;
		bIsInitialized = false;
		OutputBufferInfos = new SparseArray<>();
		CurrentOutputFormatInfo = new FOutputFormatInfo();
		PlatformCodecInfo = FVPxDecoderInfo.GetDecoderInfo(bIsVP9);

		try
		{
			if (PlatformCodecInfo != null)
			{
				String DecoderName = PlatformCodecInfo.Info.getName();

				// This is how we get information on the decoder's capabilities.
				// For now we do not use those. We merely assume the decoder is capable of what we want.
				// Caution must be exercised with profile/level values as they are defined with other values in Android than they are by the standard!
				// Eg. High profile per the standard is identified by the value 100 while Android defines it as 8 (MediaCodecInfo.CodecProfileLevel.AVCProfileHigh)
				boolean bAdaptive   						= PlatformCodecInfo.Caps.isFeatureSupported(MediaCodecInfo.CodecCapabilities.FEATURE_AdaptivePlayback);
				//boolean bSecure 							= PlatformCodecInfo.Caps.isFeatureSupported(MediaCodecInfo.CodecCapabilities.FEATURE_SecurePlayback);
				//boolean bTunneled   						= PlatformCodecInfo.Caps.isFeatureSupported(MediaCodecInfo.CodecCapabilities.FEATURE_TunneledPlayback);
				//int [] ColorFmts							= PlatformCodecInfo.Caps.colorFormats;
				//MediaCodecInfo.CodecProfileLevel [] ProfLev = PlatformCodecInfo.Caps.profileLevels;

				// Create the decoder
				DecoderHandle = MediaCodec.createByCodecName(DecoderName);
				if (DecoderHandle != null)
				{
					// Remember the decoder capabilities
					DecoderInformation = new FDecoderInformation();
					DecoderInformation.bIsAdaptive = bAdaptive;
					SetupDecoderQuirks();
				}
				else
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to create decoder by name");
					return 1;
				}
			}
			else
			{
				// No decoder found.
				Log.w(TAG, "ElectraDecoderVideoVPx: No suitable decoder found");
				return 1;
			}
		}
		catch(Exception e)
		{
			Log.w(TAG, "ElectraDecoderVideoVPx: Failed to create decoder");
			e.printStackTrace();
			return 1;
		}
		return 0;
	}


	public int ConfigureDecoder(FCreateParameters InCreateParams)
	{
		if (DecoderHandle == null || DecoderInformation == null || PlatformCodecInfo == null)
		{
			Log.w(TAG, "ElectraDecoderVideoVPx: No decoder instance to configure has been created yet");
			return 1;
		}

		try
		{
			CreationParameters = InCreateParams;

			// Create decoder configuration object
			int MaxWidth  =  InCreateParams.MaxWidth > 0 ? InCreateParams.MaxWidth  : 1920;
			int MaxHeight = InCreateParams.MaxHeight > 0 ? InCreateParams.MaxHeight : 1088;
			int Width  = InCreateParams.Width > 0  ? InCreateParams.Width : MaxWidth;
			int Height = InCreateParams.Height > 0 ? InCreateParams.Height : MaxHeight;
			MediaFormat Format = MediaFormat.createVideoFormat(bIsVP9 ? MediaFormat.MIMETYPE_VIDEO_VP9 : MediaFormat.MIMETYPE_VIDEO_VP8, Width, Height);
			// Add additional keys
			if (DecoderInformation.bIsAdaptive)
			{
				// See: https://developer.android.com/reference/android/media/MediaFormat.html
				Format.setInteger(MediaFormat.KEY_MAX_WIDTH, MaxWidth);
				Format.setInteger(MediaFormat.KEY_MAX_HEIGHT, MaxHeight);
			}
			else
			{
				// Non-adaptive decoders need the CSD set in the configuration.
				if (InCreateParams.CSD0 != null && InCreateParams.CSD0.length != 0)
				{
					ByteBuffer Csd0 = ByteBuffer.wrap(InCreateParams.CSD0);
					Format.setByteBuffer("csd-0", Csd0);
				}
			}
			// Ask for low-latency.
//			Format.setInteger("low-latency", 1);


			// Configure the decoder
			android.view.Surface OutputSurface = null;
			OutputSurface = InCreateParams.VideoCodecSurface;
			bDecoderHasSurface = (OutputSurface != null);
			DecoderHandle.configure(Format, OutputSurface, null, 0);
			bIsInitialized = true;

			// We are not storing any outside surface reference (we only hand it through to the decoder)
			CreationParameters.VideoCodecSurface = null;
		}
		catch(Exception e)
		{
			Log.w(TAG, "ElectraDecoderVideoVPx: Failed to configure the decoder");
			e.printStackTrace();
			return 1;
		}
		return 0;
	}


	@TargetApi(23)
	public int SetOutputSurface(android.view.Surface NewVideoCodecSurface)
	{
		if (Build.VERSION.SDK_INT >= 23 && bIsInitialized && NewVideoCodecSurface != null)
		{
			try
			{
				bDecoderHasSurface = true;
				DecoderHandle.setOutputSurface(NewVideoCodecSurface);
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to set decoder output surface");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int ReleaseDecoder()
	{
		if (bIsInitialized)
		{
			bIsInitialized = false;

			try
			{
				// Release the decoder
				DecoderHandle.release();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to release decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int release()
	{
		ReleaseDecoder();
		return 0;
	}


	public int Start()
	{
		if (bIsInitialized)
		{
			try
			{
				// Start the decoder
				DecoderHandle.start();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to start decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Stop()
	{
		if (bIsInitialized)
		{
			try
			{
				// Stop the decoder
				DecoderHandle.stop();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to stop decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Flush()
	{
		if (bIsInitialized)
		{
			try
			{
				// Flush the decoder
				DecoderHandle.flush();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to flush decoder");
				e.printStackTrace();
				return 1;
			}

			// Clear the current infos by creating new empty structures.
			OutputBufferInfos = new SparseArray<>();
			// NOTE: Flushing does not invalidate the current output configuration. Do not clear it.
			// CurrentOutputFormatInfo = new FOutputFormatInfo();
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int Reset()
	{
		if (bIsInitialized)
		{
			try
			{
				// Reset the decoder
				DecoderHandle.reset();
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to reset decoder");
				e.printStackTrace();
				return 1;
			}
			return 0;
		}
		else
		{
			return 1;
		}
	}


	public int DequeueInputBuffer(int InTimeoutUsec)
	{
		if (bIsInitialized)
		{
			int BufferIndex = -1;
			try
			{
				BufferIndex = DecoderHandle.dequeueInputBuffer(InTimeoutUsec);
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to dequeue input buffer");
				e.printStackTrace();
				return -10000;
			}
			return BufferIndex;
		}
		else
		{
			return -10001;
		}
	}


	public int QueueInputBuffer(int InBufferIndex, long InTimestampUSec, byte[] InAccessUnit)
	{
		if (bIsInitialized)
		{
			if (InBufferIndex >= 0)
			{
				try
				{
					int SizeOfAU = InAccessUnit.length;
					ByteBuffer Buffer = DecoderHandle.getInputBuffer(InBufferIndex);
					Buffer.put(InAccessUnit);
					DecoderHandle.queueInputBuffer(InBufferIndex, 0, SizeOfAU, InTimestampUSec, 0);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to queue input buffer");
					e.printStackTrace();
					return -10000;
				}
				return 0;
			}
			else
			{
				return -10002;
			}
		}
		else
		{
			return -10001;
		}
	}


	public int QueueCSDInputBuffer(int InBufferIndex, long InTimestampUSec, byte[] InCSD)
	{
		if (bIsInitialized)
		{
			if (InBufferIndex >= 0)
			{
				try
				{
					int SizeOfCSD = InCSD.length;
					ByteBuffer Buffer = DecoderHandle.getInputBuffer(InBufferIndex);
					Buffer.put(InCSD);
					DecoderHandle.queueInputBuffer(InBufferIndex, 0, SizeOfCSD, InTimestampUSec, MediaCodec.BUFFER_FLAG_CODEC_CONFIG);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to queue CSD input buffer");
					e.printStackTrace();
					return -10000;
				}
				return 0;
			}
			else
			{
				return -10002;
			}
		}
		else
		{
			return -10001;
		}
	}


	public int QueueEOSInputBuffer(int InBufferIndex, long InTimestampUSec)
	{
		if (bIsInitialized)
		{
			if (InBufferIndex >= 0)
			{
				try
				{
					ByteBuffer Buffer = DecoderHandle.getInputBuffer(InBufferIndex);
					Buffer.clear();
					DecoderHandle.queueInputBuffer(InBufferIndex, 0, 0, InTimestampUSec, MediaCodec.BUFFER_FLAG_END_OF_STREAM);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to queue EOS input buffer");
					e.printStackTrace();
					return -10000;
				}
				return 0;
			}
			else
			{
				return -10002;
			}
		}
		else
		{
			return -10001;
		}
	}


	public FDecoderInformation GetDecoderInformation()
	{
		return DecoderInformation;
	}


	private FOutputFormatInfo GetOutputFormatInfoFromFormat(MediaFormat Format)
	{
		FOutputFormatInfo NewFormatInfo = new FOutputFormatInfo();
		if (Format.containsKey(MediaFormat.KEY_WIDTH))
		{
			NewFormatInfo.Width = Format.getInteger(MediaFormat.KEY_WIDTH);
		}
		if (Format.containsKey(MediaFormat.KEY_HEIGHT))
		{
			NewFormatInfo.Height = Format.getInteger(MediaFormat.KEY_HEIGHT);
		}
		if (Format.containsKey("slice-height"))	// MediaFormat.KEY_SLICE_HEIGHT  with API level 23 only...
		{
			NewFormatInfo.SliceHeight = Format.getInteger("slice-height");
		}
		if (Format.containsKey("stride"))	// MediaFormat.KEY_STRIDE  with API level 23 only...
		{
			NewFormatInfo.Stride = Format.getInteger("stride");
		}
		if (Format.containsKey(MediaFormat.KEY_COLOR_FORMAT))
		{
			NewFormatInfo.ColorFormat = Format.getInteger(MediaFormat.KEY_COLOR_FORMAT);
		}
		// See: https://developer.android.com/reference/android/media/MediaCodec#accessing-raw-video-bytebuffers-on-older-devices
		if (Format.containsKey("crop-left") && Format.containsKey("crop-right") && Format.containsKey("crop-top") && Format.containsKey("crop-bottom"))
		{
			NewFormatInfo.CropLeft   = Format.getInteger("crop-left");
			NewFormatInfo.CropRight  = Format.getInteger("crop-right");
			NewFormatInfo.CropTop    = Format.getInteger("crop-top");
			NewFormatInfo.CropBottom = Format.getInteger("crop-bottom");
		}
		else
		{
			NewFormatInfo.CropLeft   = 0;
			NewFormatInfo.CropTop    = 0;
			NewFormatInfo.CropRight  = NewFormatInfo.Width - 1;
			NewFormatInfo.CropBottom = NewFormatInfo.Height - 1;
		}
		return NewFormatInfo;
	}


	public FOutputFormatInfo GetOutputFormatInfo(int InOutputBufferIndex)
	{
		if (InOutputBufferIndex < 0)
		{
			return CurrentOutputFormatInfo;
		}
		if (bIsInitialized)
		{
			MediaFormat Format = DecoderHandle.getOutputFormat(InOutputBufferIndex);
			return GetOutputFormatInfoFromFormat(Format);
		}
		return null;
	}


	public FOutputBufferInfo DequeueOutputBuffer(int InTimeoutUsec)
	{
		if (bIsInitialized)
		{
			FOutputBufferInfo OutputBufferInfo = new FOutputBufferInfo();
			MediaCodec.BufferInfo Info = new MediaCodec.BufferInfo();
			try
			{
				OutputBufferInfo.BufferIndex = DecoderHandle.dequeueOutputBuffer(Info, InTimeoutUsec);
				// If there is output we need to remember the buffer metadata for later.
				if (OutputBufferInfo.BufferIndex >= 0)
				{
					OutputBufferInfo.PresentationTimestamp = Info.presentationTimeUs;
					OutputBufferInfo.Size = Info.size;
					OutputBufferInfo.bIsEOS = (Info.flags & MediaCodec.BUFFER_FLAG_END_OF_STREAM) != 0;
					OutputBufferInfo.bIsConfig = (Info.flags & MediaCodec.BUFFER_FLAG_CODEC_CONFIG) != 0;
					OutputBufferInfos.put(OutputBufferInfo.BufferIndex, Info);
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_TRY_AGAIN_LATER)				// -1
				{
					// Have no output yet.
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_OUTPUT_FORMAT_CHANGED)			// -2
				{
					MediaFormat Format = DecoderHandle.getOutputFormat();
					CurrentOutputFormatInfo = GetOutputFormatInfoFromFormat(Format);
				}
				else if (OutputBufferInfo.BufferIndex == MediaCodec.INFO_OUTPUT_BUFFERS_CHANGED)		// -3
				{
					// This is deprecated with API level 21 and of no real value to us.
				}
			}
			catch(Exception e)
			{
				Log.w(TAG, "ElectraDecoderVideoVPx: Failed to dequeue output buffer");
				e.printStackTrace();
				return null;
			}
			return OutputBufferInfo;
		}
		return null;
	}


	public byte[] GetOutputBuffer(int InOutputBufferIndex)
	{
		byte[] Data = null;
		if (bIsInitialized && InOutputBufferIndex >= 0)
		{
			MediaCodec.BufferInfo Info = OutputBufferInfos.get(InOutputBufferIndex);
			if (Info != null)
			{
				try
				{
					ByteBuffer outBuffer = DecoderHandle.getOutputBuffer(InOutputBufferIndex);
					Data = new byte[Info.size];
					outBuffer.get(Data);
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to get output buffer");
					e.printStackTrace();
					return null;
				}
			}
		}
		return Data;
	}


	public int ReleaseOutputBuffer(int InOutputBufferIndex, boolean bRender, long releaseAt)
	{
		if (bIsInitialized && InOutputBufferIndex >= 0)
		{
			MediaCodec.BufferInfo Info = OutputBufferInfos.get(InOutputBufferIndex);
			if (Info != null)
			{
				OutputBufferInfos.delete(InOutputBufferIndex);

				FOutputFormatInfo OutFmtInfo = GetOutputFormatInfo(InOutputBufferIndex);

				// If the decoder has no output surface we cannot release the buffer to it!
				bRender &= bDecoderHasSurface;

				try
				{
					boolean bDoRender = bRender == true && (Info.size != 0) && ((Info.flags & MediaCodec.BUFFER_FLAG_CODEC_CONFIG) != MediaCodec.BUFFER_FLAG_CODEC_CONFIG);
					if (releaseAt < 0 || bDoRender == false)
					{
						if (bDoRender && CreationParameters.bSurfaceIsView)
						{
							// We use a timed output instead of the un-timed one because that one seems to use PTS values gathered from the sample(??) that of course do
							// not correclate to "realtime within 1s" - and trigger warnings in logcat...
							DecoderHandle.releaseOutputBuffer(InOutputBufferIndex, System.nanoTime());
						}
						else
						{
							DecoderHandle.releaseOutputBuffer(InOutputBufferIndex, bDoRender);
						}
					}
					else
					{
						DecoderHandle.releaseOutputBuffer(InOutputBufferIndex, releaseAt);
					}
				}
				catch(Exception e)
				{
					Log.w(TAG, "ElectraDecoderVideoVPx: Failed to release buffer to surface");
					e.printStackTrace();
					return 1;
				}
				return 0;
			}
			else
			{
				return -10000;
			}
		}
		return 1;
	}
}

