// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IElectraCodecFactory.h"

class FAACAudioDecoderAndroid
{
public:
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();

};
