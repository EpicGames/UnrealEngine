// Copyright Epic Games, Inc. All Rights Reserved.

#include "aac/AudioDecoderAAC_Android.h"
#include "DecoderErrors_Android.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputAudio.h"

#include "ElectraDecodersModule.h"
/*********************************************************************************************************************/

#include "AudioDecoderAAC_JavaWrapper_Android.h"

/*********************************************************************************************************************/

#include "HAL/IConsoleManager.h"
namespace PlatformElectraDecodersAndroid
{
extern TAutoConsoleVariable<bool> CVarElectraDecoderAndroidRebasePTS;
}

/*********************************************************************************************************************/
#define ENABLE_DETAILED_LOG 0
#if ENABLE_DETAILED_LOG
#define DETAILLOG UE_LOG
#else
#define DETAILLOG(CategoryName, Verbosity, Format, ...) while(0){}
#endif

#define DESTROY_DECODER_WHEN_FLUSHING 1
/*********************************************************************************************************************/

class FElectraAudioDecoderOutputAAC_Android : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputAAC_Android()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	int32 GetNumChannels() const override
	{ return NumChannels; }
	int32 GetSampleRate() const override
	{ return SampleRate; }
	int32 GetNumFrames() const override
	{ return NumFrames; }
	bool IsInterleaved() const override
	{ return true; }
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid; }
	ESampleFormat GetSampleFormat() const override
	{ return ESampleFormat::Float; }
	int32 GetBytesPerSample() const override
	{ return sizeof(float); }
	int32 GetBytesPerFrame() const override
	{ return GetBytesPerSample() * GetNumChannels(); }
	const void* GetData(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr; }

public:
	enum
	{
		kMaxChannels = 8
	};
	EChannelPosition ChannelPositions[kMaxChannels];
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};



class FElectraAudioDecoderAAC_Android : public IElectraAudioDecoderAAC_Android
{
public:
	FElectraAudioDecoderAAC_Android(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraAudioDecoderAAC_Android();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Audio; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override;
	void Resume() override;

private:
	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		int64 DecoderPTS = -1;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> GetCSDFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();
	bool InternalDecoderDrain();

	bool ConvertDecoderOutput(const IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo& InInfo);

	IElectraDecoder::FError LastError;

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> ConfigRecord;
	TArray<FDecoderInput> InDecoderInput;
	TSharedPtr<FElectraAudioDecoderOutputAAC_Android, ESPMode::ThreadSafe> CurrentOutput;

	TSharedPtr<IElectraAACAudioDecoderAndroidJava, ESPMode::ThreadSafe> DecoderInstance;
	TOptional<int64> DecoderPTSOffset;
	int64 LastPushedPresentationTimeUs = 0;
	bool bDidEnqueueInputBuffer = false;
	IElectraAACAudioDecoderAndroidJava::FOutputFormatInfo CurrentOutputFormatInfo;
	bool bIsOutputFormatInfoValid = false;
	bool bDidSendEOS = false;
	bool bNewDecoderRequired = false;

	EDecodeState DecodeState = EDecodeState::Decoding;

	static const uint8 NumChannelsForConfig[16];
	static const IElectraDecoderAudioOutput::EChannelPosition _1[1];
	static const IElectraDecoderAudioOutput::EChannelPosition _2[2];
	static const IElectraDecoderAudioOutput::EChannelPosition _3[3];
	static const IElectraDecoderAudioOutput::EChannelPosition _4[4];
	static const IElectraDecoderAudioOutput::EChannelPosition _5[5];
	static const IElectraDecoderAudioOutput::EChannelPosition _6[6];
	static const IElectraDecoderAudioOutput::EChannelPosition _7[7];
	static const IElectraDecoderAudioOutput::EChannelPosition _8[8];
	static const IElectraDecoderAudioOutput::EChannelPosition * const MostCommonChannelMapping[8];
};
const uint8 FElectraAudioDecoderAAC_Android::NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_1[1] = { IElectraDecoderAudioOutput::EChannelPosition::C };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_2[2] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_3[3] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_4[4] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::Cs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_5[5] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_6[6] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_7[7] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Cs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Android::_8[8] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Lsr, IElectraDecoderAudioOutput::EChannelPosition::Rsr };
const IElectraDecoderAudioOutput::EChannelPosition * const FElectraAudioDecoderAAC_Android::MostCommonChannelMapping[8] = { FElectraAudioDecoderAAC_Android::_1, FElectraAudioDecoderAAC_Android::_2, FElectraAudioDecoderAAC_Android::_3, FElectraAudioDecoderAAC_Android::_4, FElectraAudioDecoderAAC_Android::_5, FElectraAudioDecoderAAC_Android::_6, FElectraAudioDecoderAAC_Android::_7, FElectraAudioDecoderAAC_Android::_8 };


void IElectraAudioDecoderAAC_Android::GetConfigurationOptions(TMap<FString, FVariant>& OutFeatures)
{
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraAudioDecoderAAC_Android::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraAudioDecoderAAC_Android, ESPMode::ThreadSafe> New = MakeShared<FElectraAudioDecoderAAC_Android>(InCodecFormat, InAdditionalOptions);
	return New;
}


FElectraAudioDecoderAAC_Android::FElectraAudioDecoderAAC_Android(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.CSD.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->Parse(InCodecFormat.CSD))
		{
			ConfigRecord = MoveTemp(NewConfigRecord);
		}
	}
}


FElectraAudioDecoderAAC_Android::~FElectraAudioDecoderAAC_Android()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}


TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Android::GetCSDFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord;
		NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->ParseFrom(SidebandData.GetData(), SidebandData.Num()))
		{
			return NewConfigRecord;
		}
		else
		{
			LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
			LastError.Message = TEXT("Failed to parse codec specific data");
		}
	}
	return nullptr;
}


void FElectraAudioDecoderAAC_Android::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	OutFeatures.Emplace(IElectraDecoderFeature::MustBeSuspendedInBackground, FVariant(true));
}


IElectraDecoder::FError FElectraAudioDecoderAAC_Android::GetError() const
{
	return LastError;
}


void FElectraAudioDecoderAAC_Android::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}


IElectraDecoder::ECSDCompatibility FElectraAudioDecoderAAC_Android::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraAudioDecoderAAC_Android::ResetToCleanStart()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::ResetToCleanStart()"));
	InternalDecoderDestroy();

	ConfigRecord.Reset();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;
	bDidSendEOS = false;
	bIsOutputFormatInfoValid = false;
	DecoderPTSOffset.Reset();
	LastPushedPresentationTimeUs = 0;
	bDidEnqueueInputBuffer = false;

	return !LastError.IsSet();
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Android::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode call just provides new configuration?
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) != EElectraDecoderFlags::None)
	{
		ConfigRecord.Reset();
		InternalDecoderDestroy();
	}

	// Need a valid CSD to create a decoder.
	if (!ConfigRecord.IsValid())
	{
		ConfigRecord = GetCSDFromOptions(InAdditionalOptions);
		if (!ConfigRecord.IsValid())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Create decoder transform if necessary.
	if (bNewDecoderRequired)
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::DecodeAccessUnit() - require new decoder, destroying..."));
		InternalDecoderDestroy();
	}
	if (!DecoderInstance.IsValid() && !InternalDecoderCreate())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		int32 InputBufferIndex = DecoderInstance->DequeueInputBuffer(0);
		if (InputBufferIndex >= 0)
		{
			// The decoder could have issues when the PTS is large as is common in a Live stream.
			// We rebase it to work around this issue.
			int64 PTSTicks = InInputAccessUnit.PTS.GetTicks();
			if (!DecoderPTSOffset.IsSet())
			{
				DecoderPTSOffset = PlatformElectraDecodersAndroid::CVarElectraDecoderAndroidRebasePTS.GetValueOnAnyThread() ? PTSTicks : 0;
				DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::DecodeAccessUnit() - Rebase PTS offset %lld"), (long long int)DecoderPTSOffset.GetValue());
			}
			PTSTicks -= DecoderPTSOffset.GetValue();
			// Also the decoder internally truncates nanoseconds to microseconds which it then
			// sticks onto the decoded output, so we cannot use the PTS as a "user value".
			// We truncate the value ourselves to account for this.
			PTSTicks /= 10;		// Convert from HNS to microseconds

			DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::DecodeAccessUnit() - queue input %lld (%lld)"), (long long int)PTSTicks, (long long int)InInputAccessUnit.PTS.GetTicks());
			int32 Result = DecoderInstance->QueueInputBuffer(InputBufferIndex, InInputAccessUnit.Data, InInputAccessUnit.DataSize, PTSTicks);
			if (Result == 0)
			{
				bDidEnqueueInputBuffer = true;
				LastPushedPresentationTimeUs = PTSTicks;

				// Add to the list of inputs passed to the decoder.
				FDecoderInput In;
				In.AdditionalOptions = InAdditionalOptions;
				In.AccessUnit = InInputAccessUnit;
				In.DecoderPTS = PTSTicks;
				InDecoderInput.Emplace(MoveTemp(In));
				InDecoderInput.Sort([](const FDecoderInput& e1, const FDecoderInput& e2)
				{
					return e1.AccessUnit.PTS < e2.AccessUnit.PTS;
				});

				return IElectraDecoder::EDecoderError::None;
			}
			else
			{
				PostError(Result, TEXT("Failed to decode audio decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (InputBufferIndex == -1)
		{
			// No available input buffer. Try later.
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
		else
		{
			PostError(InputBufferIndex, TEXT("Failed to get a decoder input buffer"), ERRCODE_INTERNAL_DID_NOT_GET_INPUT_BUFFER);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Android::SendEndOfData()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::SendEndOfData()"));
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::SendEndOfData() - already sent"));
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance.IsValid())
	{
		if (!InternalDecoderDrain())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Android::Flush()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::Flush()"));
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

#ifdef DESTROY_DECODER_WHEN_FLUSHING
	ResetToCleanStart();

#else
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Flush();
	}
	DecodeState = EDecodeState::Decoding;
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecoderPTSOffset.Reset();
	LastPushedPresentationTimeUs = 0;
	bDidSendEOS = false;
	bDidEnqueueInputBuffer = false;
#endif
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FElectraAudioDecoderAAC_Android::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() -> Have"));
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !DecoderInstance.IsValid())
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() no / need new decoder"));
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	// Are we to drain the decoder?
	// We need an available input buffer to do this so we put it here where pulling output is sure to free
	// up an input buffer eventually.
	if (DecodeState == EDecodeState::Draining && !bDidSendEOS)
	{
		int32 Result = -1;
		int32 InputBufferIndex = DecoderInstance->DequeueInputBuffer(0);
		DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() Sending EOS"));
		if (InputBufferIndex >= 0)
		{
			Result = DecoderInstance->QueueEOSInputBuffer(InputBufferIndex, LastPushedPresentationTimeUs);
			if (Result == 0)
			{
				bDidSendEOS = true;
			}
			else
			{
				PostError(Result, TEXT("Failed to submit decoder EOS input buffer"), ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT);
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
		else if (InputBufferIndex == -1)
		{
			// No available input buffer. Try later.
		}
		else
		{
			PostError(InputBufferIndex, TEXT("Failed to get a decoder input buffer for EOS"), ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT);
			return IElectraDecoder::EOutputStatus::Error;
		}
	}

	// Check if there is available output from the decoder.
	while(1)
	{
		IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo OutputBufferInfo;
		int32 Result = DecoderInstance->DequeueOutputBuffer(OutputBufferInfo, 0);
		if (Result != 0)
		{
			PostError(Result, TEXT("Failed to get decoder output buffer"), ERRCODE_INTERNAL_DID_NOT_GET_OUTPUT_BUFFER);
			return IElectraDecoder::EOutputStatus::Error;
		}

		// Valid output buffer?
		if (OutputBufferInfo.BufferIndex >= 0)
		{
			// Is this a pure EOS buffer with no decoded data?
			if (OutputBufferInfo.bIsEOS && OutputBufferInfo.Size == 0)
			{
				DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() got EOS, %d still in?"), InDecoderInput.Num());
				// We have to release that buffer even if it's empty.
				void* DummyData = FMemory::Malloc(16384);
				DecoderInstance->GetOutputBufferAndRelease(DummyData, 16384, OutputBufferInfo);
				FMemory::Free(DummyData);

				// Did we ask for an EOS?
				if (DecodeState == EDecodeState::Draining)
				{
					DecodeState = EDecodeState::Decoding;
					bDidSendEOS = false;
					InDecoderInput.Empty();
					// The decoder will not be accepting new input after an EOS until it gets flushed.
					// Since it will be drained mostly because of a format change we rather create a new one.
					bNewDecoderRequired = true;
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				// On some Android versions there could be such empty buffers without us having requested an EOS.
				else
				{
					DETAILLOG(LogElectraDecoders, Log, TEXT("  -> UNEXPECTED, continuing!"));
					continue;
				}
			}

			if (ConvertDecoderOutput(OutputBufferInfo))
			{
				// Was there the EOS flag set and did we request it?
				if (OutputBufferInfo.bIsEOS && DecodeState == EDecodeState::Draining)
				{
					// This should not happen. But assuming that it does then return EOS instead.
					DecodeState = EDecodeState::Decoding;
					bDidSendEOS = false;
					InDecoderInput.Empty();
					CurrentOutput.Reset();
					bNewDecoderRequired = true;
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				return IElectraDecoder::EOutputStatus::Available;
			}
			else
			{
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
		else if (OutputBufferInfo.BufferIndex == IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_TRY_AGAIN_LATER)
		{
			if (DecodeState == EDecodeState::Draining)
			{
				DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() - DRAINING - TRY AGAIN LATER!?!"));
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}
			DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() - TRY AGAIN LATER!?!"));
			return IElectraDecoder::EOutputStatus::NeedInput;
		}
		else if (OutputBufferInfo.BufferIndex == IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_OUTPUT_FORMAT_CHANGED)
		{
			IElectraAACAudioDecoderAndroidJava::FOutputFormatInfo OutputFormatInfo;
			Result = DecoderInstance->GetOutputFormatInfo(OutputFormatInfo, -1);
			DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::HaveOutput() - Output format changed!"));
			if (Result == 0)
			{
				CurrentOutputFormatInfo = OutputFormatInfo;
				bIsOutputFormatInfoValid = CurrentOutputFormatInfo.NumChannels > 0 && CurrentOutputFormatInfo.SampleRate > 0;
			}
			else
			{
				PostError(Result, TEXT("Failed to get decoder output format"), ERRCODE_INTERNAL_COULD_NOT_GET_OUTPUT_FORMAT);
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
		else if (OutputBufferInfo.BufferIndex == IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_OUTPUT_BUFFERS_CHANGED)
		{
			// No-op as this is the Result of a deprecated API we are not using.
		}
		else
		{
			PostError(Result, TEXT("Unhandled output buffer index value"), ERRCODE_INTERNAL_UNEXPECTED_PROBLEM);
			return IElectraDecoder::EOutputStatus::Error;
		}
	}
	check(!"should never get here!");
	return IElectraDecoder::EOutputStatus::Error;
}


TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Android::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}


void FElectraAudioDecoderAAC_Android::Suspend()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::Suspend()"));
}


void FElectraAudioDecoderAAC_Android::Resume()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::Resume()"));
}


bool FElectraAudioDecoderAAC_Android::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}


bool FElectraAudioDecoderAAC_Android::InternalDecoderCreate()
{
	bDidSendEOS = false;
	bIsOutputFormatInfoValid = false;
	bNewDecoderRequired = false;
	bDidEnqueueInputBuffer = false;
	DecoderInstance = IElectraAACAudioDecoderAndroidJava::Create();
	int32 Result = DecoderInstance->InitializeDecoder(*ConfigRecord);
	if (Result)
	{
		PostError(Result, TEXT("Failed to create decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return false;
	}
	Result = DecoderInstance->Start();
	if (Result)
	{
		PostError(Result, TEXT("Failed to start decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return false;
	}
	return true;
}


bool FElectraAudioDecoderAAC_Android::InternalDecoderDrain()
{
	bDidSendEOS = false;
	return true;
}


void FElectraAudioDecoderAAC_Android::InternalDecoderDestroy()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::InternalDecoderDestroy()"));
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Flush();
		DecoderInstance->Stop();
		DecoderInstance->ReleaseDecoder();
		DecoderInstance.Reset();
	}
}




bool FElectraAudioDecoderAAC_Android::ConvertDecoderOutput(const IElectraAACAudioDecoderAndroidJava::FOutputBufferInfo& InInfo)
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("AudioDecoderAAC::ConvertDecoderOutput()"));
	// We take the frontmost entry of decoder inputs. The data is not expected to be decoded out of order.
	// We also remove the frontmost entry even if conversion of the output actually fails.
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	FDecoderInput MatchingInput = InDecoderInput[0];
	InDecoderInput.RemoveAt(0);

	TSharedPtr<FElectraAudioDecoderOutputAAC_Android, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraAudioDecoderOutputAAC_Android>();
	NewOutput->PTS = MatchingInput.AccessUnit.PTS;
	NewOutput->UserValue = MatchingInput.AccessUnit.UserValue;

	IElectraAACAudioDecoderAndroidJava::FOutputFormatInfo OutputFormatInfo;
	int32 Result = DecoderInstance->GetOutputFormatInfo(OutputFormatInfo, InInfo.BufferIndex);
	if (Result == 0)
	{
		NewOutput->SampleRate = OutputFormatInfo.SampleRate;
		NewOutput->NumChannels = OutputFormatInfo.NumChannels;
		check(NewOutput->NumChannels <= 8);
		for(uint32 i=0; i<NewOutput->NumChannels; ++i)
		{
			NewOutput->ChannelPositions[i] = MostCommonChannelMapping[NewOutput->NumChannels-1][i];
		}
		NewOutput->NumFrames = InInfo.Size / (OutputFormatInfo.NumChannels * OutputFormatInfo.BytesPerSample);

		// Allocate buffer and convert from int16 to float if necessary.
		int32 BufferSize = NewOutput->NumFrames * NewOutput->NumChannels * sizeof(float);
		NewOutput->Buffer = (float*)FMemory::Malloc(BufferSize);
		void* Start = reinterpret_cast<void*>(NewOutput->Buffer);
		Result = DecoderInstance->GetOutputBufferAndRelease(Start, BufferSize, InInfo);
		if (Result == 0)
		{
			if (OutputFormatInfo.BytesPerSample == 2)
			{
				const int16* SourceEnd = reinterpret_cast<const int16*>(reinterpret_cast<const int8_t*>(NewOutput->Buffer) + InInfo.Size);
				float* DestEnd = NewOutput->Buffer + NewOutput->NumFrames * NewOutput->NumChannels;
				while(DestEnd > NewOutput->Buffer)
				{
					*(--DestEnd) = *(--SourceEnd) / 32768.0f;
				}
			}
			CurrentOutput = MoveTemp(NewOutput);
			return true;
		}
		else
		{
			PostError(Result, TEXT("Failed to get data and release buffer"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}
	}
	else
	{
		PostError(Result, TEXT("Failed to get buffer format info"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
}
