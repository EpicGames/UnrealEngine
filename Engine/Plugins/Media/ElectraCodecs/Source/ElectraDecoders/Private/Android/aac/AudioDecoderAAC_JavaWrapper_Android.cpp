// Copyright Epic Games, Inc. All Rights Reserved.

#include "aac/AudioDecoderAAC_JavaWrapper_Android.h"

#include "HAL/Platform.h"

#if (defined(USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY) && USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY != 0) || USE_ANDROID_JNI
#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"

#if UE_BUILD_SHIPPING
// always clear any exceptions in shipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id)								\
	if (Id == 0)											\
	{														\
		if (!bIsOptional)									\
		{													\
			JEnv->ExceptionDescribe();						\
			checkf(Id != 0, TEXT("Failed to find " #Id));	\
		}													\
		JEnv->ExceptionClear();								\
	}
#endif


class FElectraAACAudioDecoderAndroidJava : public IElectraAACAudioDecoderAndroidJava, public FJavaClassObject
{
public:
	FElectraAACAudioDecoderAndroidJava();
	virtual ~FElectraAACAudioDecoderAndroidJava();

	virtual int32 InitializeDecoder(const ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord& InParsedConfigurationRecord) override;
	virtual int32 ReleaseDecoder() override;
	virtual int32 Start() override;
	virtual int32 Stop() override;
	virtual int32 Flush() override;
	virtual int32 DequeueInputBuffer(int32 InTimeoutUsec) override;
	virtual int32 QueueInputBuffer(int32 InBufferIndex, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec) override;
	virtual int32 QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec) override;
	virtual int32 GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex) override;
	virtual int32 DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec) override;
	virtual int32 GetOutputBufferAndRelease(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo) override;

private:
	static FName GetClassName()
	{
		return FName("com/epicgames/unreal/ElectraDecoderAudioAAC");
	}

	jfieldID FindField(JNIEnv* JEnv, jclass InClass, const ANSICHAR* InFieldName, const ANSICHAR* InFieldType, bool bIsOptional)
	{
		jfieldID Field = InClass == nullptr ? nullptr : JEnv->GetFieldID(InClass, InFieldName, InFieldType);
		CHECK_JNI_RESULT(Field);
		return Field;
	}

	// Create a Java byte array. Must DeleteLocalRef() after use or handling over to Java.
	jbyteArray MakeJavaByteArray(const uint8* InData, int32 InNumBytes)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jbyteArray RawBuffer = JEnv->NewByteArray(InNumBytes);
		JEnv->SetByteArrayRegion(RawBuffer, 0, InNumBytes, reinterpret_cast<const jbyte*>(InData));
		return RawBuffer;
	}

	template<typename ReturnType>
	ReturnType CallMethodNoVerify(FJavaClassMethod Method, ...);

	template<>
	void CallMethodNoVerify<void>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		JEnv->CallVoidMethodV(Object, Method.Method, Params);
		va_end(Params);
	}

	template<>
	bool CallMethodNoVerify<bool>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		bool RetVal = JEnv->CallBooleanMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	int CallMethodNoVerify<int>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		int RetVal = JEnv->CallIntMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	jobject CallMethodNoVerify<jobject>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
		va_end(Params);
		jobject RetVal = JEnv->NewGlobalRef(val);
		JEnv->DeleteLocalRef(val);
		return RetVal;
	}

	template<>
	jobjectArray CallMethodNoVerify<jobjectArray>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
		va_end(Params);
		jobjectArray RetVal = (jobjectArray)JEnv->NewGlobalRef(val);
		JEnv->DeleteLocalRef(val);
		return RetVal;
	}

	template<>
	int64 CallMethodNoVerify<int64>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		int64 RetVal = JEnv->CallLongMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	FString CallMethodNoVerify<FString>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jstring RetVal = static_cast<jstring>(
			JEnv->CallObjectMethodV(Object, Method.Method, Params));
		va_end(Params);
		auto Result = FJavaHelper::FStringFromLocalRef(JEnv, RetVal);
		return Result;
	}

	// Java methods
	FJavaClassMethod	CreateDecoderFN;
	FJavaClassMethod	ReleaseDecoderFN;
	FJavaClassMethod	StartFN;
	FJavaClassMethod	StopFN;
	FJavaClassMethod	FlushFN;
	FJavaClassMethod	DequeueInputBufferFN;
	FJavaClassMethod	QueueInputBufferFN;
	FJavaClassMethod	QueueEOSInputBufferFN;
	FJavaClassMethod	GetOutputFormatInfoFN;
	FJavaClassMethod	DequeueOutputBufferFN;
	FJavaClassMethod	GetOutputBufferAndReleaseFN;

	// FOutputFormatInfo member field IDs
	jclass				FOutputFormatInfoClass;
	jfieldID			FOutputFormatInfo_SampleRate;
	jfieldID			FOutputFormatInfo_NumChannels;
	jfieldID			FOutputFormatInfo_BytesPerSample;

	// FOutputBufferInfo member field IDs
	jclass				FOutputBufferInfoClass;
	jfieldID			FOutputBufferInfo_BufferIndex;
	jfieldID			FOutputBufferInfo_PresentationTimestamp;
	jfieldID			FOutputBufferInfo_Size;
	jfieldID			FOutputBufferInfo_bIsEOS;
	jfieldID			FOutputBufferInfo_bIsConfig;

	// Internal state
	bool				bHaveDecoder;
	bool				bIsStarted;

};


//-----------------------------------------------------------------------------
/**
 * Create a Java audio decoder wrapper.
 *
 * @return Decoder wrapper instance.
 */
TSharedPtr<IElectraAACAudioDecoderAndroidJava, ESPMode::ThreadSafe> IElectraAACAudioDecoderAndroidJava::Create()
{
	return MakeShared<FElectraAACAudioDecoderAndroidJava>();
}


//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FElectraAACAudioDecoderAndroidJava::FElectraAACAudioDecoderAndroidJava()
	: FJavaClassObject(GetClassName(), "()V")
	, CreateDecoderFN(GetClassMethod("CreateDecoder", "(II[B)I"))
	, ReleaseDecoderFN(GetClassMethod("ReleaseDecoder", "()I"))
	, StartFN(GetClassMethod("Start", "()I"))
	, StopFN(GetClassMethod("Stop", "()I"))
	, FlushFN(GetClassMethod("Flush", "()I"))
	, DequeueInputBufferFN(GetClassMethod("DequeueInputBuffer", "(I)I"))
	, QueueInputBufferFN(GetClassMethod("QueueInputBuffer", "(IJ[B)I"))
	, QueueEOSInputBufferFN(GetClassMethod("QueueEOSInputBuffer", "(IJ)I"))
	, GetOutputFormatInfoFN(GetClassMethod("GetOutputFormatInfo", "(I)Lcom/epicgames/unreal/ElectraDecoderAudioAAC$FOutputFormatInfo;"))
	, DequeueOutputBufferFN(GetClassMethod("DequeueOutputBuffer", "(I)Lcom/epicgames/unreal/ElectraDecoderAudioAAC$FOutputBufferInfo;"))
	, GetOutputBufferAndReleaseFN(GetClassMethod("GetOutputBufferAndRelease", "(I)[B"))
{
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();

	// Get field IDs for FOutputFormatInfo class members
	jclass localOutputFormatInfoClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderAudioAAC$FOutputFormatInfo");
	FOutputFormatInfoClass = (jclass)JEnv->NewGlobalRef(localOutputFormatInfoClass);
	JEnv->DeleteLocalRef(localOutputFormatInfoClass);
	FOutputFormatInfo_SampleRate     = FindField(JEnv, FOutputFormatInfoClass, "SampleRate", "I", false);
	FOutputFormatInfo_NumChannels    = FindField(JEnv, FOutputFormatInfoClass, "NumChannels", "I", false);
	FOutputFormatInfo_BytesPerSample = FindField(JEnv, FOutputFormatInfoClass, "BytesPerSample", "I", false);

	// Get field IDs for FOutputBufferInfo class members
	jclass localOutputBufferInfoClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderAudioAAC$FOutputBufferInfo");
	FOutputBufferInfoClass = (jclass)JEnv->NewGlobalRef(localOutputBufferInfoClass);
	JEnv->DeleteLocalRef(localOutputBufferInfoClass);
	FOutputBufferInfo_BufferIndex   		= FindField(JEnv, FOutputBufferInfoClass, "BufferIndex", "I", false);
	FOutputBufferInfo_PresentationTimestamp = FindField(JEnv, FOutputBufferInfoClass, "PresentationTimestamp", "J", false);
	FOutputBufferInfo_Size  				= FindField(JEnv, FOutputBufferInfoClass, "Size", "I", false);
	FOutputBufferInfo_bIsEOS				= FindField(JEnv, FOutputBufferInfoClass, "bIsEOS", "Z", false);
	FOutputBufferInfo_bIsConfig 			= FindField(JEnv, FOutputBufferInfoClass, "bIsConfig", "Z", false);

	// Internal state
	bHaveDecoder = false;
	bIsStarted   = false;
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FElectraAACAudioDecoderAndroidJava::~FElectraAACAudioDecoderAndroidJava()
{
	Stop();
	ReleaseDecoder();

	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	JEnv->DeleteGlobalRef(FOutputBufferInfoClass);
	JEnv->DeleteGlobalRef(FOutputFormatInfoClass);
}


//-----------------------------------------------------------------------------
/**
 * Creates and initializes an instance of an AAC audio decoder.
 *
 * @param
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::InitializeDecoder(const ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord& InParsedConfigurationRecord)
{
	// Already have a decoder?
	if (bHaveDecoder)
	{
		// Do not care about potential error return values here.
		Stop();
		ReleaseDecoder();
		bHaveDecoder = false;
	}

	// Create one
	const uint8 NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };
	JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
	jbyteArray  CSD			= MakeJavaByteArray(InParsedConfigurationRecord.GetCodecSpecificData().GetData(), InParsedConfigurationRecord.GetCodecSpecificData().Num());
	int 		NumChannels = (int) InParsedConfigurationRecord.PSSignal == 1 ? 2 : NumChannelsForConfig[InParsedConfigurationRecord.ChannelConfiguration];
	int			SampleRate  = (int)(InParsedConfigurationRecord.ExtSamplingFrequency ? InParsedConfigurationRecord.ExtSamplingFrequency : InParsedConfigurationRecord.SamplingRate);
	int32 result = CallMethodNoVerify<int>(CreateDecoderFN, NumChannels, SampleRate, CSD);
	JEnv->DeleteLocalRef(CSD);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
	}
	if (result != 0)
	{
		return result;
	}
	bHaveDecoder = true;
	return 0;
}


//-----------------------------------------------------------------------------
/**
 * Releases (destroys) the Java audio decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::ReleaseDecoder()
{
	if (bHaveDecoder)
	{
		int32 result = CallMethodNoVerify<int>(ReleaseDecoderFN);
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		bHaveDecoder = false;
		return result;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Starts the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::Start()
{
	if (bHaveDecoder && !bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(StartFN);
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		if (result)
		{
			return result;
		}
		bIsStarted = true;
		return 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Stops the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::Stop()
{
	if (bHaveDecoder && bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(StopFN);
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		if (result)
		{
			return result;
		}
		bIsStarted = false;
		return 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Flushes the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::Flush()
{
	// Synchronously operating decoders must be in the started state to be flushed.
	if (bHaveDecoder && bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(FlushFN);
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		return result;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Dequeues an input buffer.
 *
 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
 *
 * @return >= 0 returns the index of the successfully dequeued buffer, negative values indicate an error.
 */
int32 FElectraAACAudioDecoderAndroidJava::DequeueInputBuffer(int32 InTimeoutUsec)
{
	if (bHaveDecoder)
	{
		int32 result = CallMethodNoVerify<int>(DequeueInputBufferFN, InTimeoutUsec);
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		return result;
	}
	return -1;
}


//-----------------------------------------------------------------------------
/**
 * Queues input for decoding in the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
 *
 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
 * @param InAccessUnitData Data to be decoded.
 * @param InAccessUnitSize Size of the data to be decoded.
 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::QueueInputBuffer(int32 InBufferIndex, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec)
{
	if (bHaveDecoder)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		jbyteArray  InData = MakeJavaByteArray((const uint8*)InAccessUnitData, InAccessUnitSize);
		int32 result = CallMethodNoVerify<int>(QueueInputBufferFN, InBufferIndex, (jlong)InTimestampUSec, InData);
		JEnv->DeleteLocalRef(InData);
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		return result;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Queues end of stream for the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
 *
 * @param InBufferIndex Index of the buffer to put the EOS flag into and enqueue for decoding (see DequeueInputBuffer()).
 * @param InTimestampUSec Timestamp the previous data had. Can be 0.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec)
{
	if (bHaveDecoder)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		int32 result = CallMethodNoVerify<int>(QueueEOSInputBufferFN, InBufferIndex, (jlong)InTimestampUSec);
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Returns format information of the decoded samples.
 *
 * @param OutFormatInfo
 * @param InOutputBufferIndex RESERVED FOR NOW - Pass any negative value to get the output format after DequeueOutputBuffer() returns a BufferIndex of MediaCodec_INFO_OUTPUT_FORMAT_CHANGED.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraAACAudioDecoderAndroidJava::GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex)
{
	if (bHaveDecoder)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		jobject OutputInfo = JEnv->CallObjectMethod(Object, GetOutputFormatInfoFN.Method, InOutputBufferIndex);
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		// Failure will return no object.
		if (OutputInfo != nullptr)
		{
			OutFormatInfo.SampleRate	 = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_SampleRate);
			OutFormatInfo.NumChannels    = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_NumChannels);
			OutFormatInfo.BytesPerSample = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_BytesPerSample);
			JEnv->DeleteLocalRef(OutputInfo);
			return 0;
		}
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Dequeues an output buffer.
 *
 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
 *
 * @return 0 on success, 1 on failure. The OutBufferInfo.BufferIndex indicates the buffer index.
 */
int32 FElectraAACAudioDecoderAndroidJava::DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec)
{
	if (bHaveDecoder)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		jobject OutputInfo = JEnv->CallObjectMethod(Object, DequeueOutputBufferFN.Method, InTimeoutUsec);
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
		// Failure will return no object.
		if (OutputInfo != nullptr)
		{
			OutBufferInfo.BufferIndex   		= JEnv->GetIntField(OutputInfo, FOutputBufferInfo_BufferIndex);
			OutBufferInfo.PresentationTimestamp = JEnv->GetLongField(OutputInfo, FOutputBufferInfo_PresentationTimestamp);
			OutBufferInfo.Size  				= JEnv->GetIntField(OutputInfo, FOutputBufferInfo_Size);
			OutBufferInfo.bIsEOS				= JEnv->GetBooleanField(OutputInfo, FOutputBufferInfo_bIsEOS);
			OutBufferInfo.bIsConfig 			= JEnv->GetBooleanField(OutputInfo, FOutputBufferInfo_bIsConfig);
			JEnv->DeleteLocalRef(OutputInfo);
			return 0;
		}
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Returns the decoded samples from a decoder output buffer in the decoder native format!
 * Check the output format first to check if the format uses 2 bytes per sample (int16) or 4 (float)
 * and the number of channels.
 *
 * @param OutBufferDataPtr
 * @param OutBufferDataSize
 * @param InOutBufferInfo
 *
 * @return 0 on success, 1 on failure.
 */
int32 FElectraAACAudioDecoderAndroidJava::GetOutputBufferAndRelease(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo)
{
	if (bHaveDecoder)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		jbyteArray RawDecoderArray = (jbyteArray)JEnv->CallObjectMethod(Object, GetOutputBufferAndReleaseFN.Method, InOutBufferInfo.BufferIndex);
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}

		if (RawDecoderArray != nullptr)
		{
			jbyte* RawDataPtr = JEnv->GetByteArrayElements(RawDecoderArray, 0);
			int32 RawBufferSize = JEnv->GetArrayLength(RawDecoderArray);

			check(RawBufferSize == InOutBufferInfo.Size);
			check(RawBufferSize <= OutBufferDataSize)
			if (RawDataPtr != nullptr)
			{
				memcpy(OutBufferDataPtr, RawDataPtr, RawBufferSize <= OutBufferDataSize ? RawBufferSize : OutBufferDataSize);
				JEnv->ReleaseByteArrayElements(RawDecoderArray, RawDataPtr, JNI_ABORT);
			}

			JEnv->DeleteLocalRef(RawDecoderArray);

			return 0;
		}
		else
		{
			// Error!
		}
	}
	return 1;
}



#else

class FElectraAACAudioDecoderAndroidJava : public IElectraAACAudioDecoderAndroidJava
{
public:
	FElectraAACAudioDecoderAndroidJava()
	{ }
	virtual ~FElectraAACAudioDecoderAndroidJava()
	{ }

	virtual int32 InitializeDecoder(const MPEG::FAACDecoderConfigurationRecord& InParsedConfigurationRecord) override
	{ return 1; }

	virtual int32 ReleaseDecoder() override
	{ return 1; }

	virtual int32 Start() override
	{ return 1; }

	virtual int32 Stop() override
	{ return 1; }

	virtual int32 Flush() override
	{ return 1; }

	virtual int32 DequeueInputBuffer(int32 InTimeoutUsec) override
	{ return -1; }

	virtual int32 QueueInputBuffer(int32 InBufferIndex, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec) override
	{ return 1; }

	virtual int32 QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec) override
	{ return 1; }

	virtual int32 GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex) override
	{ return 1; }

	virtual int32 DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec) override
	{ return 1; }

	virtual int32 GetOutputBufferAndRelease(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo) override
	{ return 1; }
};

TSharedPtr<IElectraAACAudioDecoderAndroidJava, ESPMode::ThreadSafe> IElectraAACAudioDecoderAndroidJava::Create()
{
	return MakeShared<FElectraAACAudioDecoderAndroidJava>();
}

#endif

