// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HAL/Platform.h"

#if (defined(USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY) && USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY != 0) || USE_ANDROID_JNI
#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"

#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"


class IElectraAACAudioDecoderAndroidJava
{
public:
	static TSharedPtr<IElectraAACAudioDecoderAndroidJava, ESPMode::ThreadSafe> Create();

	virtual ~IElectraAACAudioDecoderAndroidJava() = default;

	struct FOutputFormatInfo
	{
		FOutputFormatInfo()
		{
			Clear();
		}

		void Clear()
		{
			SampleRate = 0;
			NumChannels = 0;
			BytesPerSample = 0;
		}

		bool IsValid() const
		{
			return SampleRate != 0 && NumChannels != 0;
		}

		int32	SampleRate;
		int32	NumChannels;
		int32	BytesPerSample;
	};

	struct FOutputBufferInfo
	{
		enum EBufferIndexValues
		{
			MediaCodec_INFO_TRY_AGAIN_LATER 	   = -1,
			MediaCodec_INFO_OUTPUT_FORMAT_CHANGED  = -2,
			MediaCodec_INFO_OUTPUT_BUFFERS_CHANGED = -3
		};

		FOutputBufferInfo()
		{
			Clear();
		}

		void Clear()
		{
			BufferIndex 		  = -1;
			PresentationTimestamp = -1;
			Size				  = 0;
			bIsEOS  			  = false;
			bIsConfig   		  = false;
		}

		int32	BufferIndex;
		int64	PresentationTimestamp;
		int32	Size;
		bool	bIsEOS;
		bool	bIsConfig;
	};

	/**
	 * Creates and initializes a Java instance of an AAC audio decoder.
	 *
	 * @param
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 InitializeDecoder(const ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord& InParsedConfigurationRecord) = 0;

	/**
	 * Releases (destroys) the Java audio decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 ReleaseDecoder() = 0;

	/**
	 * Starts the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Start() = 0;

	/**
	 * Stops the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Stop() = 0;

	/**
	 * Flushes the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Flush() = 0;

	/**
	 * Dequeues an input buffer.
	 *
	 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
	 *
	 * @return >= 0 returns the index of the successfully dequeued buffer, negative values indicate an error.
	 */
	virtual int32 DequeueInputBuffer(int32 InTimeoutUsec) = 0;

	/**
	 * Queues input for decoding in the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
	 *
	 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
	 * @param InAccessUnitData Data to be decoded.
	 * @param InAccessUnitSize Size of the data to be decoded.
	 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 QueueInputBuffer(int32 InBufferIndex, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec) = 0;

	/**
	 * Queues end of stream for the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
	 *
	 * @param InBufferIndex Index of the buffer to put the EOS flag into and enqueue for decoding (see DequeueInputBuffer()).
	 * @param InTimestampUSec Timestamp the previous data had. Can be 0.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec) = 0;

	/**
	 * Returns format information of the decoded samples.
	 *
	 * @param OutFormatInfo
	 * @param InOutputBufferIndex RESERVED FOR NOW - Pass any negative value to get the output format after DequeueOutputBuffer() returns a BufferIndex of MediaCodec_INFO_OUTPUT_FORMAT_CHANGED.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex) = 0;

	/**
	 * Dequeues an output buffer.
	 *
	 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
	 *
	 * @return 0 on success, 1 on failure. The OutBufferInfo.BufferIndex indicates the buffer index.
	 */
	virtual int32 DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec) = 0;

	/**
	 * Returns the decoded samples from a decoder output buffer in the decoder native format!
	 * Check the output format first to check if the format uses 2 bytes per sample (int16) or 4 (float)
	 * and the number of channels.
	 *
	 * @param OutBufferDataPtr
	 * @param OutBufferDataSize
	 * @param InOutBufferInfo
	 *
	 * @return 0 on success, 1 on failure.
	 */
	virtual int32 GetOutputBufferAndRelease(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo) = 0;

};

#endif
