// Copyright Epic Games, Inc. All Rights Reserved.

#include "h264/H264_VideoDecoder_Android.h"
#include "h264/VideoDecoderH264_Android.h"

#include "ElectraDecodersUtils.h"


class FH264VideoDecoderFactoryAndroid : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FH264VideoDecoderFactoryAndroid()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderH264_Android::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','1') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','3'))
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vid = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		const uint32 Width = vid.Width;
		const uint32 Height = vid.Height;
		// Baseline, Main or High profile only.
		if (vid.Profile.Profile != 66 && vid.Profile.Profile != 77 && vid.Profile.Profile != 100)
		{
			return 0;
		}
		// Limit to 1080p for now (orientation-independent, so portrait sources are not rejected).
		if (ElectraDecodersUtil::ExceedsMaxDimensions(Width, Height, 1920, 1088))
		{
			return 0;
		}
		return 1;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderH264_Android::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("Android")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FH264VideoDecoderAndroid::CreateFactory()
{
	return MakeShared<FH264VideoDecoderFactoryAndroid, ESPMode::ThreadSafe>();
}
