// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HAL/Platform.h"

#if (defined(USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY) && USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY != 0) || USE_ANDROID_JNI
#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"

#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"

class IElectraH264VideoDecoderAndroidJava
{
public:

	static TSharedPtr<IElectraH264VideoDecoderAndroidJava, ESPMode::ThreadSafe> Create();

	virtual ~IElectraH264VideoDecoderAndroidJava() = default;

	struct FCreateParameters
	{
		int32 MaxWidth = 0;
		int32 MaxHeight = 0;
		int32 MaxProfile = 0;
		int32 MaxProfileLevel = 0;
		int32 MaxFrameRate= 0;
		const ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord* CodecData = nullptr;
		uint32 NativeDecoderID = 0;
		bool bUseVideoCodecSurface = false;
		jobject VideoCodecSurface = nullptr; // global reference expected
		bool bSurfaceIsView = false;
	};

	struct FDecoderInformation
	{
		int32 ApiLevel = 0;
		bool bIsAdaptive = false;
		bool bCanUse_SetOutputSurface = false;
	};

	struct FOutputFormatInfo
	{
		FOutputFormatInfo()
		{
			Clear();
		}

		void Clear()
		{
			Width = 0;
			Height = 0;
			CropTop = 0;
			CropBottom = 0;
			CropLeft = 0;
			CropRight = 0;
			Stride = 0;
			SliceHeight = 0;
			ColorFormat = 0;
		}

		bool IsValid() const
		{
			return Width != 0 && Height != 0;
		}

		int32 Width;
		int32 Height;
		int32 CropTop;
		int32 CropBottom;
		int32 CropLeft;
		int32 CropRight;
		int32 Stride;
		int32 SliceHeight;
		int32 ColorFormat;
	};

	struct FOutputBufferInfo
	{
		enum EBufferIndexValues
		{
			MediaCodec_INFO_TRY_AGAIN_LATER = -1,
			MediaCodec_INFO_OUTPUT_FORMAT_CHANGED = -2,
			MediaCodec_INFO_OUTPUT_BUFFERS_CHANGED = -3
		};

		FOutputBufferInfo()
		{
			Clear();
		}

		void Clear()
		{
			PresentationTimestamp = -1;
			BufferIndex = -1;
			Size = 0;
			ValidCount = -1;
			bIsEOS = false;
			bIsConfig = false;
		}

		int64 PresentationTimestamp;
		int32 BufferIndex;
		int32 Size;
		int32 ValidCount;
		bool bIsEOS;
		bool bIsConfig;
	};


	/**
	 * Creates a Java instance of an H.264 video decoder.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 CreateDecoder() = 0;

	/**
	 * Initializes the video decoder instance.
	 *
	 * @param InCreateParams
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 InitializeDecoder(const FCreateParameters& InCreateParams) = 0;

	/**
	 * Attempts to set a new output surface on an existing and configured decoder.
	 *
	 * @param InNewOutputSurface
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 SetOutputSurface(jobject InNewOutputSurface) = 0;

	/**
	 * Releases (destroys) the Java video decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 ReleaseDecoder() = 0;

	/**
	 * Returns decoder information after a successful InitializeDecoder().
	 *
	 * @return Pointer to decoder information or null when no decoder has been created.
	 */
	virtual const FDecoderInformation* GetDecoderInformation() = 0;

	/**
	 * Starts the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Start() = 0;

	/**
	 * Stops the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Stop() = 0;

	/**
	 * Flushes the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Flush() = 0;

	/**
	 * Resets the decoder instance.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 Reset() = 0;

	/**
	 * Dequeues an input buffer.
	 *
	 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
	 *
	 * @return >= 0 returns the index of the successfully dequeued buffer, negative values indicate an error.
	 */
	virtual int32 DequeueInputBuffer(int32 InTimeoutUsec) = 0;

	/**
	 * Queues input for decoding in the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
	 *
	 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
	 * @param InPrefixData Data to prefix the data to decode with. If given it is usually the SPS & PPS.
	 * @param InPrefixSize Size of the prefix data.
	 * @param InAccessUnitData Data to be decoded.
	 * @param InAccessUnitSize Size of the data to be decoded.
	 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 QueueInputBuffer(int32 InBufferIndex, const void* InPrefixData, int32 InPrefixSize, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec) = 0;

	/**
	 * Queues codec specific data for the following to-be-decoded data buffers.
	 *
	 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
	 * @param InCSDData Codec specific data
	 * @param InCSDSize Size of the codec specific data
	 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds. Must be the same as the next data to be decoded.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 QueueCSDInputBuffer(int32 InBufferIndex, const void* InCSDData, int32 InCSDSize, int64 InTimestampUSec) = 0;

	/**
	 * Queues end of stream for the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
	 *
	 * @param InBufferIndex Index of the buffer to put the EOS flag into and enqueue for decoding (see DequeueInputBuffer()).
	 * @param InTimestampUSec Timestamp the previous data had. Can be 0.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec) = 0;

	/**
	 * Returns format information of the decoded samples.
	 *
	 * @param OutFormatInfo
	 * @param InOutputBufferIndex RESERVED FOR NOW - Pass any negative value to get the output format after DequeueOutputBuffer() returns a BufferIndex of MediaCodec_INFO_OUTPUT_FORMAT_CHANGED.
	 *
	 * @return 0 if successful, 1 on error.
	 */
	virtual int32 GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex) = 0;

	/**
	 * Dequeues an output buffer.
	 *
	 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
	 *
	 * @return 0 on success, 1 on failure. The OutBufferInfo.BufferIndex indicates the buffer index.
	 */
	virtual int32 DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec) = 0;

	/**
	 * Returns the decoded samples from a decoder output buffer in the decoder native format.
	 *
	 * @param OutBufferDataPtr
	 * @param OutBufferDataSize
	 * @param InOutBufferInfo
	 *
	 * @return 0 on success, 1 on failure.
	 */
	virtual int32 GetOutputBuffer(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo) = 0;

	/**
	 * Releases the decoder output buffer back to the decoder.
	 *
	 * @return 0 on success, 1 on failure.
	 */
	virtual int32 ReleaseOutputBuffer(int32 BufferIndex, int32 ValidCount, bool bRender, int64 releaseAt) = 0;
};

#endif
