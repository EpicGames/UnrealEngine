// Copyright Epic Games, Inc. All Rights Reserved.

#include "h265/VideoDecoderH265_Android.h"
#include "DecoderErrors_Android.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"
#include "Utils/ElectraCodecFormatUtils.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersPlatformResources.h"

#include "ElectraDecodersModule.h"

#include <atomic>
/*********************************************************************************************************************/

#include "h265/VideoDecoderH265_JavaWrapper_Android.h"

/*********************************************************************************************************************/

#include "HAL/IConsoleManager.h"
namespace PlatformElectraDecodersAndroid
{
extern TAutoConsoleVariable<bool> CVarElectraDecoderAndroidRebasePTS;
}

/*********************************************************************************************************************/
#define ENABLE_DETAILED_LOG 0
#if ENABLE_DETAILED_LOG
#define DETAILLOG UE_LOG
#else
#define DETAILLOG(CategoryName, Verbosity, Format, ...) while(0){}
#endif

#define DESTROY_DECODER_WHEN_FLUSHING 1

/*********************************************************************************************************************/

class FElectraVideoDecoderH265_Android;


class FElectraVideoDecoderOutputH265_Android : public IElectraDecoderVideoOutput
{
public:
	virtual ~FElectraVideoDecoderOutputH265_Android()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return Width; }
	int32 GetDecodedHeight() const override
	{ return Height; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{ return nullptr; }
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;
};



class FElectraVideoDecoderH265_Android : public IElectraVideoDecoderH265_Android
{
public:
	FElectraVideoDecoderH265_Android(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraVideoDecoderH265_Android();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorH265::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	class FSurfaceRequestCallback : public FElectraDecodersPlatformResources::IDecoderResourceInstance::ISurfaceRequestCallback
	{
	public:
		virtual ~FSurfaceRequestCallback() = default;
		bool HasCompleted() const
		{ return bIsSet; }
		ESurfaceType GetType() const
		{ return SurfaceType; }
		jobject GetSurface() const
		{ return Surface; }
	private:
		void OnNewSurface(ESurfaceType InSurfaceType, jobject InSurface)
		{
			SurfaceType = InSurfaceType;
			Surface = InSurface;
			FPlatformMisc::MemoryBarrier();
			bIsSet = true;
		}
		ESurfaceType SurfaceType = ESurfaceType::Error;
		jobject Surface = nullptr;
		volatile bool bIsSet = false;
	};

	struct FInitialMaxValues
	{
		int32 Width = 0;
		int32 Height = 0;
		uint32 Profile = 0;
		uint32 Level = 0;
		bool IsSet() const
		{
			return Width && Height && Profile && Level;
		}
	};

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> SPS;
		int64 DecoderPTS = -1;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining,
		CreatingDecoder
	};

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord, ESPMode::ThreadSafe> GetDCRFromOptions(const TMap<FString, FVariant>& InOptions);
	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> GetSPSFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate();
	bool InternalHandleDecoderCreate();
	bool InternalFinishDecoderCreate();
	void InternalDecoderDestroy();
	bool InternalDecoderDrain();

	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};
	EConvertResult ConvertDecoderOutput(const IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo& InInfo);

	IElectraDecoder::FError LastError;
	FInitialMaxValues InitialMaxValues;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	FElectraDecodersPlatformResources::IDecoderResourceInstance* ResourceInstance = nullptr;

	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord, ESPMode::ThreadSafe> ConfigRecord;
	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> CurrentSPS;
	TSharedPtr<FElectraVideoDecoderOutputH265_Android, ESPMode::ThreadSafe> CurrentOutput;
	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;

	TSharedPtr<IElectraH265VideoDecoderAndroidJava, ESPMode::ThreadSafe> DecoderInstance;
	IElectraH265VideoDecoderAndroidJava::FDecoderInformation DecoderInfo;
	EDecodeState DecodeState = EDecodeState::Decoding;
	TOptional<int64> DecoderPTSOffset;
	int64 LastPushedPresentationTime = -1;
	bool bDidSendEOS = false;
	bool bNewDecoderRequired = false;
	volatile uint32 DecoderCreatedAtResumeCount = 0;
	std::atomic<uint32> CurrentResumeCount;
	bool bTriggerReplay = false;

	TSharedPtr<FSurfaceRequestCallback, ESPMode::ThreadSafe> CurrentSurfaceRequest;
	uint32 NativeDecoderID = 0;

	static std::atomic<uint32> NextNativeDecoderID;
};
std::atomic<uint32>  FElectraVideoDecoderH265_Android::NextNativeDecoderID { 0 };


namespace IElectraVideoDecoderH265_Android_Platform
{
	static TArray<IElectraVideoDecoderH265_Android::FSupportedConfiguration> DecoderConfigurations;
	static bool bDecoderConfigurationsDirty = true;
}

void IElectraVideoDecoderH265_Android::PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations)
{
	if (IElectraVideoDecoderH265_Android_Platform::bDecoderConfigurationsDirty)
	{
		IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations.Empty();

		// Perhaps these can be determined dynamically for each type of device?
		// What is really supported isn't quite clear so allow for UHD.

		// Main
		IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Android::FSupportedConfiguration(0, 1, 0, 153, 60, 4096, 2304, 0));
		IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Android::FSupportedConfiguration(0, 1, 0, 153, 0, 0, 0, (3840 / 8) * (2160 / 8) * 60));
		// Main10
		IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Android::FSupportedConfiguration(0, 2, 0, 153, 60, 4096, 2304, 0));
		IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Android::FSupportedConfiguration(0, 2, 0, 153, 0, 0, 0, (3840 / 8) * (2160 / 8) * 60));

		IElectraVideoDecoderH265_Android_Platform::bDecoderConfigurationsDirty = false;
	}
	OutSupportedConfigurations = IElectraVideoDecoderH265_Android_Platform::DecoderConfigurations;
}

void IElectraVideoDecoderH265_Android::GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
{
	OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
	OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	OutOptions.Emplace(IElectraDecoderFeature::NeedReplayDataOnDecoderLoss, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::MustBeSuspendedInBackground, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraVideoDecoderH265_Android::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraVideoDecoderH265_Android, ESPMode::ThreadSafe> New = MakeShared<FElectraVideoDecoderH265_Android>(InCodecFormat, InAdditionalOptions);
	return New;
}


FElectraVideoDecoderH265_Android::FElectraVideoDecoderH265_Android(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	ResourceInstance = FElectraDecodersPlatformResources::CreateResourceInstance(ElectraDecodersUtil::GetVariantValueSafeI64(InitialCreationOptions, FElectraDecodersPlatformResources::OptionKeyOutputTexturePoolID().ToString(), 0));

	NativeDecoderID = ++NextNativeDecoderID;
	DecoderCreatedAtResumeCount = 0;
	CurrentResumeCount = 0;

	// Get the maximum values if provided.
	InitialMaxValues.Width = (int32) Align(ElectraDecodersUtil::GetVariantValueSafeI64(InitialCreationOptions, TEXT("max_width"), 0), 16);
	InitialMaxValues.Height = (int32) Align(ElectraDecodersUtil::GetVariantValueSafeI64(InitialCreationOptions, TEXT("max_height"), 0), 16);
	const FVariant* Var = InitialCreationOptions.Find(TEXT("max_codecprofile"));
	if (Var && Var->GetType() == EVariantTypes::String)
	{
		Electra::FCodecTypeFormat ctf;
		ctf.RFC6381 = Var->GetValue<FString>();
		if (FElectraCodecFormatUtilsModularFeature::SetupCodecTypeFormat(ctf))
		{
			const Electra::FCodecTypeFormat::FVideo& vid = ctf.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
			InitialMaxValues.Profile = vid.Profile.Profile;
			InitialMaxValues.Level = vid.Profile.Level;
		}
	}
}


FElectraVideoDecoderH265_Android::~FElectraVideoDecoderH265_Android()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();

	FElectraDecodersPlatformResources::ReleaseResourceInstance(ResourceInstance);
}


TSharedPtr<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord, ESPMode::ThreadSafe> FElectraVideoDecoderH265_Android::GetDCRFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("dcr"));
	if (SidebandData.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord;
		NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->Parse(SidebandData))
		{
			return NewConfigRecord;
		}
		else
		{
			LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
			LastError.Message = TEXT("Failed to parse codec specific data");
		}
	}
	return nullptr;
}

TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> FElectraVideoDecoderH265_Android::GetSPSFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
		ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, SidebandData.GetData(), SidebandData.Num());
		for(int32 i=0; i<NALUs.Num(); ++i)
		{
			if (NALUs[i].Type == 33)
			{
				TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = MakeShared<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe>();
				if (ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(*NewSPS, ElectraDecodersUtil::AdvancePointer(SidebandData.GetData(), NALUs[i].Offset + NALUs[i].UnitLength), NALUs[i].Size))
				{
					return NewSPS;
				}
				else
				{
					break;
				}
			}
		}
		PostError(0, TEXT("Failed to parse codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}
	return nullptr;
}


void FElectraVideoDecoderH265_Android::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
	OutFeatures.Emplace(IElectraDecoderFeature::MustBeSuspendedInBackground, FVariant(true));
}


IElectraDecoder::FError FElectraVideoDecoderH265_Android::GetError() const
{
	return LastError;
}


void FElectraVideoDecoderH265_Android::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}


IElectraDecoder::ECSDCompatibility FElectraVideoDecoderH265_Android::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!DecoderInstance.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}

	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = GetSPSFromOptions(CSDAndAdditionalOptions);
	if (!NewSPS.IsValid())
	{
		UE_LOGF(LogElectraDecoders, Warning, "No CSD provided to IsCompatibleWith(), returning to drain and reset.");
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}

	return DecoderInfo.bIsAdaptive ? IElectraDecoder::ECSDCompatibility::Compatible : IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraVideoDecoderH265_Android::ResetToCleanStart()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::ResetToCleanStart()"));
	InternalDecoderDestroy();

	ConfigRecord.Reset();
	CurrentSPS.Reset();
	CurrentOutput.Reset();
	InDecoderInput.Empty();
	DecodeState = EDecodeState::Decoding;
	DecoderPTSOffset.Reset();
	LastPushedPresentationTime = -1;
	bDidSendEOS = false;

	return !LastError.IsSet();
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Android::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// Still creating the decoder?
	if (DecodeState == EDecodeState::CreatingDecoder)
	{
		if (!InternalHandleDecoderCreate())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (DecodeState == EDecodeState::CreatingDecoder)
		{
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
	}

	// Need a valid CSD to create a decoder.
	if (!ConfigRecord.IsValid())
	{
		ConfigRecord = GetDCRFromOptions(InAdditionalOptions);
		if (!ConfigRecord.IsValid())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Resuming after suspending will re-create a decoder by replaying previous data.
	// This may not be necessary on all devices, but is a safe catch-all.
	if (DecoderCreatedAtResumeCount != CurrentResumeCount)
	{
		bTriggerReplay = true;
		CurrentOutput.Reset();
		InDecoderInput.Empty();
		DecoderPTSOffset.Reset();
		LastPushedPresentationTime = -1;
		bNewDecoderRequired = true;
	}

	// Create decoder transform if necessary.
	if (bNewDecoderRequired)
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::DecodeAccessUnit() - require new decoder, destroying..."));
		InternalDecoderDestroy();
	}
	if (!DecoderInstance.IsValid())
	{
		InternalDecoderCreate();
		return InternalHandleDecoderCreate() ? IElectraDecoder::EDecoderError::NoBuffer : IElectraDecoder::EDecoderError::Error;
	}

	if (bTriggerReplay)
	{
		bTriggerReplay = false;
		return IElectraDecoder::EDecoderError::LostDecoder;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		int32 InputBufferIndex = DecoderInstance->DequeueInputBuffer(0);
		if (InputBufferIndex >= 0)
		{
			// The decoder could have issues when the PTS is large as is common in a Live stream.
			// We rebase it to work around this issue.
			int64 PTSTicks = InInputAccessUnit.PTS.GetTicks();
			if (!DecoderPTSOffset.IsSet())
			{
				DecoderPTSOffset = PlatformElectraDecodersAndroid::CVarElectraDecoderAndroidRebasePTS.GetValueOnAnyThread() ? PTSTicks : 0;
				DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::DecodeAccessUnit() - Rebase PTS offset %lld"), (long long int)DecoderPTSOffset.GetValue());
			}
			PTSTicks -= DecoderPTSOffset.GetValue();
			// Also the decoder internally truncates nanoseconds to microseconds which it then
			// sticks onto the decoded output, so we cannot use the PTS as a "user value".
			// We truncate the value ourselves to account for this.
			PTSTicks /= 10;		// Convert from HNS to microseconds

			int32 Result = 0;
			// If this is a sync sample we prepend the CSD. While this is not necessary on a running stream we need to have the CSD
			// on the first frame and it is easier to prepend it to all IDR frames when seeking etc.
			if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
			{
				CurrentSPS = GetSPSFromOptions(InAdditionalOptions);
				if (!CurrentSPS.IsValid())
				{
					PostError(Result, TEXT("Failed to parse video SPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}

				TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
				Result = DecoderInstance->QueueInputBuffer(InputBufferIndex, CSD.GetData(), CSD.Num(), InInputAccessUnit.Data, InInputAccessUnit.DataSize, PTSTicks);
			}
			else
			{
				Result = DecoderInstance->QueueInputBuffer(InputBufferIndex, nullptr, 0, InInputAccessUnit.Data, InInputAccessUnit.DataSize, PTSTicks);
			}
			DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::DecodeAccessUnit() - queue input %lld (%lld), 0x%x"), (long long int)PTSTicks, (long long int)InInputAccessUnit.PTS.GetTicks(), InInputAccessUnit.Flags);
			if (Result == 0)
			{
				LastPushedPresentationTime = PTSTicks;

				// Add to the list of inputs passed to the decoder.
				TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
				In->AdditionalOptions = InAdditionalOptions;
				In->AccessUnit = InInputAccessUnit;
				In->SPS = CurrentSPS;
				In->DecoderPTS = PTSTicks;

				// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
				// values here that would lead us to think that we do.
				In->AccessUnit.Data = nullptr;
				In->AccessUnit.DataSize = 0;
				InDecoderInput.Emplace(MoveTemp(In));
				InDecoderInput.Sort([](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e1, const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e2)
				{
					return e1->AccessUnit.PTS < e2->AccessUnit.PTS;
				});

				return IElectraDecoder::EDecoderError::None;
			}
			else
			{
				PostError(Result, TEXT("Failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else if (InputBufferIndex == -1)
		{
			// No available input buffer. Try later.
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
		else
		{
			PostError(InputBufferIndex, TEXT("Failed to get a decoder input buffer"), ERRCODE_INTERNAL_DID_NOT_GET_INPUT_BUFFER);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Android::SendEndOfData()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::SendEndOfData()"));
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::SendEndOfData() - already sent"));
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance.IsValid())
	{
		if (!InternalDecoderDrain())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Android::Flush()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::Flush()"));
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

#ifdef DESTROY_DECODER_WHEN_FLUSHING
	ResetToCleanStart();

#else
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Flush();
	}
	CurrentOutput.Reset();
	InDecoderInput.Empty();
	DecoderPTSOffset.Reset();
	LastPushedPresentationTime = -1;
	bDidSendEOS = false;
	DecodeState = EDecodeState::Decoding;
#endif
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FElectraVideoDecoderH265_Android::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() -> Have"));
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !DecoderInstance.IsValid())
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() no / need new decoder"));
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	// Are we to drain the decoder?
	// We need an available input buffer to do this so we put it here where pulling output is sure to free
	// up an input buffer eventually.
	if (DecodeState == EDecodeState::Draining && !bDidSendEOS)
	{
		int32 Result = -1;
		int32 InputBufferIndex = DecoderInstance->DequeueInputBuffer(0);
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() Sending EOS"));
		if (InputBufferIndex >= 0)
		{
			Result = DecoderInstance->QueueEOSInputBuffer(InputBufferIndex, LastPushedPresentationTime);
			if (Result == 0)
			{
				bDidSendEOS = true;
			}
			else
			{
				PostError(Result, TEXT("Failed to submit decoder EOS input buffer"), ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT);
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
		else if (InputBufferIndex == -1)
		{
			// No available input buffer. Try later.
		}
		else
		{
			PostError(InputBufferIndex, TEXT("Failed to get a decoder input buffer for EOS"), ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT);
			return IElectraDecoder::EOutputStatus::Error;
		}
	}

	// Check if there is available output from the decoder.
	while(1)
	{
		IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo OutputBufferInfo;
		int32 Result = DecoderInstance->DequeueOutputBuffer(OutputBufferInfo, 0);
		if (Result != 0)
		{
			PostError(Result, TEXT("Failed to get decoder output buffer"), ERRCODE_INTERNAL_DID_NOT_GET_OUTPUT_BUFFER);
			return IElectraDecoder::EOutputStatus::Error;
		}

		// Valid output buffer?
		if (OutputBufferInfo.BufferIndex >= 0)
		{
			// Is this a pure EOS buffer with no decoded data?
			if (OutputBufferInfo.bIsEOS && OutputBufferInfo.Size == 0)
			{
				DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() got EOS, %d still in?"), InDecoderInput.Num());
				// We have to release that buffer even if it's empty.
				Result = DecoderInstance->ReleaseOutputBuffer(OutputBufferInfo.BufferIndex, OutputBufferInfo.ValidCount, false, -1);

				// Did we ask for an EOS?
				if (DecodeState == EDecodeState::Draining)
				{
					DecodeState = EDecodeState::Decoding;
					bDidSendEOS = false;
					InDecoderInput.Empty();
					// The decoder will not be accepting new input after an EOS until it gets flushed.
					// Since it will be drained mostly because of a format change we rather create a new one.
					bNewDecoderRequired = true;
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				// On some Android versions there could be such empty buffers without us having requested an EOS.
				else
				{
					DETAILLOG(LogElectraDecoders, Log, TEXT("  -> UNEXPECTED, continuing!"));
					continue;
				}
			}
			// If a configuration buffer is returned to us we ignore it. This serves no purpose.
			if (OutputBufferInfo.bIsConfig)
			{
				Result = DecoderInstance->ReleaseOutputBuffer(OutputBufferInfo.BufferIndex, OutputBufferInfo.ValidCount, false, -1);
				continue;
			}

			EConvertResult CnvRes = ConvertDecoderOutput(OutputBufferInfo);
			if (CnvRes == EConvertResult::Success)
			{
				// Was there the EOS flag set and did we request it?
				if (OutputBufferInfo.bIsEOS && DecodeState == EDecodeState::Draining)
				{
					// This should not happen. But assuming that it does then return EOS instead.
					DecodeState = EDecodeState::Decoding;
					bDidSendEOS = false;
					InDecoderInput.Empty();
					CurrentOutput.Reset();
					bNewDecoderRequired = true;
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				return IElectraDecoder::EOutputStatus::Available;
			}
			else
			{
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
		else if (OutputBufferInfo.BufferIndex == IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_TRY_AGAIN_LATER)
		{
			if (DecodeState == EDecodeState::Draining)
			{
				DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() - DRAINING - TRY AGAIN LATER!?!"));
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}
			DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::HaveOutput() - TRY AGAIN LATER!?!"));
			return IElectraDecoder::EOutputStatus::NeedInput;
		}
		else if (OutputBufferInfo.BufferIndex == IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_OUTPUT_FORMAT_CHANGED)
		{
			// Not needed.
		}
		else if (OutputBufferInfo.BufferIndex == IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo::EBufferIndexValues::MediaCodec_INFO_OUTPUT_BUFFERS_CHANGED)
		{
			// No-op as this is the Result of a deprecated API we are not using.
		}
		else
		{
			PostError(Result, TEXT("Unhandled output buffer index value"), ERRCODE_INTERNAL_UNEXPECTED_PROBLEM);
			return IElectraDecoder::EOutputStatus::Error;
		}
	}
	check(!"should never get here!");
	return IElectraDecoder::EOutputStatus::Error;
}


TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraVideoDecoderH265_Android::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}


void FElectraVideoDecoderH265_Android::Suspend()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::Suspend()"));
}


void FElectraVideoDecoderH265_Android::Resume()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::Resume()"));
	// Increment the resume counter to re-create the decoder.
	++CurrentResumeCount;
}


bool FElectraVideoDecoderH265_Android::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}


bool FElectraVideoDecoderH265_Android::InternalDecoderCreate()
{
	bDidSendEOS = false;
	bNewDecoderRequired = false;

	CurrentSurfaceRequest.Reset();

	DecodeState = EDecodeState::CreatingDecoder;
	return true;
}


bool FElectraVideoDecoderH265_Android::InternalHandleDecoderCreate()
{
	if (CurrentSurfaceRequest.IsValid())
	{
		// Request complete?
		if (!CurrentSurfaceRequest->HasCompleted())
		{
			// Not yet. Wait until it has.
			return true;
		}
		switch(CurrentSurfaceRequest->GetType())
		{
			//case FSurfaceRequestCallback::ESurfaceType::Error:
			default:
			{
				return PostError(0, TEXT("Failed to create decoder. Failed to obtain a surface to decode onto."), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
			}
			case FSurfaceRequestCallback::ESurfaceType::NoSurface:
			{
				return PostError(0, TEXT("Failed to create decoder. Decoding into CPU buffer is not supported."), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
			}
			case FSurfaceRequestCallback::ESurfaceType::Surface:
			{
				return InternalFinishDecoderCreate();
			}
		}
	}
	else
	{
		// Create a request to get a surface and request one.
		CurrentSurfaceRequest = MakeShared<FSurfaceRequestCallback, ESPMode::ThreadSafe>();
		ResourceInstance->RequestSurface(CurrentSurfaceRequest);
	}
	return true;
}

bool FElectraVideoDecoderH265_Android::InternalFinishDecoderCreate()
{
	if (CurrentSurfaceRequest.IsValid())
	{
		DecoderInstance = IElectraH265VideoDecoderAndroidJava::Create();

		int32 Width = InitialMaxValues.Width ? InitialMaxValues.Width : 640;
		int32 Height = InitialMaxValues.Height ? InitialMaxValues.Height : 480;
		int32 ProfileIdc = InitialMaxValues.Profile ? InitialMaxValues.Profile : 100;
		int32 LevelIdc = InitialMaxValues.Level ? InitialMaxValues.Level : 30;
		if (ConfigRecord.IsValid())
		{
			if (ensure(!ConfigRecord->GetParsedSequenceParameterSets().IsEmpty()))
			{
				const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& SPS = ConfigRecord->GetParsedSequenceParameterSets()[0];
				if (SPS.GetWidth() > Width || SPS.GetHeight() > Height || SPS.profile_tier_level.general_profile_idc > ProfileIdc || SPS.profile_tier_level.general_level_idc > LevelIdc)
				{
					Width = SPS.GetWidth();
					Height = SPS.GetHeight();
					ProfileIdc = SPS.profile_tier_level.general_profile_idc;
					LevelIdc = SPS.profile_tier_level.general_level_idc;
				}
			}
		}

		// Remember at which resume count we created the decoder.
		DecoderCreatedAtResumeCount = CurrentResumeCount;

		IElectraH265VideoDecoderAndroidJava::FCreateParameters cp;
		cp.CodecData = ConfigRecord.Get();
		cp.MaxWidth = Width;
		cp.MaxHeight = Height;
		cp.MaxTier = 1;					// not currently used, set to 1 (high tier)
		cp.MaxProfile = ProfileIdc;
		cp.MaxProfileLevel = LevelIdc;
		cp.MaxFrameRate = 60;
		cp.NativeDecoderID = NativeDecoderID;
		cp.VideoCodecSurface = CurrentSurfaceRequest->GetSurface();

		int32 Result = DecoderInstance->CreateDecoder();
		if (Result)
		{
			return PostError(Result, TEXT("Failed to create decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		}
		Result = DecoderInstance->InitializeDecoder(cp);
		if (Result)
		{
			return PostError(Result, TEXT("Failed to initialize decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		}

		const IElectraH265VideoDecoderAndroidJava::FDecoderInformation* DecInf = DecoderInstance->GetDecoderInformation();
		if (!DecInf)
		{
			return PostError(0, TEXT("Failed to initialize decoder, could not get decoder information"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		}
		DecoderInfo = *DecInf;

		Result = DecoderInstance->Start();
		if (Result)
		{
			return PostError(Result, TEXT("Failed to start decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		}
		DecodeState = EDecodeState::Decoding;

		return true;
	}
	else
	{
		return PostError(0, TEXT("Failed to start decoder, surface interface disappeared."), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
}

bool FElectraVideoDecoderH265_Android::InternalDecoderDrain()
{
	bDidSendEOS = false;
	return true;
}


void FElectraVideoDecoderH265_Android::InternalDecoderDestroy()
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::InternalDecoderDestroy()"));
	if (DecoderInstance.IsValid())
	{
		CurrentSurfaceRequest.Reset();
		DecoderInstance->Flush();
		DecoderInstance->Stop();
		DecoderInstance->ReleaseDecoder();
		DecoderInstance.Reset();
	}
}


FElectraVideoDecoderH265_Android::EConvertResult FElectraVideoDecoderH265_Android::ConvertDecoderOutput(const IElectraH265VideoDecoderAndroidJava::FOutputBufferInfo& InInfo)
{
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::ConvertDecoderOutput()"));
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	// Find the input corresponding to this output.
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec]->DecoderPTS == InInfo.PresentationTimestamp)
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}
	if (!In->SPS.IsValid())
	{
		PostError(0, TEXT("There is no SPS associated with the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	// If this is a replay frame or a frame that will not be output, we do not need to set it up and merely release the buffer immediately.
	if ((In->AccessUnit.Flags & (EElectraDecoderFlags::IsReplaySample | EElectraDecoderFlags::DoNotOutput)) != EElectraDecoderFlags::None)
	{
		DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::ConvertDecoderOutput() - Discard output %lld (%lld), 0x%x"), (long long int)In->DecoderPTS, (long long int)In->AccessUnit.PTS.GetTicks(), In->AccessUnit.Flags);
		if (DecoderInstance.IsValid())
		{
			DecoderInstance->ReleaseOutputBuffer(InInfo.BufferIndex, InInfo.ValidCount, false, -1);
		}
		return EConvertResult::Success;
	}
	DETAILLOG(LogElectraDecoders, Log, TEXT("VideoDecoderH265::ConvertDecoderOutput() - Convert output %lld (%lld), 0x%x"), (long long int)In->DecoderPTS, (long long int)In->AccessUnit.PTS.GetTicks(), In->AccessUnit.Flags);

	TSharedPtr<FElectraVideoDecoderOutputH265_Android, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraVideoDecoderOutputH265_Android>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;

	NewOutput->Width = In->SPS->GetWidth();
	NewOutput->Height = In->SPS->GetHeight();
	NewOutput->Pitch = NewOutput->Width;
	NewOutput->NumBits = 8 + In->SPS->bit_depth_luma_minus8;
	NewOutput->PixelFormat = 0;
	In->SPS->GetCrop(NewOutput->Crop.Left, NewOutput->Crop.Right, NewOutput->Crop.Top, NewOutput->Crop.Bottom);
	In->SPS->GetAspect(NewOutput->AspectW, NewOutput->AspectH);
	if (NewOutput->AspectW == 0 || NewOutput->AspectH == 0)
	{
		NewOutput->AspectW = 1;
		NewOutput->AspectH = 1;
	}
	ElectraDecodersUtil::FFractionalValue fr(In->SPS->GetTiming());
	if (fr.Denom)
	{
		NewOutput->FrameRateN = fr.Num;
		NewOutput->FrameRateD = fr.Denom;
	}

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("android")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("hevc")));

	CurrentOutput = MoveTemp(NewOutput);

	if (DecoderInstance.IsValid())
	{
		int32 Result = DecoderInstance->ReleaseOutputBuffer(InInfo.BufferIndex, InInfo.ValidCount, true, -1);
		if (Result)
		{
			PostError(Result, TEXT("Failed to release decoder output buffer"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return EConvertResult::Failure;
		}
	}

	return EConvertResult::Success;
}
