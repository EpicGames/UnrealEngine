// Copyright Epic Games, Inc. All Rights Reserved.

#include "vpx/VPx_VideoDecoder_Android.h"
#include "vpx/VideoDecoderVPx_Android.h"

#include "ElectraDecodersUtils.h"


class FVPxVideoDecoderFactoryAndroid : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FVPxVideoDecoderFactoryAndroid()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderVPx_Android::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('v','p','0','8') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('v','p','0','9'))
		{
			return 0;
		}
		return 1;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderVPx_Android::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}

	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("Android")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FVPxVideoDecoderAndroid::CreateFactory()
{
	return MakeShared<FVPxVideoDecoderFactoryAndroid, ESPMode::ThreadSafe>();
}
