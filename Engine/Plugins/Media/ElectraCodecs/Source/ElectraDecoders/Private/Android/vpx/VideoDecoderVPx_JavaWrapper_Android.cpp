// Copyright Epic Games, Inc. All Rights Reserved.

#include "vpx/VideoDecoderVPx_JavaWrapper_Android.h"

#if PLATFORM_ANDROID
#include "HAL/Platform.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"

#if (defined(USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY) && USE_ANDROID_JNI_WITHOUT_GAMEACTIVITY != 0) || USE_ANDROID_JNI
#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"

#if UE_BUILD_SHIPPING
// always clear any exceptions in shipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id)								\
	if (Id == 0)											\
	{														\
		if (!bIsOptional)									\
		{													\
			JEnv->ExceptionDescribe();						\
			checkf(Id != 0, TEXT("Failed to find " #Id));	\
		}													\
		JEnv->ExceptionClear();								\
	}
#endif

//----------------------------------------------------------------------------------------------------

class FElectraVPxVideoDecoderAndroidJava : public IElectraVPxVideoDecoderAndroidJava, public FJavaClassObject
{
public:

	FElectraVPxVideoDecoderAndroidJava();
	virtual ~FElectraVPxVideoDecoderAndroidJava();

	virtual int32 CreateDecoder(bool bCreateVP9) override;
	virtual int32 InitializeDecoder(const FCreateParameters& InCreateParams) override;
	virtual int32 SetOutputSurface(jobject InNewOutputSurface) override;
	virtual int32 ReleaseDecoder() override;
	virtual const FDecoderInformation* GetDecoderInformation() override;
	virtual int32 Start() override;
	virtual int32 Stop() override;
	virtual int32 Flush() override;
	virtual int32 Reset() override;
	virtual int32 DequeueInputBuffer(int32 InTimeoutUsec) override;
	virtual int32 QueueInputBuffer(int32 InBufferIndex, const void* InPrefixData, int32 InPrefixSize, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec) override;
	virtual int32 QueueCSDInputBuffer(int32 InBufferIndex, const void* InCSDData, int32 InCSDSize, int64 InTimestampUSec) override;
	virtual int32 QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec) override;
	virtual int32 GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex) override;
	virtual int32 DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec) override;
	virtual int32 GetOutputBuffer(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo) override;
	virtual int32 ReleaseOutputBuffer(int32 BufferIndex, int32 ValidCount, bool bRender, int64 releaseAt) override;

private:
	//-----------------------------------------------------------------------------
	/**
	 * Checks for and clears Java exceptions
	 *
	 * @param JEnv
	 */
	static void GClearException(JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv())
	{
		if (JEnv->ExceptionCheck())
		{
			JEnv->ExceptionDescribe();
			JEnv->ExceptionClear();
		}
	}

	static FName GetClassName()
	{
		return FName("com/epicgames/unreal/ElectraDecoderVideoVPx");
	}

	jfieldID FindField(JNIEnv* JEnv, jclass InClass, const ANSICHAR* InFieldName, const ANSICHAR* InFieldType, bool bIsOptional)
	{
		jfieldID Field = InClass == nullptr ? nullptr : JEnv->GetFieldID(InClass, InFieldName, InFieldType);
		CHECK_JNI_RESULT(Field);
		return Field;
	}

	jmethodID FindMethod(JNIEnv* JEnv, jclass InClass, const ANSICHAR* InMethodName, const ANSICHAR* InMethodSignature, bool bIsOptional)
	{
		jmethodID Method = InClass == nullptr ? nullptr : JEnv->GetMethodID(InClass, InMethodName, InMethodSignature);
		CHECK_JNI_RESULT(Method);
		return Method;
	}

	// Create a Java byte array. Must DeleteLocalRef() after use or handling over to Java.
	jbyteArray MakeJavaByteArray(const uint8* InData, int32 InNumBytes)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jbyteArray RawBuffer = JEnv->NewByteArray(InNumBytes);
		JEnv->SetByteArrayRegion(RawBuffer, 0, InNumBytes, reinterpret_cast<const jbyte*>(InData));
		return RawBuffer;
	}

	template<typename ReturnType>
	ReturnType CallMethodNoVerify(FJavaClassMethod Method, ...);

	template<>
	void CallMethodNoVerify<void>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		JEnv->CallVoidMethodV(Object, Method.Method, Params);
		va_end(Params);
	}

	template<>
	bool CallMethodNoVerify<bool>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		bool RetVal = JEnv->CallBooleanMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	int CallMethodNoVerify<int>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		int RetVal = JEnv->CallIntMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	jobject CallMethodNoVerify<jobject>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
		va_end(Params);
		jobject RetVal = JEnv->NewGlobalRef(val);
		JEnv->DeleteLocalRef(val);
		return RetVal;
	}

	template<>
	jobjectArray CallMethodNoVerify<jobjectArray>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jobject val = JEnv->CallObjectMethodV(Object, Method.Method, Params);
		va_end(Params);
		jobjectArray RetVal = (jobjectArray)JEnv->NewGlobalRef(val);
		JEnv->DeleteLocalRef(val);
		return RetVal;
	}

	template<>
	int64 CallMethodNoVerify<int64>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		int64 RetVal = JEnv->CallLongMethodV(Object, Method.Method, Params);
		va_end(Params);
		return RetVal;
	}

	template<>
	FString CallMethodNoVerify<FString>(FJavaClassMethod Method, ...)
	{
		JNIEnv*	JEnv = AndroidJavaEnv::GetJavaEnv();
		va_list Params;
		va_start(Params, Method);
		jstring RetVal = static_cast<jstring>(
			JEnv->CallObjectMethodV(Object, Method.Method, Params));
		va_end(Params);
		auto Result = FJavaHelper::FStringFromLocalRef(JEnv, RetVal);
		return Result;
	}

	void SetupDecoderInformation();

	int32 release();

public:
	// Java methods
	FJavaClassMethod	CreateDecoderFN;
	FJavaClassMethod	ReleaseDecoderFN;
	FJavaClassMethod	ConfigureDecoderFN;
	FJavaClassMethod	SetOutputSurfaceFN;
	FJavaClassMethod	ReleaseFN;
	FJavaClassMethod	StartFN;
	FJavaClassMethod	StopFN;
	FJavaClassMethod	FlushFN;
	FJavaClassMethod	ResetFN;
	FJavaClassMethod	DequeueInputBufferFN;
	FJavaClassMethod	QueueInputBufferFN;
	FJavaClassMethod	QueueCSDInputBufferFN;
	FJavaClassMethod	QueueEOSInputBufferFN;
	FJavaClassMethod	GetDecoderInformationFN;
	FJavaClassMethod	GetOutputFormatInfoFN;
	FJavaClassMethod	DequeueOutputBufferFN;
	FJavaClassMethod	GetOutputBufferFN;
	FJavaClassMethod	ReleaseOutputBufferFN;

	// FCreateParameters member field IDs
	jclass				FCreateParametersClass;
	jmethodID			FCreateParametersCTOR;
	jfieldID			FCreateParameters_MaxWidth;
	jfieldID			FCreateParameters_MaxHeight;
	jfieldID			FCreateParameters_MaxFPS;
	jfieldID			FCreateParameters_Width;
	jfieldID			FCreateParameters_Height;
	jfieldID			FCreateParameters_bNeedSecure;
	jfieldID			FCreateParameters_bNeedTunneling;
	jfieldID			FCreateParameters_CSD0;
	jfieldID			FCreateParameters_NativeDecoderID;
	jfieldID			FCreateParameters_VideoCodecSurface;
	jfieldID			FCreateParameters_bSurfaceIsView;

	// FDecoderInformation member field IDs
	jclass				FDecoderInformationClass;
	jfieldID			FDecoderInformation_bIsAdaptive;
	jfieldID			FDecoderInformation_ApiLevel;
	jfieldID			FDecoderInformation_bCanUse_SetOutputSurface;

	// FOutputFormatInfo member field IDs
	jclass				FOutputFormatInfoClass;
	jfieldID			FOutputFormatInfo_Width;
	jfieldID			FOutputFormatInfo_Height;
	jfieldID			FOutputFormatInfo_CropTop;
	jfieldID			FOutputFormatInfo_CropBottom;
	jfieldID			FOutputFormatInfo_CropLeft;
	jfieldID			FOutputFormatInfo_CropRight;
	jfieldID			FOutputFormatInfo_Stride;
	jfieldID			FOutputFormatInfo_SliceHeight;
	jfieldID			FOutputFormatInfo_ColorFormat;

	// FOutputBufferInfo member field IDs
	jclass				FOutputBufferInfoClass;
	jfieldID			FOutputBufferInfo_BufferIndex;
	jfieldID			FOutputBufferInfo_PresentationTimestamp;
	jfieldID			FOutputBufferInfo_Size;
	jfieldID			FOutputBufferInfo_bIsEOS;
	jfieldID			FOutputBufferInfo_bIsConfig;

	// Internal state
	FCriticalSection				Lock;
	TUniquePtr<FDecoderInformation>	CurrentDecoderInformation;
	int32							CurrentValidCount = 0;
	bool							bHaveDecoder = false;
	bool							bIsStarted = false;
};


//-----------------------------------------------------------------------------
/**
 * Create a Java video decoder wrapper.
 *
 * @return Decoder wrapper instance.
 */
TSharedPtr<IElectraVPxVideoDecoderAndroidJava, ESPMode::ThreadSafe> IElectraVPxVideoDecoderAndroidJava::Create()
{
	return MakeShared<FElectraVPxVideoDecoderAndroidJava>();
}


//-----------------------------------------------------------------------------
/**
 * CTOR
 */
FElectraVPxVideoDecoderAndroidJava::FElectraVPxVideoDecoderAndroidJava()
	: FJavaClassObject(GetClassName(), "()V")
	, CreateDecoderFN(GetClassMethod("CreateDecoder", "(Z)I"))
	, ReleaseDecoderFN(GetClassMethod("ReleaseDecoder", "()I"))
	, ConfigureDecoderFN(GetClassMethod("ConfigureDecoder", "(Lcom/epicgames/unreal/ElectraDecoderVideoVPx$FCreateParameters;)I"))
	, SetOutputSurfaceFN(GetClassMethod("SetOutputSurface", "(Landroid/view/Surface;)I"))
	, ReleaseFN(GetClassMethod("release", "()I"))
	, StartFN(GetClassMethod("Start", "()I"))
	, StopFN(GetClassMethod("Stop", "()I"))
	, FlushFN(GetClassMethod("Flush", "()I"))
	, ResetFN(GetClassMethod("Reset", "()I"))
	, DequeueInputBufferFN(GetClassMethod("DequeueInputBuffer", "(I)I"))
	, QueueInputBufferFN(GetClassMethod("QueueInputBuffer", "(IJ[B)I"))
	, QueueCSDInputBufferFN(GetClassMethod("QueueCSDInputBuffer", "(IJ[B)I"))
	, QueueEOSInputBufferFN(GetClassMethod("QueueEOSInputBuffer", "(IJ)I"))
	, GetDecoderInformationFN(GetClassMethod("GetDecoderInformation", "()Lcom/epicgames/unreal/ElectraDecoderVideoVPx$FDecoderInformation;"))
	, GetOutputFormatInfoFN(GetClassMethod("GetOutputFormatInfo", "(I)Lcom/epicgames/unreal/ElectraDecoderVideoVPx$FOutputFormatInfo;"))
	, DequeueOutputBufferFN(GetClassMethod("DequeueOutputBuffer", "(I)Lcom/epicgames/unreal/ElectraDecoderVideoVPx$FOutputBufferInfo;"))
	, GetOutputBufferFN(GetClassMethod("GetOutputBuffer", "(I)[B"))
	, ReleaseOutputBufferFN(GetClassMethod("ReleaseOutputBuffer", "(IZJ)I"))
{
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();

	// Get field IDs for FCreateParameters class members
	jclass localCreateParametersClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderVideoVPx$FCreateParameters");
	FCreateParametersClass = (jclass)JEnv->NewGlobalRef(localCreateParametersClass);
	JEnv->DeleteLocalRef(localCreateParametersClass);
	FCreateParametersCTOR = FindMethod(JEnv, FCreateParametersClass, "<init>", "()V", false);
	FCreateParameters_MaxWidth = FindField(JEnv, FCreateParametersClass, "MaxWidth", "I", false);
	FCreateParameters_MaxHeight = FindField(JEnv, FCreateParametersClass, "MaxHeight", "I", false);
	FCreateParameters_MaxFPS = FindField(JEnv, FCreateParametersClass, "MaxFPS", "I", false);
	FCreateParameters_Width = FindField(JEnv, FCreateParametersClass, "Width", "I", false);
	FCreateParameters_Height = FindField(JEnv, FCreateParametersClass, "Height", "I", false);
	FCreateParameters_bNeedSecure = FindField(JEnv, FCreateParametersClass, "bNeedSecure", "Z", false);
	FCreateParameters_bNeedTunneling = FindField(JEnv, FCreateParametersClass, "bNeedTunneling", "Z", false);
	FCreateParameters_CSD0 = FindField(JEnv, FCreateParametersClass, "CSD0", "[B", false);
	FCreateParameters_NativeDecoderID = FindField(JEnv, FCreateParametersClass, "NativeDecoderID", "I", false);
	FCreateParameters_VideoCodecSurface = FindField(JEnv, FCreateParametersClass, "VideoCodecSurface", "Landroid/view/Surface;", false);
	FCreateParameters_bSurfaceIsView = FindField(JEnv, FCreateParametersClass, "bSurfaceIsView", "Z", false);

	// Get field IDs for FDecoderInformation class members
	jclass localDecoderInformationClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderVideoVPx$FDecoderInformation");
	FDecoderInformationClass = (jclass)JEnv->NewGlobalRef(localDecoderInformationClass);
	JEnv->DeleteLocalRef(localDecoderInformationClass);
	FDecoderInformation_bIsAdaptive = FindField(JEnv, FDecoderInformationClass, "bIsAdaptive", "Z", false);
	FDecoderInformation_ApiLevel = FindField(JEnv, FDecoderInformationClass, "ApiLevel", "I", false);
	FDecoderInformation_bCanUse_SetOutputSurface = FindField(JEnv, FDecoderInformationClass, "bCanUse_SetOutputSurface", "Z", false);

	// Get field IDs for FOutputFormatInfo class members
	jclass localOutputFormatInfoClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderVideoVPx$FOutputFormatInfo");
	FOutputFormatInfoClass = (jclass)JEnv->NewGlobalRef(localOutputFormatInfoClass);
	JEnv->DeleteLocalRef(localOutputFormatInfoClass);
	FOutputFormatInfo_Width = FindField(JEnv, FOutputFormatInfoClass, "Width", "I", false);
	FOutputFormatInfo_Height = FindField(JEnv, FOutputFormatInfoClass, "Height", "I", false);
	FOutputFormatInfo_CropTop = FindField(JEnv, FOutputFormatInfoClass, "CropTop", "I", false);
	FOutputFormatInfo_CropBottom = FindField(JEnv, FOutputFormatInfoClass, "CropBottom", "I", false);
	FOutputFormatInfo_CropLeft = FindField(JEnv, FOutputFormatInfoClass, "CropLeft", "I", false);
	FOutputFormatInfo_CropRight = FindField(JEnv, FOutputFormatInfoClass, "CropRight", "I", false);
	FOutputFormatInfo_Stride = FindField(JEnv, FOutputFormatInfoClass, "Stride", "I", false);
	FOutputFormatInfo_SliceHeight = FindField(JEnv, FOutputFormatInfoClass, "SliceHeight", "I", false);
	FOutputFormatInfo_ColorFormat = FindField(JEnv, FOutputFormatInfoClass, "ColorFormat", "I", false);

	// Get field IDs for FOutputBufferInfo class members
	jclass localOutputBufferInfoClass = AndroidJavaEnv::FindJavaClass("com/epicgames/unreal/ElectraDecoderVideoVPx$FOutputBufferInfo");
	FOutputBufferInfoClass = (jclass)JEnv->NewGlobalRef(localOutputBufferInfoClass);
	JEnv->DeleteLocalRef(localOutputBufferInfoClass);
	FOutputBufferInfo_BufferIndex = FindField(JEnv, FOutputBufferInfoClass, "BufferIndex", "I", false);
	FOutputBufferInfo_PresentationTimestamp = FindField(JEnv, FOutputBufferInfoClass, "PresentationTimestamp", "J", false);
	FOutputBufferInfo_Size = FindField(JEnv, FOutputBufferInfoClass, "Size", "I", false);
	FOutputBufferInfo_bIsEOS = FindField(JEnv, FOutputBufferInfoClass, "bIsEOS", "Z", false);
	FOutputBufferInfo_bIsConfig = FindField(JEnv, FOutputBufferInfoClass, "bIsConfig", "Z", false);

	// Set up the initial decoder information.
	SetupDecoderInformation();
}


//-----------------------------------------------------------------------------
/**
 * DTOR
 */
FElectraVPxVideoDecoderAndroidJava::~FElectraVPxVideoDecoderAndroidJava()
{
	FScopeLock lock(&Lock);
	release();

	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	JEnv->DeleteGlobalRef(FCreateParametersClass);
	JEnv->DeleteGlobalRef(FDecoderInformationClass);
	JEnv->DeleteGlobalRef(FOutputBufferInfoClass);
	JEnv->DeleteGlobalRef(FOutputFormatInfoClass);

	CurrentDecoderInformation.Reset();
}


//-----------------------------------------------------------------------------
/**
 * Sets up the current decoder information.
 */
void FElectraVPxVideoDecoderAndroidJava::SetupDecoderInformation()
{
	// Create an instance of the init param structure and fill in the members.
	JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
	jobject OutputInfo = JEnv->CallObjectMethod(Object, GetDecoderInformationFN.Method);
	GClearException(JEnv);
	// Failure will return no object.
	check(OutputInfo != nullptr);
	if (OutputInfo != nullptr)
	{
		CurrentDecoderInformation = MakeUnique<FDecoderInformation>();
		CurrentDecoderInformation->ApiLevel = JEnv->GetIntField(OutputInfo, FDecoderInformation_ApiLevel);
		CurrentDecoderInformation->bIsAdaptive = JEnv->GetBooleanField(OutputInfo, FDecoderInformation_bIsAdaptive);
		CurrentDecoderInformation->bCanUse_SetOutputSurface = JEnv->GetBooleanField(OutputInfo, FDecoderInformation_bCanUse_SetOutputSurface);
		JEnv->DeleteLocalRef(OutputInfo);
	}
}


//-----------------------------------------------------------------------------
/**
 * Creates a Java instance of a VP8 or VP9 video decoder.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::CreateDecoder(bool bCreateVP9)
{
	FScopeLock lock(&Lock);
	int32 result = -1;

	check(!bHaveDecoder);

	CurrentDecoderInformation.Reset();

	// Create and initialize a decoder instance.
	result = CallMethodNoVerify<int>(CreateDecoderFN, bCreateVP9);
	GClearException();
	if (result != 0)
	{
		return 1;
	}

	// Get decoder information
	SetupDecoderInformation();

	bHaveDecoder = true;
	return 0;
}

//-----------------------------------------------------------------------------
/**
 * Initializes the video decoder instance.
 *
 * @param InCreateParams
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::InitializeDecoder(const FCreateParameters& InCreateParams)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result = -1;

		// Create an instance of the init param structure and fill in the members.
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jobject CreateParams = JEnv->NewObject(FCreateParametersClass, FCreateParametersCTOR);
		JEnv->SetIntField(CreateParams, FCreateParameters_MaxWidth, InCreateParams.MaxWidth);
		JEnv->SetIntField(CreateParams, FCreateParameters_MaxHeight, InCreateParams.MaxHeight);
		JEnv->SetIntField(CreateParams, FCreateParameters_MaxFPS, InCreateParams.MaxFrameRate);
		//JEnv->SetBooleanField(CreateParams, FCreateParameters_bNeedSecure, false);
		//JEnv->SetBooleanField(CreateParams, FCreateParameters_bNeedTunneling, false);
		JEnv->SetIntField(CreateParams, FCreateParameters_NativeDecoderID, InCreateParams.NativeDecoderID);
/*
		if (InCreateParams.CodecData)
		{
			jbyteArray CSD0 = MakeJavaByteArray(InCreateParams.CodecData->GetCodecSpecificData().GetData(), InCreateParams.CodecData->GetCodecSpecificData().Num());
			JEnv->SetObjectField(CreateParams, FCreateParameters_CSD0, CSD0);
			JEnv->DeleteLocalRef(CSD0);
		}
*/
		// Pass along decoder output surface. This is a global ref that we take control over now.
		jobject GlobalSurfaceRef = InCreateParams.VideoCodecSurface;
		if (GlobalSurfaceRef)
		{
			JEnv->SetObjectField(CreateParams, FCreateParameters_VideoCodecSurface, GlobalSurfaceRef);
		}
		JEnv->SetBooleanField(CreateParams, FCreateParameters_bSurfaceIsView, InCreateParams.bSurfaceIsView);

		// Create and initialize a decoder instance.
		result = CallMethodNoVerify<int>(ConfigureDecoderFN, CreateParams);
		JEnv->DeleteLocalRef(CreateParams);
		// We now release the global ref of the surface we were given. The decoder should have taken ownership of it now.
		if (GlobalSurfaceRef)
		{
			JEnv->DeleteGlobalRef(GlobalSurfaceRef);
		}

		GClearException(JEnv);
		if (result != 0)
		{
			return 1;
		}

		return 0;
	}
	else
	{
		return 1;
	}
}


//-----------------------------------------------------------------------------
/**
 * Attempts to set a new output surface on an existing and configured decoder.
 *
 * @param InNewOutputSurface
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::SetOutputSurface(jobject InNewOutputSurface)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result = -1;
		result = CallMethodNoVerify<int>(SetOutputSurfaceFN, InNewOutputSurface);
		GClearException();
		if (result != 0)
		{
			return 1;
		}
		return 0;
	}
	else
	{
		return 1;
	}
}


//-----------------------------------------------------------------------------
/**
 * Releases (destroys) the Java video decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::ReleaseDecoder()
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result = CallMethodNoVerify<int>(ReleaseDecoderFN);
		GClearException();
		bHaveDecoder = false;
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Releases (destroys) the Java texture renderer the Java decoder has created
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::release()
{
	Stop();

	int32 result = CallMethodNoVerify<int>(ReleaseFN);
	GClearException();
	return result ? 1 : 0;
}


//-----------------------------------------------------------------------------
/**
 * Returns decoder information after a successful InitializeDecoder().
 *
 * @return Pointer to decoder information or null when no decoder has been created.
 */
const IElectraVPxVideoDecoderAndroidJava::FDecoderInformation* FElectraVPxVideoDecoderAndroidJava::GetDecoderInformation()
{
	return CurrentDecoderInformation.Get();
}


//-----------------------------------------------------------------------------
/**
 * Starts the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::Start()
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder && !bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(StartFN);
		GClearException();
		if (result)
		{
			return result;
		}
		bIsStarted = true;
		return 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Stops the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::Stop()
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder && bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(StopFN);
		GClearException();
		if (result)
		{
			return result;
		}
		bIsStarted = false;
		return 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Flushes the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::Flush()
{
	FScopeLock lock(&Lock);

	// Need to increment the valid count to ensure we will not try to release or render output buffers
	// that have become invalid.
	FPlatformAtomics::InterlockedIncrement(&CurrentValidCount);

	// Synchronously operating decoders must be in the started state to be flushed.
	if (bHaveDecoder && bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(FlushFN);
		GClearException();
		return result;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Resets the decoder instance.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::Reset()
{
	FScopeLock lock(&Lock);
	// Synchronously operating decoders should (must?) be in the stopped state to be reset.
	if (bHaveDecoder && !bIsStarted)
	{
		int32 result = CallMethodNoVerify<int>(ResetFN);
		GClearException();
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Dequeues an input buffer.
 *
 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
 *
 * @return >= 0 returns the index of the successfully dequeued buffer, negative values indicate an error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::DequeueInputBuffer(int32 InTimeoutUsec)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result = CallMethodNoVerify<int>(DequeueInputBufferFN, InTimeoutUsec);
		GClearException();
		return result;
	}
	return -1;
}


//-----------------------------------------------------------------------------
/**
 * Queues input for decoding in the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
 *
 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
 * @param InPrefixData Data to prefix the data to decode with. If given it is usually the SPS & PPS.
 * @param InPrefixSize Size of the prefix data.
 * @param InAccessUnitData Data to be decoded.
 * @param InAccessUnitSize Size of the data to be decoded.
 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::QueueInputBuffer(int32 InBufferIndex, const void* InPrefixData, int32 InPrefixSize, const void* InAccessUnitData, int32 InAccessUnitSize, int64 InTimestampUSec)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result;
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		if (InPrefixData && InPrefixSize)
		{
			uint8* Temp = (uint8*)FMemory::Malloc(InPrefixSize + InAccessUnitSize);
			FMemory::Memcpy(Temp, InPrefixData, InPrefixSize);
			FMemory::Memcpy(Temp+InPrefixSize, InAccessUnitData, InAccessUnitSize);
			jbyteArray InData = MakeJavaByteArray((const uint8*)Temp, InPrefixSize + InAccessUnitSize);
			result = CallMethodNoVerify<int>(QueueInputBufferFN, InBufferIndex, (jlong)InTimestampUSec, InData);
			JEnv->DeleteLocalRef(InData);
			FMemory::Free(Temp);
		}
		else
		{
			jbyteArray InData = MakeJavaByteArray((const uint8*)InAccessUnitData, InAccessUnitSize);
			result = CallMethodNoVerify<int>(QueueInputBufferFN, InBufferIndex, (jlong)InTimestampUSec, InData);
			JEnv->DeleteLocalRef(InData);
		}
		GClearException(JEnv);
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Queues codec specific data for the following to-be-decoded data buffers.
 *
 * @param InBufferIndex Index of the buffer to put data into and enqueue for decoding (see DequeueInputBuffer()).
 * @param InCSDData Codec specific data
 * @param InCSDSize Size of the codec specific data
 * @param InTimestampUSec Timestamp (PTS) of the data, in microseconds. Must be the same as the next data to be decoded.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::QueueCSDInputBuffer(int32 InBufferIndex, const void* InCSDData, int32 InCSDSize, int64 InTimestampUSec)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jbyteArray  InData = MakeJavaByteArray((const uint8*)InCSDData, InCSDSize);
		int32 result = CallMethodNoVerify<int>(QueueCSDInputBufferFN, InBufferIndex, (jlong)InTimestampUSec, InData);
		JEnv->DeleteLocalRef(InData);
		GClearException(JEnv);
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Queues end of stream for the buffer with a previously dequeued (calling DequeueInputBuffer()) index.
 *
 * @param InBufferIndex Index of the buffer to put the EOS flag into and enqueue for decoding (see DequeueInputBuffer()).
 * @param InTimestampUSec Timestamp the previous data had. Can be 0.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::QueueEOSInputBuffer(int32 InBufferIndex, int64 InTimestampUSec)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		int32 result = CallMethodNoVerify<int>(QueueEOSInputBufferFN, InBufferIndex, (jlong)InTimestampUSec);
		GClearException();
		return result ? 1 : 0;
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Returns format information of the decoded samples.
 *
 * @param OutFormatInfo
 * @param InOutputBufferIndex RESERVED FOR NOW - Pass any negative value to get the output format after DequeueOutputBuffer() returns a BufferIndex of MediaCodec_INFO_OUTPUT_FORMAT_CHANGED.
 *
 * @return 0 if successful, 1 on error.
 */
int32 FElectraVPxVideoDecoderAndroidJava::GetOutputFormatInfo(FOutputFormatInfo& OutFormatInfo, int32 InOutputBufferIndex)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jobject OutputInfo = JEnv->CallObjectMethod(Object, GetOutputFormatInfoFN.Method, InOutputBufferIndex);
		GClearException(JEnv);
		// Failure will return no object.
		if (OutputInfo != nullptr)
		{
			OutFormatInfo.Width = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_Width);
			OutFormatInfo.Height = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_Height);
			OutFormatInfo.CropTop = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_CropTop);
			OutFormatInfo.CropBottom = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_CropBottom);
			OutFormatInfo.CropLeft = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_CropLeft);
			OutFormatInfo.CropRight = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_CropRight);
			OutFormatInfo.Stride = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_Stride);
			OutFormatInfo.SliceHeight = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_SliceHeight);
			OutFormatInfo.ColorFormat = JEnv->GetIntField(OutputInfo, FOutputFormatInfo_ColorFormat);
			JEnv->DeleteLocalRef(OutputInfo);
			return 0;
		}
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Dequeues an output buffer.
 *
 * @param InTimeoutUsec Timeout in microseconds to wait for an available buffer.
 *
 * @return 0 on success, 1 on failure. The OutBufferInfo.BufferIndex indicates the buffer index.
 */
int32 FElectraVPxVideoDecoderAndroidJava::DequeueOutputBuffer(FOutputBufferInfo& OutBufferInfo, int32 InTimeoutUsec)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
		jobject OutputInfo = JEnv->CallObjectMethod(Object, DequeueOutputBufferFN.Method, InTimeoutUsec);
		GClearException(JEnv);
		// Failure will return no object.
		if (OutputInfo != nullptr)
		{
			OutBufferInfo.ValidCount = CurrentValidCount;
			OutBufferInfo.BufferIndex = JEnv->GetIntField(OutputInfo, FOutputBufferInfo_BufferIndex);
			OutBufferInfo.PresentationTimestamp = JEnv->GetLongField(OutputInfo, FOutputBufferInfo_PresentationTimestamp);
			OutBufferInfo.Size = JEnv->GetIntField(OutputInfo, FOutputBufferInfo_Size);
			OutBufferInfo.bIsEOS = JEnv->GetBooleanField(OutputInfo, FOutputBufferInfo_bIsEOS);
			OutBufferInfo.bIsConfig = JEnv->GetBooleanField(OutputInfo, FOutputBufferInfo_bIsConfig);
			JEnv->DeleteLocalRef(OutputInfo);
			return 0;
		}
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Returns the decoded samples from a decoder output buffer in the decoder native format!
 *
 * @param OutBufferDataPtr
 * @param OutBufferDataSize
 * @param InOutBufferInfo
 *
 * @return 0 on success, 1 on failure.
 */
int32 FElectraVPxVideoDecoderAndroidJava::GetOutputBuffer(void*& OutBufferDataPtr, int32 OutBufferDataSize, const FOutputBufferInfo& InOutBufferInfo)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		if (InOutBufferInfo.ValidCount == CurrentValidCount)
		{
			JNIEnv* JEnv = AndroidJavaEnv::GetJavaEnv();
			jbyteArray RawDecoderArray = (jbyteArray)JEnv->CallObjectMethod(Object, GetOutputBufferFN.Method, InOutBufferInfo.BufferIndex);
			GClearException(JEnv);

			if (RawDecoderArray != nullptr && OutBufferDataPtr != nullptr)
			{
				jbyte* RawDataPtr = JEnv->GetByteArrayElements(RawDecoderArray, 0);
				int32 RawBufferSize = JEnv->GetArrayLength(RawDecoderArray);

				check(RawBufferSize == InOutBufferInfo.Size);
				check(RawBufferSize <= OutBufferDataSize)
				if (RawDataPtr != nullptr)
				{
					FMemory::Memcpy(OutBufferDataPtr, RawDataPtr, RawBufferSize <= OutBufferDataSize ? RawBufferSize : OutBufferDataSize);
					JEnv->ReleaseByteArrayElements(RawDecoderArray, RawDataPtr, JNI_ABORT);
				}

				JEnv->DeleteLocalRef(RawDecoderArray);
			}
			return 0;
		}
		else
		{
			// Decoder got flushed, buffer index no longer valid.
			return 1;
		}
	}
	return 1;
}


//-----------------------------------------------------------------------------
/**
 * Releases the decoder output buffer back to the decoder.
 *
 * @param BufferIndex
 * @param ValidCount
 * @param bRender
 * @param releaseAt
 *
 * @return 0 on success, 1 on failure.
 */
int32 FElectraVPxVideoDecoderAndroidJava::ReleaseOutputBuffer(int32 BufferIndex, int32 ValidCount, bool bRender, int64 releaseAt)
{
	FScopeLock lock(&Lock);
	if (bHaveDecoder)
	{
		// Still same decoder instance?
		if (ValidCount == CurrentValidCount)
		{
			// Yes...
			int32 result = CallMethodNoVerify<int>(ReleaseOutputBufferFN, BufferIndex, bRender, releaseAt);
			GClearException();
			return result ? 1 : 0;
		}
		else
		{
			// Decoder got flushed, buffer index no longer valid.
			// This is NOT an error!
			return 0;
		}
	}
	return 1;

}


#endif

#endif
