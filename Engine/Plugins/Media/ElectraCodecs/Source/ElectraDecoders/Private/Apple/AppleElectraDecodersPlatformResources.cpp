// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleElectraDecodersPlatformResources.h"
#include "Templates/SharedPointer.h"
#include "RHI.h"

#include "ElectraDecodersModule.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MediaDecoderOutput.h"
#include "MediaVideoDecoderOutput.h"
#include "ElectraTextureSample.h"

#if PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS

#include "IMetalDynamicRHI.h"

bool FElectraDecodersPlatformResourcesApple::GetMetalDevice(void** OutMetalDevice)
{
    *OutMetalDevice = nullptr;
	if (!GDynamicRHI || !OutMetalDevice)
	{
		return false;
	}
	auto RHIType = RHIGetInterfaceType();
	if (RHIType != ERHIInterfaceType::Metal)
	{
		return false;
	}
	*OutMetalDevice = GetIMetalDynamicRHI()->RHIGetDevice();
	return true;
}

FElectraDecodersPlatformResourcesApple::IDecoderPlatformResource* FElectraDecodersPlatformResourcesApple::CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions)
{
	return nullptr;
}

void FElectraDecodersPlatformResourcesApple::ReleasePlatformVideoResource(void* InOwnerHandle, FElectraDecodersPlatformResourcesApple::IDecoderPlatformResource* InHandleToRelease)
{
}

bool FElectraDecodersPlatformResourcesApple::SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, FElectraDecodersPlatformResourcesApple::IDecoderPlatformResource* InPlatformResource)
{
	if (!InOutTextureSampleToSetup.IsValid() || !InDecoderOutput.IsValid() || !InOutBufferPropertes.IsValid())// || !InPlatformResource)
	{
		OutErrorMessage = TEXT("Bad parameter");
		return false;
	}

	TMap<FString, FVariant> ExtraValues;
	InDecoderOutput->GetExtraValues(ExtraValues);

	FElectraVideoDecoderOutputCropValues Crop = InDecoderOutput->GetCropValues();
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Width, Electra::FVariantValue((int64)InDecoderOutput->GetWidth()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Height, Electra::FVariantValue((int64)InDecoderOutput->GetHeight()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropLeft, Electra::FVariantValue((int64)Crop.Left));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropRight, Electra::FVariantValue((int64)Crop.Right));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropTop, Electra::FVariantValue((int64)Crop.Top));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropBottom, Electra::FVariantValue((int64)Crop.Bottom));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectRatio, Electra::FVariantValue((double)InDecoderOutput->GetAspectRatioW() / (double)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectW, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioW()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectH, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSNumerator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateNumerator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSDenominator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateDenominator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64)InDecoderOutput->GetNumberOfBits()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelDataScale, Electra::FVariantValue((double)ElectraDecodersUtil::GetVariantValueSafeDouble(ExtraValues, TEXT("pix_datascale"), 1.0)));

	IElectraDecoderVideoOutputImageBuffers* ImageBuffers = reinterpret_cast<IElectraDecoderVideoOutputImageBuffers*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::ImageBuffers));
	if (ImageBuffers && ImageBuffers->GetNumberOfBuffers() == 1)
	{
		EPixelFormat PixFmt = ImageBuffers->GetBufferFormatByIndex(0);
		EElectraTextureSamplePixelEncoding PixEnc = ImageBuffers->GetBufferEncodingByIndex(0);
		int32 Pitch = ImageBuffers->GetBufferPitchByIndex(0);
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)PixFmt));
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelEncoding, Electra::FVariantValue((int64)PixEnc));
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));

		InOutTextureSampleToSetup->InitializeCommon(*InOutBufferPropertes);

		bool bIsValid = false;
		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> ColorBuffer = ImageBuffers->GetBufferDataByIndex(0);
		if (ColorBuffer.IsValid() && ColorBuffer->Num())
		{
			//
			// CPU side buffer
			//
			int32 Width = InDecoderOutput->GetDecodedWidth();
			int32 Height = InDecoderOutput->GetDecodedHeight();

			InOutTextureSampleToSetup->Buffer = MoveTemp(ColorBuffer);
			InOutTextureSampleToSetup->Stride = Pitch;
			InOutTextureSampleToSetup->SetDim(FIntPoint(Width, Height));
			bIsValid = true;
		}
		else
		{
			//
			// GPU texture (CVImageBuffer)
			//
			CVImageBufferRef ImageBuffer = static_cast<CVImageBufferRef>(ImageBuffers->GetBufferTextureByIndex(0));
			if (ImageBuffer != nullptr)
			{
				if (InOutTextureSampleToSetup->ImageBufferRef)
				{
					CFRelease(InOutTextureSampleToSetup->ImageBufferRef);
				}
				InOutTextureSampleToSetup->ImageBufferRef = ImageBuffer;
				CFRetain(InOutTextureSampleToSetup->ImageBufferRef);
				InOutTextureSampleToSetup->Stride = CVPixelBufferGetBytesPerRow(InOutTextureSampleToSetup->ImageBufferRef);
				bIsValid = true;
			}
		}
		// Finish initialization.
		if (bIsValid)
		{
			if (!InOutTextureSampleToSetup->FinishInitialization())
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Unsupported pixel format encoding"));
				return false;
			}
			return true;
		}
	}
	OutErrorMessage = TEXT("Unhandled decoder output format");
	return false;
}



#endif // PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS
