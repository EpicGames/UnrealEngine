// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplePlatformElectraDecoders.h"
#include "IElectraCodecRegistry.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

#include "h264/H264_VideoDecoder_Apple.h"
#include "h265/H265_VideoDecoder_Apple.h"
#include "aac/AAC_AudioDecoder_Apple.h"

namespace PlatformElectraDecodersApple
{
	static TArray<TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> Factories;
}

void FPlatformElectraDecodersApple::Startup()
{
}

void FPlatformElectraDecodersApple::Shutdown()
{
	PlatformElectraDecodersApple::Factories.Empty();
}

void FPlatformElectraDecodersApple::RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith)
{
	check(InCodecFactoryToRegisterWith);
	if (InCodecFactoryToRegisterWith)
	{
		auto RegisterFactory = [&](TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InFactory) -> void
		{
			if (InFactory.IsValid())
			{
				PlatformElectraDecodersApple::Factories.Emplace(InFactory);
				InCodecFactoryToRegisterWith->AddCodecFactory(InFactory);
			}
		};

		// H.264 video decoder
		RegisterFactory(FH264VideoDecoderApple::CreateFactory());
		// H.265 video decoder
		RegisterFactory(FH265VideoDecoderApple::CreateFactory());
		// AAC audio decoder
		RegisterFactory(FAACAudioDecoderApple::CreateFactory());
	}
}

#endif
