// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

class IElectraCodecRegistry;

class FPlatformElectraDecodersApple
{
public:
	static void Startup();
	static void Shutdown();
	static void RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith);
};

using FPlatformElectraDecoders = FPlatformElectraDecodersApple;

#endif
