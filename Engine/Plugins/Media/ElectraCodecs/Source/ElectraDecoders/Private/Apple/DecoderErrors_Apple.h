// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_UNEXPECTED_PROBLEM					1
#define ERRCODE_INTERNAL_ALREADY_CLOSED						2
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				3
#define ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				5
#define ERRCODE_INTERNAL_DID_NOT_GET_INPUT_BUFFER			6
#define ERRCODE_INTERNAL_DID_NOT_GET_OUTPUT_BUFFER			7
#define ERRCODE_INTERNAL_COULD_NOT_GET_OUTPUT_FORMAT		8
#define ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT			9
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	10

