// Copyright Epic Games, Inc. All Rights Reserved.

#include "aac/AAC_AudioDecoder_Apple.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

#include "ElectraDecodersUtils.h"

#include "aac/AudioDecoderAAC_Apple.h"

class FAACAudioDecoderFactoryApple : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FAACAudioDecoderFactoryApple()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraAudioDecoderAAC_Apple::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Audio || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('m','p','4','a'))
		{
			return 0;
		}

		const Electra::FCodecTypeFormat::FAudio& ai = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FAudio>();
		if (ai.ObjectType.IsType<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>())
		{
			// AAC-LC, AAC-HE (SBR), AAC-HEv2 (PS) ?
			const Electra::FCodecTypeFormat::FAudio::FMPEGObjectType& OT(ai.ObjectType.Get<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>());
			if (OT.ObjectType != 0x40 || (OT.AudioObjectType != 2 && OT.AudioObjectType != 5 && OT.AudioObjectType != 29))
			{
				return 0;
			}
		}
		// At most 8 channels. Configurations 1-3, 6 and 12 are supported.
		if (!(ai.NumChannels <= 8 && (
			ai.ChannelConfiguration == 0 ||		// unspecified
			ai.ChannelConfiguration == 1 ||		// Mono
			ai.ChannelConfiguration == 2 ||		// Stereo
			ai.ChannelConfiguration == 3 ||		// L/C/R
			ai.ChannelConfiguration == 6 ||		// 5.1
			ai.ChannelConfiguration == 12)))	// 7.1
		{
			return 0;
		}
		return 1;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>() ? IElectraAudioDecoderAAC_Apple::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("Audio Toolbox")); }
	FString GetVersion() const override
	{ return FString(); }
	FString GetImplementation() const override
	{ return FString(); }
	FString GetVendor() const override
	{ return(FString(TEXT("Apple"))); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FAACAudioDecoderApple::CreateFactory()
{
	return MakeShared<FAACAudioDecoderFactoryApple, ESPMode::ThreadSafe>();
}

#endif
