// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

#if PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS

#include "IElectraCodecFactory.h"

class FAACAudioDecoderApple
{
public:
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();

};

#endif
