// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDecoderAAC_Apple.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

#include "DecoderErrors_Apple.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputAudio.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

/*********************************************************************************************************************/
#include <AudioToolbox/AudioToolbox.h>

#define MAGIC_INSUFFICIENT_INPUT_MARKER		4711
/*********************************************************************************************************************/

class FElectraAudioDecoderOutputAAC_Apple : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputAAC_Apple()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	int32 GetNumChannels() const override
	{ return NumChannels; }
	int32 GetSampleRate() const override
	{ return SampleRate; }
	int32 GetNumFrames() const override
	{ return NumFrames; }
	bool IsInterleaved() const override
	{ return true; }
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid; }
	ESampleFormat GetSampleFormat() const override
	{ return ESampleFormat::Float; }
	int32 GetBytesPerSample() const override
	{ return sizeof(float); }
	int32 GetBytesPerFrame() const override
	{ return GetBytesPerSample() * GetNumChannels(); }
	const void* GetData(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr; }

public:
	enum
	{
		kMaxChannels = 8
	};
	EChannelPosition ChannelPositions[kMaxChannels];
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};



class FElectraAudioDecoderAAC_Apple : public IElectraAudioDecoderAAC_Apple
{
public:
	FElectraAudioDecoderAAC_Apple(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraAudioDecoderAAC_Apple();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Audio; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override;
	void Resume() override;

private:
	struct FAudioConverterInstance
	{
		FAudioConverterInstance()
		{
			ConverterRef = nullptr;
			InputDescr = {};
			OutputDescr = {};
			PacketDescr = {};
			NumDecodedSamplesPerBlock = 0;
		}
		struct FWorkData
		{
			FWorkData()
			{
				Clear();
			}
			void Clear()
			{
				InputData = nullptr;
				InputSize = 0;
				InputConsumed = 0;
			}
			void* InputData;
			uint32 InputSize;
			uint32 InputConsumed;
		};
		AudioConverterRef ConverterRef;
		AudioStreamBasicDescription InputDescr;
		AudioStreamBasicDescription OutputDescr;
		AudioStreamPacketDescription PacketDescr;
		FWorkData WorkData;
		int32 NumDecodedSamplesPerBlock = 0;

		// Methods for Audio Toolbox
		OSStatus AudioToolbox_ComplexInputCallback(AudioConverterRef InAudioConverter, UInt32* InOutNumberDataPackets, AudioBufferList* InOutData, AudioStreamPacketDescription** OutDataPacketDescription);
		static OSStatus _AudioToolbox_ComplexInputCallback(AudioConverterRef InAudioConverter, UInt32* InOutNumberDataPackets, AudioBufferList* InOutData, AudioStreamPacketDescription** OutDataPacketDescription, void* InUserData)
		{
			return static_cast<FElectraAudioDecoderAAC_Apple::FAudioConverterInstance*>(InUserData)->AudioToolbox_ComplexInputCallback(InAudioConverter, InOutNumberDataPackets, InOutData, OutDataPacketDescription);
		}
	};

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining,
		Drained
	};

	bool PostError(OSStatus ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> GetCSDFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();
	bool InternalDecoderDrain();
	bool InternalDecoderFlush();

	bool ConvertDecoderOutput(int32 InNumSamplesProduced);

	IElectraDecoder::FError LastError;

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> ConfigRecord;
	TArray<FDecoderInput> InDecoderInput;
	TSharedPtr<FElectraAudioDecoderOutputAAC_Apple, ESPMode::ThreadSafe> CurrentOutput;

	FAudioConverterInstance* DecoderInstance = nullptr;

	float* PCMBuffer = nullptr;
	int32 PCMBufferSize = 0;
	int32 NumSamplesInPCMBuffer = 0;
	EDecodeState DecodeState = EDecodeState::Decoding;

	static const uint8 NumChannelsForConfig[16];
	static const IElectraDecoderAudioOutput::EChannelPosition _1[1];
	static const IElectraDecoderAudioOutput::EChannelPosition _2[2];
	static const IElectraDecoderAudioOutput::EChannelPosition _3[3];
	static const IElectraDecoderAudioOutput::EChannelPosition _4[4];
	static const IElectraDecoderAudioOutput::EChannelPosition _5[5];
	static const IElectraDecoderAudioOutput::EChannelPosition _6[6];
	static const IElectraDecoderAudioOutput::EChannelPosition _7[7];
	static const IElectraDecoderAudioOutput::EChannelPosition _8[8];
	static const IElectraDecoderAudioOutput::EChannelPosition * const MostCommonChannelMapping[8];
};
const uint8 FElectraAudioDecoderAAC_Apple::NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_1[1] = { IElectraDecoderAudioOutput::EChannelPosition::C };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_2[2] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_3[3] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_4[4] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::Cs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_5[5] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_6[6] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::LFE };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_7[7] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Cs, IElectraDecoderAudioOutput::EChannelPosition::LFE };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Apple::_8[8] = { IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Lsr, IElectraDecoderAudioOutput::EChannelPosition::Rsr, IElectraDecoderAudioOutput::EChannelPosition::LFE };
const IElectraDecoderAudioOutput::EChannelPosition * const FElectraAudioDecoderAAC_Apple::MostCommonChannelMapping[8] = { FElectraAudioDecoderAAC_Apple::_1, FElectraAudioDecoderAAC_Apple::_2, FElectraAudioDecoderAAC_Apple::_3, FElectraAudioDecoderAAC_Apple::_4, FElectraAudioDecoderAAC_Apple::_5, FElectraAudioDecoderAAC_Apple::_6, FElectraAudioDecoderAAC_Apple::_7, FElectraAudioDecoderAAC_Apple::_8 };


//-----------------------------------------------------------------------------
/**
 * Callback from the AudioConverter to get input to be converted.
 *
 * @param InAudioConverter
 * @param InOutNumberDataPackets
 * @param InOutData
 * @param OutDataPacketDescription
 */
OSStatus FElectraAudioDecoderAAC_Apple::FAudioConverterInstance::AudioToolbox_ComplexInputCallback(AudioConverterRef InAudioConverter, UInt32* InOutNumberDataPackets, AudioBufferList* InOutData, AudioStreamPacketDescription** OutDataPacketDescription)
{
	// Make sure we have input work data when being called
	if (WorkData.InputData)
	{
		if (WorkData.InputSize)
		{
			if (InOutNumberDataPackets)
			{
				*InOutNumberDataPackets = 1;
				InOutData->mNumberBuffers = 1;
				InOutData->mBuffers[0].mData = WorkData.InputData;
				InOutData->mBuffers[0].mDataByteSize = WorkData.InputSize;

				if (OutDataPacketDescription)
				{
					*OutDataPacketDescription = &PacketDescr;
					PacketDescr.mStartOffset = 0;
					PacketDescr.mVariableFramesInPacket = 0;
					PacketDescr.mDataByteSize = WorkData.InputSize;
				}
				WorkData.InputConsumed = WorkData.InputSize;
				WorkData.InputData = nullptr;
				WorkData.InputSize = 0;
			}
		}
		else
		{
			if (InOutNumberDataPackets)
			{
				*InOutNumberDataPackets = 0;
			}
			if (OutDataPacketDescription)
			{
				*OutDataPacketDescription = &PacketDescr;
				PacketDescr.mStartOffset = 0;
				PacketDescr.mVariableFramesInPacket = 0;
				PacketDescr.mDataByteSize = 0;
			}
		}
		return 0;
	}
	else
	{
		// When we have no pending input access unit we can't return data.
		// We return some dummy value to indicate that we have no data at the moment.
		// Returning 0 would indicate an EOF and make the decoder stop working.
		*InOutNumberDataPackets = 0;
		return MAGIC_INSUFFICIENT_INPUT_MARKER;
	}
}

void IElectraAudioDecoderAAC_Apple::GetConfigurationOptions(TMap<FString, FVariant>& OutFeatures)
{
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraAudioDecoderAAC_Apple::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraAudioDecoderAAC_Apple, ESPMode::ThreadSafe> New = MakeShared<FElectraAudioDecoderAAC_Apple>(InCodecFormat, InAdditionalOptions);
	return New;
}


FElectraAudioDecoderAAC_Apple::FElectraAudioDecoderAAC_Apple(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.CSD.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->Parse(InCodecFormat.CSD))
		{
			ConfigRecord = MoveTemp(NewConfigRecord);
		}
	}

	PCMBufferSize = 2048 * FElectraAudioDecoderOutputAAC_Apple::kMaxChannels * sizeof(float);
	// We accumulate output from at least two blocks before passing them out, so the buffer size must be larger than for just a single block.
	PCMBufferSize *= 4;
	PCMBuffer = (float*)FMemory::Malloc(PCMBufferSize);
	NumSamplesInPCMBuffer = 0;
}


FElectraAudioDecoderAAC_Apple::~FElectraAudioDecoderAAC_Apple()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();

	FMemory::Free(PCMBuffer);
}


TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Apple::GetCSDFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord;
		NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->ParseFrom(SidebandData.GetData(), SidebandData.Num()))
		{
			return NewConfigRecord;
		}
		else
		{
			LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
			LastError.Message = TEXT("Failed to parse codec specific data");
		}
	}
	return nullptr;
}


void FElectraAudioDecoderAAC_Apple::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
#if PLATFORM_IOS
	OutFeatures.Emplace(IElectraDecoderFeature::MustBeSuspendedInBackground, FVariant(true));
#endif
}


IElectraDecoder::FError FElectraAudioDecoderAAC_Apple::GetError() const
{
	return LastError;
}


void FElectraAudioDecoderAAC_Apple::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FElectraAudioDecoderAAC_Apple::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraAudioDecoderAAC_Apple::ResetToCleanStart()
{
	if (DecoderInstance)
	{
		InternalDecoderFlush();
	}
	InternalDecoderDestroy();

	ConfigRecord.Reset();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;
	NumSamplesInPCMBuffer = 0;

	return !LastError.IsSet();
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Apple::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining || DecodeState == EDecodeState::Drained)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode call just provides new configuration?
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) != EElectraDecoderFlags::None)
	{
		ConfigRecord.Reset();
		InternalDecoderDestroy();
	}

	// Need a valid CSD to create a decoder.
	if (!ConfigRecord.IsValid())
	{
		ConfigRecord = GetCSDFromOptions(InAdditionalOptions);
		if (!ConfigRecord.IsValid())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Create decoder transform if necessary.
	if (DecoderInstance == nullptr && !InternalDecoderCreate())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Add to the list of inputs passed to the decoder.
		FDecoderInput In;
		In.AdditionalOptions = InAdditionalOptions;
		In.AccessUnit = InInputAccessUnit;
		InDecoderInput.Emplace(MoveTemp(In));
		InDecoderInput.Sort([](const FDecoderInput& e1, const FDecoderInput& e2)
		{
			return e1.AccessUnit.PTS < e2.AccessUnit.PTS;
		});

		uint32 NumberOfChannels = DecoderInstance->OutputDescr.mChannelsPerFrame;
		uint32 SamplingRate = DecoderInstance->InputDescr.mSampleRate;

		DecoderInstance->WorkData.Clear();
		DecoderInstance->WorkData.InputData = const_cast<void*>(InInputAccessUnit.Data);
		DecoderInstance->WorkData.InputSize = InInputAccessUnit.DataSize;
		while(1)
		{
			float* CurrentOutputBufferAddr = PCMBuffer + NumSamplesInPCMBuffer * NumberOfChannels;
			int32 CurrentOutputBufferAvailSize = PCMBufferSize - NumSamplesInPCMBuffer * NumberOfChannels * sizeof(float);

			AudioBuffer OutputBuffer;
			AudioBufferList OutputBufferList;
			OutputBuffer.mNumberChannels = NumberOfChannels;
			OutputBuffer.mDataByteSize = CurrentOutputBufferAvailSize;
			OutputBuffer.mData = CurrentOutputBufferAddr;
			OutputBufferList.mNumberBuffers = 1;
			OutputBufferList.mBuffers[0] = OutputBuffer;

			UInt32 InOutPacketSize = (UInt32)DecoderInstance->NumDecodedSamplesPerBlock;
			OSStatus Result = AudioConverterFillComplexBuffer(DecoderInstance->ConverterRef, FAudioConverterInstance::_AudioToolbox_ComplexInputCallback, DecoderInstance, &InOutPacketSize, &OutputBufferList, nullptr);
			if (Result != 0 && Result != MAGIC_INSUFFICIENT_INPUT_MARKER)
			{
				PostError(Result, "Failed to decode", ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}

			if (InOutPacketSize)
			{
				NumSamplesInPCMBuffer += InOutPacketSize;

				if (InOutPacketSize * NumberOfChannels * sizeof(float) > CurrentOutputBufferAvailSize)
				{
					PostError(0, "Output buffer not large enough", ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}
			}
			if (InOutPacketSize == 0 || Result == MAGIC_INSUFFICIENT_INPUT_MARKER)
			{
				break;
			}
		}
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Apple::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining || DecodeState == EDecodeState::Drained)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance)
	{
		if (!InternalDecoderDrain())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Apple::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderInstance)
	{
		if (!InternalDecoderFlush())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Decoding;
		InDecoderInput.Empty();
		CurrentOutput.Reset();
		NumSamplesInPCMBuffer = 0;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FElectraAudioDecoderAAC_Apple::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (!DecoderInstance)
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	// Need to drain all pending input?
	if (DecodeState == EDecodeState::Draining)
	{
		uint8 NoInput[16] = {0};
		uint32 NumberOfChannels = DecoderInstance->OutputDescr.mChannelsPerFrame;
		uint32 SamplingRate = DecoderInstance->InputDescr.mSampleRate;

		DecoderInstance->WorkData.Clear();
		DecoderInstance->WorkData.InputData = NoInput;
		DecoderInstance->WorkData.InputSize = 0;
		while(1)
		{
			float* CurrentOutputBufferAddr = PCMBuffer + NumSamplesInPCMBuffer * NumberOfChannels;
			int32 CurrentOutputBufferAvailSize = PCMBufferSize - NumSamplesInPCMBuffer * NumberOfChannels * sizeof(float);

			AudioBuffer OutputBuffer;
			AudioBufferList OutputBufferList;
			OutputBuffer.mNumberChannels = NumberOfChannels;
			OutputBuffer.mDataByteSize = CurrentOutputBufferAvailSize;
			OutputBuffer.mData = CurrentOutputBufferAddr;
			OutputBufferList.mNumberBuffers = 1;
			OutputBufferList.mBuffers[0] = OutputBuffer;

			UInt32 InOutPacketSize = (UInt32)DecoderInstance->NumDecodedSamplesPerBlock;
			OSStatus Result = AudioConverterFillComplexBuffer(DecoderInstance->ConverterRef, FAudioConverterInstance::_AudioToolbox_ComplexInputCallback, DecoderInstance, &InOutPacketSize, &OutputBufferList, nullptr);
			if (Result != 0 && Result != MAGIC_INSUFFICIENT_INPUT_MARKER)
			{
				PostError(Result, "Failed to drain decoder", ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EOutputStatus::Error;
			}

			if (InOutPacketSize)
			{
				NumSamplesInPCMBuffer += InOutPacketSize;

				if (InOutPacketSize * NumberOfChannels * sizeof(float) > CurrentOutputBufferAvailSize)
				{
					PostError(0, "Output buffer not large enough", ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EOutputStatus::Error;
				}
			}
			if (InOutPacketSize == 0 || Result == MAGIC_INSUFFICIENT_INPUT_MARKER)
			{
				break;
			}
		}
		DecodeState = EDecodeState::Drained;
	}

	// Check if there is enough accumulated decoded output to produce a new packet.
	if (NumSamplesInPCMBuffer >= DecoderInstance->NumDecodedSamplesPerBlock)
	{
		return ConvertDecoderOutput(DecoderInstance->NumDecodedSamplesPerBlock) ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
	}
	else if (DecodeState == EDecodeState::Drained)
	{
		if (NumSamplesInPCMBuffer)
		{
			return ConvertDecoderOutput(DecoderInstance->NumDecodedSamplesPerBlock) ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
		}
		InDecoderInput.Empty();
		InternalDecoderFlush();
		DecodeState = EDecodeState::Decoding;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}

	return IElectraDecoder::EOutputStatus::NeedInput;
}


TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Apple::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}


void FElectraAudioDecoderAAC_Apple::Suspend()
{
}


void FElectraAudioDecoderAAC_Apple::Resume()
{
}


bool FElectraAudioDecoderAAC_Apple::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}


bool FElectraAudioDecoderAAC_Apple::InternalDecoderCreate()
{
	DecoderInstance = new FAudioConverterInstance;
	// Set input configuration specific to our current format.
	FMemory::Memzero(DecoderInstance->InputDescr);
	DecoderInstance->InputDescr.mSampleRate = ConfigRecord->ExtSamplingFrequency ? ConfigRecord->ExtSamplingFrequency : ConfigRecord->SamplingRate;
	DecoderInstance->InputDescr.mFormatID = ConfigRecord->PSSignal > 0 ? kAudioFormatMPEG4AAC_HE_V2 : ConfigRecord->SBRSignal > 0 ? kAudioFormatMPEG4AAC_HE : kAudioFormatMPEG4AAC;
	DecoderInstance->InputDescr.mFormatFlags = DecoderInstance->InputDescr.mFormatID == kAudioFormatMPEG4AAC ? kMPEG4Object_AAC_LC : kMPEG4Object_AAC_SBR;
	DecoderInstance->InputDescr.mFramesPerPacket = DecoderInstance->InputDescr.mFormatID == kAudioFormatMPEG4AAC ? 1024 : 2048;
	// Remember how many decoded samples one input access unit should decode into.
	DecoderInstance->NumDecodedSamplesPerBlock = DecoderInstance->InputDescr.mFramesPerPacket;
    // Decoding parametric stereo (PS) implies the output to go from 1 to 2 channels.
	DecoderInstance->InputDescr.mChannelsPerFrame = ConfigRecord->PSSignal > 0 ? 2 : NumChannelsForConfig[ConfigRecord->ChannelConfiguration];

	// Want LPCM output, use convenience method to set this up.
	FMemory::Memzero(DecoderInstance->OutputDescr);
	FillOutASBDForLPCM(DecoderInstance->OutputDescr, DecoderInstance->InputDescr.mSampleRate, DecoderInstance->InputDescr.mChannelsPerFrame, 32, 32, true, false, false);
	// Try to create audio converter
	OSStatus Result = AudioConverterNew(&DecoderInstance->InputDescr, &DecoderInstance->OutputDescr, &DecoderInstance->ConverterRef);
	if (Result)
	{
		InternalDecoderDestroy();
		return PostError(Result, "Failed to create decoder", ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	return true;
}


bool FElectraAudioDecoderAAC_Apple::InternalDecoderDrain()
{
	// The decoder does not allow for draining.
	return true;
}


bool FElectraAudioDecoderAAC_Apple::InternalDecoderFlush()
{
	if (DecoderInstance)
	{
		OSStatus Result = 0;
		if (DecoderInstance->ConverterRef)
		{
			Result = AudioConverterReset(DecoderInstance->ConverterRef);
		}
	}
	return true;
}


void FElectraAudioDecoderAAC_Apple::InternalDecoderDestroy()
{
	if (DecoderInstance)
	{
		OSStatus Result = 0;
		if (DecoderInstance->ConverterRef)
		{
			Result = AudioConverterReset(DecoderInstance->ConverterRef);
			check(Result == 0);	// not really that important
			Result = AudioConverterDispose(DecoderInstance->ConverterRef);
			check(Result == 0);
			DecoderInstance->ConverterRef = nullptr;
		}
		delete DecoderInstance;
		DecoderInstance = nullptr;
	}
}


bool FElectraAudioDecoderAAC_Apple::ConvertDecoderOutput(int32 InNumSamplesProduced)
{
	// We take the frontmost entry of decoder inputs. The data is not expected to be decoded out of order.
	// We also remove the frontmost entry even if conversion of the output actually fails.
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	FDecoderInput MatchingInput = InDecoderInput[0];
	InDecoderInput.RemoveAt(0);

	TSharedPtr<FElectraAudioDecoderOutputAAC_Apple, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraAudioDecoderOutputAAC_Apple>();
	NewOutput->PTS = MatchingInput.AccessUnit.PTS;
	NewOutput->UserValue = MatchingInput.AccessUnit.UserValue;

	int32 NumberOfChannels = DecoderInstance->OutputDescr.mChannelsPerFrame;
	int32 SamplingRate = DecoderInstance->InputDescr.mSampleRate;

	NewOutput->SampleRate = SamplingRate;
	NewOutput->NumChannels = NumberOfChannels;
	for(uint32 i=0; i<NewOutput->NumChannels; ++i)
	{
		NewOutput->ChannelPositions[i] = MostCommonChannelMapping[NewOutput->NumChannels-1][i];
	}
	NewOutput->NumFrames = InNumSamplesProduced;

	NewOutput->Buffer = (float*)FMemory::Malloc(NewOutput->NumFrames * NewOutput->NumChannels * sizeof(float));
	FMemory::Memcpy(NewOutput->Buffer, PCMBuffer, NewOutput->NumFrames * NewOutput->NumChannels * sizeof(float));

	// Move the remainder of the decoded data down in the accumulation buffer.
	int32 NumRemainingSamples = NumSamplesInPCMBuffer - InNumSamplesProduced;
	if (NumRemainingSamples)
	{
		float* RemainderStart = PCMBuffer + NewOutput->NumFrames * NewOutput->NumChannels;
		FMemory::Memmove(PCMBuffer, RemainderStart, NumRemainingSamples *  NewOutput->NumChannels * sizeof(float));
	}
	NumSamplesInPCMBuffer = NumRemainingSamples;

	CurrentOutput = MoveTemp(NewOutput);
	return true;
}


#endif // ELECTRA_DECODERS_ENABLE_APPLE
