// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

class IElectraAudioDecoderAAC_Apple : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraAudioDecoderAAC_Apple() = default;
};

#endif
