// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS

#include "IElectraCodecFactory.h"

class FH265VideoDecoderApple
{
public:
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();

};

#endif
