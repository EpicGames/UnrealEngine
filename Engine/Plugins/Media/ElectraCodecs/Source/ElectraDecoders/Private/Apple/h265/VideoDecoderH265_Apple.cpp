// Copyright Epic Games, Inc. All Rights Reserved.

#include "h265/VideoDecoderH265_Apple.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

#include "DecoderErrors_Apple.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersModule.h"
#include "PixelFormat.h"

#include "Apple/AppleElectraDecoderPlatformOutputFormatTypes.h"

#include <VideoToolbox/VideoToolbox.h>

/*********************************************************************************************************************/

class FElectraVideoDecoderH265_Apple;


class FElectraVideoDecoderOutputH265_Apple : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FElectraVideoDecoderOutputH265_Apple()
	{
		ReleaseOutputBuffer();
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return Width; }
	int32 GetDecodedHeight() const override
	{ return Height; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FElectraVideoDecoderOutputH265_Apple*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return 0x31637668;
	}
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		return InBufferIndex == 0 ? ImageBuffer : nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		return PixelFormat;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		return PixelEncoding;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		return Pitch;
	}

public:
	void ReleaseOutputBuffer()
	{
		if (ImageBuffer)
		{
			CFRelease(ImageBuffer);
			ImageBuffer = nullptr;
		}
	}

	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
	TMap<FString, FVariant> ExtraValues;

	CVImageBufferRef ImageBuffer = nullptr;
};



class FElectraVideoDecoderH265_Apple : public IElectraVideoDecoderH265_Apple
{
	// The decoder may not output frames in display order depending on the presence of B-frames.
	// We may need to hold back a number of frames and sort them by PTS to get them in display order.
	// This value can be configured using the "max_output_buffers" additional option.
	static constexpr int32 kDefaultNumReorderFrames = 5;

public:
	FElectraVideoDecoderH265_Apple(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraVideoDecoderH265_Apple();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorH265::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	struct FDecoderHandle
	{
		FDecoderHandle() = default;

		~FDecoderHandle()
		{
			Close();
		}

		void Close()
		{
			if (DecompressionSession)
			{
				// Note: Do not finish delayed frames here. We don't want them and have to destroy the session immediately!
				//VTDecompressionSessionFinishDelayedFrames(DecompressionSession);
				VTDecompressionSessionWaitForAsynchronousFrames(DecompressionSession);
				VTDecompressionSessionInvalidate(DecompressionSession);
				CFRelease(DecompressionSession);
				DecompressionSession = nullptr;
			}
			if (FormatDescription)
			{
				CFRelease(FormatDescription);
				FormatDescription = nullptr;
			}
		}

		void Drain()
		{
			if (DecompressionSession)
			{
				// This waits until all pending frames have been processed.
				VTDecompressionSessionWaitForAsynchronousFrames(DecompressionSession);
			}
		}

		bool IsCompatibleWith(CMFormatDescriptionRef NewFormatDescription)
		{
			if (DecompressionSession)
			{
				Boolean bIsCompatible = VTDecompressionSessionCanAcceptFormatDescription(DecompressionSession, NewFormatDescription);
				return bIsCompatible;
			}
			return false;
		}
		CMFormatDescriptionRef		FormatDescription = nullptr;
		VTDecompressionSessionRef	DecompressionSession = nullptr;
	};

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> SPS;
		bool bDropOutput = false;
	};

	struct FDecodedImage
	{
		FDecodedImage() = default;

		FDecodedImage(const FDecodedImage& rhs) : ImageBufferRef(nullptr)
		{
			InternalCopy(rhs);
		}

		FDecodedImage& operator=(const FDecodedImage& rhs)
		{
			if (this != &rhs)
			{
				InternalCopy(rhs);
			}
			return *this;
		}

		bool operator<(const FDecodedImage& rhs) const
		{
			return SourceInfo->AccessUnit.PTS < rhs.SourceInfo->AccessUnit.PTS;
		}

		void SetImageBufferRef(CVImageBufferRef InImageBufferRef)
		{
			if (ImageBufferRef)
			{
				CFRelease(ImageBufferRef);
				ImageBufferRef = nullptr;
			}
			if (InImageBufferRef)
			{
				CFRetain(InImageBufferRef);
				ImageBufferRef = InImageBufferRef;
			}
		}

		CVImageBufferRef ReleaseImageBufferRef()
		{
			CVImageBufferRef ref = ImageBufferRef;
			ImageBufferRef = nullptr;
			return ref;
		}

		~FDecodedImage()
		{
			if (ImageBufferRef)
			{
				CFRelease(ImageBufferRef);
			}
		}
		TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>	SourceInfo;
	private:
		void InternalCopy(const FDecodedImage& rhs)
		{
			SourceInfo = rhs.SourceInfo;
			SetImageBufferRef(rhs.ImageBufferRef);
		}
		CVImageBufferRef ImageBufferRef = nullptr;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	void DecodeCallback(void* pSrcRef, OSStatus status, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration);
	static void _DecodeCallback(void* pUser, void* pSrcRef, OSStatus status, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration)
	{ static_cast<FElectraVideoDecoderH265_Apple*>(pUser)->DecodeCallback(pSrcRef, status, infoFlags, imageBuffer, presentationTimeStamp, presentationDuration); }

	bool PostError(int32_t ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> GetSPSFromOptions(const TMap<FString, FVariant>& InOptions);
	bool CreateFormatDescriptionFromOptions(CMFormatDescriptionRef& OutFormatDescription, const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();
	void InternalFlushAllInputAndOutput();

	bool CheckForAvailableOutput();
	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};
	EConvertResult ConvertDecoderOutput();

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	IElectraDecoder::FError LastError;

	FCriticalSection InDecoderInputLock;
	TArray<TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>> InDecoderInput;

	TSharedPtr<FElectraVideoDecoderOutputH265_Apple, ESPMode::ThreadSafe> CurrentOutput;
	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> CurrentSPS;

	TSharedPtr<FDecoderHandle, ESPMode::ThreadSafe> Decoder;

	FCriticalSection ReadyImageMutex;
	TArray<FDecodedImage> ReadyImages;

	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bNewDecoderRequired = false;

	int32 NumReorderFrames = kDefaultNumReorderFrames;
};


namespace IElectraVideoDecoderH265_Apple_Platform
{
	static TArray<IElectraVideoDecoderH265_Apple::FSupportedConfiguration> DecoderConfigurations;
	static bool bDecoderConfigurationsDirty = true;
}


void IElectraVideoDecoderH265_Apple::PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations)
{
	if (IElectraVideoDecoderH265_Apple_Platform::bDecoderConfigurationsDirty)
	{
		IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations.Empty();

		// Perhaps these can be determined dynamically for each type of device (Mac, iPad, iPhone, AppleTV)?
		// What is really supported isn't quite clear so allow for 4K.

		// Main
		IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Apple::FSupportedConfiguration(0, 1, 0, 153, 120, 4096, 2304, 0));
		IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Apple::FSupportedConfiguration(0, 1, 0, 153, 0, 0, 0, (3840 / 8) * (2160 / 8) * 120));
		// Main10
		IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Apple::FSupportedConfiguration(0, 2, 0, 153, 120, 4096, 2304, 0));
		IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH265_Apple::FSupportedConfiguration(0, 2, 0, 153, 0, 0, 0, (3840 / 8) * (2160 / 8) * 120));

		IElectraVideoDecoderH265_Apple_Platform::bDecoderConfigurationsDirty = false;
	}
	OutSupportedConfigurations = IElectraVideoDecoderH265_Apple_Platform::DecoderConfigurations;
}


void IElectraVideoDecoderH265_Apple::GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
{
	OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
	OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	// We do not want the length of the NAL unit to be replaced with a startcode!
	OutOptions.Emplace(IElectraDecoderFeature::StartcodeToLength, FVariant(int32(0)));

	OutOptions.Emplace(IElectraDecoderFeature::NeedReplayDataOnDecoderLoss, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::MustBeSuspendedInBackground, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraVideoDecoderH265_Apple::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraVideoDecoderH265_Apple, ESPMode::ThreadSafe> New = MakeShared<FElectraVideoDecoderH265_Apple>(InCodecFormat, InAdditionalOptions);
	return New;
}

FElectraVideoDecoderH265_Apple::FElectraVideoDecoderH265_Apple(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const int32 MaxOutputBuffers = static_cast<int32>(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), kDefaultNumReorderFrames));
	NumReorderFrames = FMath::Max(MaxOutputBuffers, 1);
}

FElectraVideoDecoderH265_Apple::~FElectraVideoDecoderH265_Apple()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> FElectraVideoDecoderH265_Apple::GetSPSFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
		ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, SidebandData.GetData(), SidebandData.Num());
		for(int32 i=0; i<NALUs.Num(); ++i)
		{
			if (NALUs[i].Type == 33)
			{
				TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = MakeShared<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe>();
				if (ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(*NewSPS, ElectraDecodersUtil::AdvancePointer(SidebandData.GetData(), NALUs[i].Offset + NALUs[i].UnitLength), NALUs[i].Size))
				{
					return NewSPS;
				}
				else
				{
					break;
				}
			}
		}
		PostError(0, TEXT("Failed to parse codec specific data"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}
	return nullptr;
}



bool FElectraVideoDecoderH265_Apple::CreateFormatDescriptionFromOptions(CMFormatDescriptionRef& OutFormatDescription, const TMap<FString, FVariant>& InOptions)
{
	OSStatus Result = -1;
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
		ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, SidebandData.GetData(), SidebandData.Num());
		int32 NumRecords = NALUs.Num();
		if (NumRecords)
		{
			uint8_t const* * DataPointers = new uint8_t const* [NumRecords];
			SIZE_T* DataSizes = new SIZE_T [NumRecords];
			CFDictionaryRef NoExtras = nullptr;
			for(int32 i=0; i<NumRecords; ++i)
			{
				DataPointers[i] = ElectraDecodersUtil::AdvancePointer(SidebandData.GetData(), NALUs[i].Offset + NALUs[i].UnitLength);
				DataSizes[i] = NALUs[i].Size;
			}
			Result = CMVideoFormatDescriptionCreateFromHEVCParameterSets(kCFAllocatorDefault, NumRecords, DataPointers, DataSizes, 4, NoExtras, &OutFormatDescription);
			delete [] DataPointers;
			delete [] DataSizes;
			if (Result == 0)
			{
				return true;
			}
			else if (OutFormatDescription)
			{
				CFRelease(OutFormatDescription);
				OutFormatDescription = nullptr;
			}
		}
	}
	return PostError(Result, TEXT("Failed to create video format description from CSD"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
}


void FElectraVideoDecoderH265_Apple::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraVideoDecoderH265_Apple::GetError() const
{
	return LastError;
}

void FElectraVideoDecoderH265_Apple::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

bool FElectraVideoDecoderH265_Apple::ResetToCleanStart()
{
	InternalDecoderDestroy();
	InternalFlushAllInputAndOutput();
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

void FElectraVideoDecoderH265_Apple::InternalFlushAllInputAndOutput()
{
	InDecoderInputLock.Lock();
	InDecoderInput.Empty();
	InDecoderInputLock.Unlock();

	ReadyImageMutex.Lock();
	ReadyImages.Empty();
	ReadyImageMutex.Unlock();

	CurrentOutput.Reset();
}


IElectraDecoder::ECSDCompatibility FElectraVideoDecoderH265_Apple::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!Decoder.IsValid() || !Decoder->DecompressionSession || !Decoder->FormatDescription)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}

	CMFormatDescriptionRef NewFormatDescr = nullptr;
	if (!CreateFormatDescriptionFromOptions(NewFormatDescr, CSDAndAdditionalOptions))
	{
		UE_LOGF(LogElectraDecoders, Warning, "No CSD provided to IsCompatibleWith(), returning to drain and reset.");
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}

	Boolean bIsCompatible = VTDecompressionSessionCanAcceptFormatDescription(Decoder->DecompressionSession, NewFormatDescr);
	CFRelease(NewFormatDescr);
	// We are internally resetting the decoder after draining, so returning to drain is sufficient.
	return bIsCompatible ? IElectraDecoder::ECSDCompatibility::Compatible : IElectraDecoder::ECSDCompatibility::Drain;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Apple::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// If a new decoder is needed, destroy the current one.
	if (bNewDecoderRequired)
	{
		InternalDecoderDestroy();
	}

	// Create decoder if necessary.
	if (!Decoder.IsValid() && !InternalDecoderCreate(InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
		{
			CurrentSPS = GetSPSFromOptions(InAdditionalOptions);
			if (!CurrentSPS.IsValid())
			{
				PostError(0, TEXT("Failed to parse video SPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}

		CMBlockBufferRef AUDataBlock = nullptr;
		SIZE_T AUDataSize = (SIZE_T ) InInputAccessUnit.DataSize;
		OSStatus Result = CMBlockBufferCreateWithMemoryBlock(nullptr, nullptr, AUDataSize, nullptr, nullptr, 0, AUDataSize, kCMBlockBufferAssureMemoryNowFlag, &AUDataBlock);
		if (Result != kCMBlockBufferNoErr)
		{
			PostError(Result, TEXT("Failed to create decoder input memory block"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		Result = CMBlockBufferReplaceDataBytes(InInputAccessUnit.Data, AUDataBlock, 0, InInputAccessUnit.DataSize);
		if (Result != kCMBlockBufferNoErr)
		{
			PostError(Result, TEXT("Failed to setup decoder input memory block"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Set up the timing info with DTS, PTS and duration.
		CMSampleTimingInfo TimingInfo;
		const int64_t HNS_PER_S = 10000000;
		TimingInfo.decodeTimeStamp = CMTimeMake(InInputAccessUnit.DTS.GetTicks(), HNS_PER_S);
		TimingInfo.presentationTimeStamp = CMTimeMake(InInputAccessUnit.PTS.GetTicks(), HNS_PER_S);
		TimingInfo.duration = CMTimeMake(InInputAccessUnit.Duration.GetTicks(), HNS_PER_S);

		CMSampleBufferRef SampleBufferRef = nullptr;
		Result = CMSampleBufferCreate(kCFAllocatorDefault, AUDataBlock, true, nullptr, nullptr, Decoder->FormatDescription, 1, 1, &TimingInfo, 0, nullptr, &SampleBufferRef);
		// The buffer is now held by the sample, so we release our ref count.
		CFRelease(AUDataBlock);
		if (Result)	// see CMSampleBuffer for kCMSampleBufferError_AllocationFailed and such.
		{
			PostError(Result, TEXT("Failed to create video sample buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Add to the list of inputs passed to the decoder.
		TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> In(new FDecoderInput);
		In->AdditionalOptions = InAdditionalOptions;
		In->AccessUnit = InInputAccessUnit;
		In->bDropOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::DoNotOutput;
		In->SPS = CurrentSPS;
		// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
		// values here that would lead us to think that we do.
		In->AccessUnit.Data = nullptr;
		In->AccessUnit.DataSize = 0;
		InDecoderInputLock.Lock();
		InDecoderInput.Emplace(In);
		InDecoderInput.Sort([](const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& e1, const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& e2)
		{
			return e1->AccessUnit.PTS < e2->AccessUnit.PTS;
		});
		InDecoderInputLock.Unlock();

		// Decode
		VTDecodeFrameFlags DecodeFlags = kVTDecodeFrame_EnableAsynchronousDecompression | kVTDecodeFrame_EnableTemporalProcessing;
		if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsReplaySample | EElectraDecoderFlags::IsLastReplaySample)) != EElectraDecoderFlags::None)
		{
			DecodeFlags |= kVTDecodeFrame_DoNotOutputFrame;
		}
		VTDecodeInfoFlags  InfoFlags = 0;
		Result = VTDecompressionSessionDecodeFrame(Decoder->DecompressionSession, SampleBufferRef, DecodeFlags, In.Get(), &InfoFlags);
		CFRelease(SampleBufferRef);
		if (Result)
		{
			InDecoderInputLock.Lock();
			InDecoderInput.Remove(In);
			InDecoderInputLock.Unlock();
			// Did we lose the session?
			if (Result == kVTInvalidSessionErr)
			{
				InternalFlushAllInputAndOutput();
				bNewDecoderRequired = true;
				return IElectraDecoder::EDecoderError::LostDecoder;
			}

			PostError(Result, TEXT("Failed to decode video sample buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Apple::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a decoder send an end-of-stream and drain message.
	if (Decoder.IsValid())
	{
		DecodeState = EDecodeState::Draining;
		// Note: This call will block until all frames have been processed!
		Decoder->Drain();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH265_Apple::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	InternalDecoderDestroy();
	InternalFlushAllInputAndOutput();
	CurrentSPS.Reset();
	DecodeState = EDecodeState::Decoding;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraVideoDecoderH265_Apple::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !Decoder.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	// See if there is new output.
	if (CheckForAvailableOutput())
	{
		EConvertResult CnvRes = ConvertDecoderOutput();
		if (CnvRes == EConvertResult::Success)
		{
			return IElectraDecoder::EOutputStatus::Available;
		}
		else if (CnvRes == EConvertResult::GotEOS)
		{
			// Clear out the internal state for neatness.
			DecodeState = EDecodeState::Decoding;
			InternalFlushAllInputAndOutput();
			// For now we always create a new decoder. This could maybe be skipped
			// if the next access unit is compatible with the current configuration, but we're not sure
			// if discontinuities may be an issue so we don't risk this.
			bNewDecoderRequired = true;
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		else
		{
			return IElectraDecoder::EOutputStatus::Error;
		}
	}

	// With no output available, if we sent an EOS then we don't need new input.
	return DecodeState == EDecodeState::Draining ? IElectraDecoder::EOutputStatus::TryAgainLater : IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraVideoDecoderH265_Apple::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

void FElectraVideoDecoderH265_Apple::Suspend()
{
}

void FElectraVideoDecoderH265_Apple::Resume()
{
}

bool FElectraVideoDecoderH265_Apple::PostError(int32_t ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

bool FElectraVideoDecoderH265_Apple::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	CMFormatDescriptionRef NewFormatDescr = nullptr;
	if (!CreateFormatDescriptionFromOptions(NewFormatDescr, InAdditionalOptions))
	{
		return PostError(0, TEXT("Have no CSD to create decoder with"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}

	Decoder = MakeShared<FDecoderHandle, ESPMode::ThreadSafe>();;
	Decoder->FormatDescription = NewFormatDescr;

	VTDecompressionOutputCallbackRecord CallbackRecord;
	CallbackRecord.decompressionOutputCallback = _DecodeCallback;
	CallbackRecord.decompressionOutputRefCon   = this;

	// Output image format configuration
	CFMutableDictionaryRef OutputImageFormat = CFDictionaryCreateMutable(nullptr, 3, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

	// Allow for an array of output formats, so the decoder can deliver SDR and HDR content in suitable buffers
	constexpr int NumPixelFormats = 4;
	int PixelFormatTypes[NumPixelFormats] = {
		kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange,
		kCVPixelFormatType_420YpCbCr8BiPlanarFullRange,
		kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange,
		kCVPixelFormatType_420YpCbCr10BiPlanarFullRange
	};
	CFNumberRef PixelFormats[NumPixelFormats] = {
		CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &PixelFormatTypes[0]),
		CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &PixelFormatTypes[1]),
		CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &PixelFormatTypes[2]),
		CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &PixelFormatTypes[3])
	};
	CFArrayRef PixelFormatArray = CFArrayCreate(kCFAllocatorDefault, (const void**)PixelFormats, NumPixelFormats, &kCFTypeArrayCallBacks);

	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferPixelFormatTypeKey, PixelFormatArray);

	// Choice of: kCVPixelBufferOpenGLCompatibilityKey (all)  kCVPixelBufferOpenGLESCompatibilityKey (iOS only)   kCVPixelBufferMetalCompatibilityKey (all)
	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferMetalCompatibilityKey, kCFBooleanTrue);
#if UE_PLATFORM_IOS_ONLY || PLATFORM_TVOS
	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferOpenGLESCompatibilityKey, kCFBooleanFalse);
#endif

	CFRelease(PixelFormatArray);
	for (int Idx = 0; Idx < NumPixelFormats; ++Idx)
	{
		CFRelease(PixelFormats[Idx]);
	}

	// Session configuration
	CFMutableDictionaryRef SessionConfiguration = CFDictionaryCreateMutable(nullptr, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
	// Ask for hardware decoding
	//	CFDictionarySetValue(SessionConfiguration, kVTVideoDecoderSpecification_EnableHardwareAcceleratedVideoDecoder, kCFBooleanTrue);

	OSStatus Result = VTDecompressionSessionCreate(kCFAllocatorDefault, Decoder->FormatDescription, SessionConfiguration, OutputImageFormat, &CallbackRecord, &Decoder->DecompressionSession);
	CFRelease(SessionConfiguration);
	CFRelease(OutputImageFormat);
	if (Result != 0)
	{
		PostError(Result, TEXT("VTDecompressionSessionCreate() failed to create video decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return false;
	}

	InternalFlushAllInputAndOutput();
	CurrentSPS.Reset();
	bNewDecoderRequired = false;
	return true;
}

void FElectraVideoDecoderH265_Apple::InternalDecoderDestroy()
{
	if (Decoder.IsValid())
	{
		Decoder->Close();
		Decoder.Reset();
	}
	CurrentSPS.Reset();
	InternalFlushAllInputAndOutput();
}

bool FElectraVideoDecoderH265_Apple::CheckForAvailableOutput()
{
	// No decoder?
	if (!Decoder.IsValid())
	{
		return false;
	}

	FScopeLock lock(&ReadyImageMutex);
	bool bHaveEnough = ReadyImages.Num() >= NumReorderFrames || DecodeState == EDecodeState::Draining;
	return bHaveEnough;
}

FElectraVideoDecoderH265_Apple::EConvertResult FElectraVideoDecoderH265_Apple::ConvertDecoderOutput()
{
	FScopeLock lock(&ReadyImageMutex);
	if (!ReadyImages.Num())
	{
		return DecodeState == EDecodeState::Draining ? EConvertResult::GotEOS : EConvertResult::Success;
	}
	FDecodedImage NextImage(ReadyImages[0]);
	ReadyImages.RemoveAt(0);
	lock.Unlock();

	TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> In = NextImage.SourceInfo;

	TSharedPtr<FElectraVideoDecoderOutputH265_Apple, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraVideoDecoderOutputH265_Apple>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;

	NewOutput->Width = In->SPS->GetWidth();
	NewOutput->Height = In->SPS->GetHeight();
	NewOutput->Pitch = NewOutput->Width;

	NewOutput->ImageBuffer = NextImage.ReleaseImageBufferRef();
	int pixelFormat = CVPixelBufferGetPixelFormatType(NewOutput->ImageBuffer);

	switch(pixelFormat)
	{
		case kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr8BiPlanarFullRange:
		{
			NewOutput->PixelFormat = EPixelFormat::PF_NV12;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		case kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr10BiPlanarFullRange:
		{
			NewOutput->PixelFormat = EPixelFormat::PF_P010;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		default:
		{
			check(!"Unexpected output format!");
			NewOutput->PixelFormat = EPixelFormat::PF_Unknown;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
		}
	}

	NewOutput->NumBits = ElectraVideoDecoderFormatTypesApple::GetNumComponentBitsForPixelFormat(pixelFormat);
	if (NewOutput->NumBits < 0)
	{
		PostError(0, TEXT("Unrecognized CVPixelBuffer format (number of bits)"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	In->SPS->GetCrop(NewOutput->Crop.Left, NewOutput->Crop.Right, NewOutput->Crop.Top, NewOutput->Crop.Bottom);
	In->SPS->GetAspect(NewOutput->AspectW, NewOutput->AspectH);
	if (NewOutput->AspectW == 0 || NewOutput->AspectH == 0)
	{
		NewOutput->AspectW = 1;
		NewOutput->AspectH = 1;
	}

	ElectraDecodersUtil::FFractionalValue fr(In->SPS->GetTiming());
	if (fr.Denom)
	{
		NewOutput->FrameRateN = fr.Num;
		NewOutput->FrameRateD = fr.Denom;
	}

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("apple")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("hevc")));

	CurrentOutput = MoveTemp(NewOutput);

	return EConvertResult::Success;
}


//-----------------------------------------------------------------------------
/**
 * Callback from the video decoder when a new decoded image is ready.
 *
 * @param pSrcRef
 * @param Result
 * @param infoFlags
 * @param imageBuffer
 * @param presentationTimeStamp
 * @param presentationDuration
 */
void FElectraVideoDecoderH265_Apple::DecodeCallback(void* pSrcRef, OSStatus Result, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration)
{
	// Remove the source info even if there ultimately was a decode error or if the frame was dropped.
	TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> MatchingInput;
	InDecoderInputLock.Lock();
	int32 NumCurrentDecodeInputs = InDecoderInput.Num();
	for(int32 i=0; i<NumCurrentDecodeInputs; ++i)
	{
		if (InDecoderInput[i].Get() == pSrcRef)
		{
			MatchingInput = InDecoderInput[i];
			InDecoderInput.RemoveSingle(MatchingInput);
			break;
		}
	}
	InDecoderInputLock.Unlock();

	if (!MatchingInput.IsValid())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return;
	}

	if (Result == 0)
	{
		if (imageBuffer != nullptr && (infoFlags & kVTDecodeInfo_FrameDropped) == 0 && !MatchingInput->bDropOutput)
		{
			FDecodedImage NextImage;
			NextImage.SourceInfo = MatchingInput;
			NextImage.SetImageBufferRef(imageBuffer);
			// Recall decoded frame for later processing. All output processing happens on the decoder thread.
 			ReadyImageMutex.Lock();
			ReadyImages.Add(NextImage);
			ReadyImages.Sort();
			ReadyImageMutex.Unlock();
		}
	}
	else if (Result == kVTVideoDecoderReferenceMissingErr)
	{
		// Ignore this.
	}
	else
	{
		FString Msg = FString::Printf(TEXT("Failed to decode video: DecodeCallback() returned Result 0x%x; flags 0x%x"), Result, infoFlags);
		PostError(Result, Msg, ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	}
}

#endif
