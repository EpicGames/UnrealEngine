// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"

#ifdef ELECTRA_DECODERS_ENABLE_APPLE

class IElectraVideoDecoderH265_Apple : public IElectraDecoder
{
public:
	struct FSupportedConfiguration
	{
		FSupportedConfiguration(int32 InTier, int32 InProfile, uint32 InProfileCompatibility, int32 InLevel, int32 InFramesPerSecond, int32 InWidth, int32 InHeight, int32 InNum8x8Macroblocks)
			: Tier(InTier), Profile(InProfile), ProfileCompatibility(InProfileCompatibility), Level(InLevel), FramesPerSecond(InFramesPerSecond), Width(InWidth), Height(InHeight), Num8x8Macroblocks(InNum8x8Macroblocks)
		{}

		int32 Tier = 0;
		int32 Profile = 0;
		int32 ProfileCompatibility = 0;
		int32 Level = 0;
		int32 FramesPerSecond = 0;
		int32 Width = 0;
		int32 Height = 0;
		int32 Num8x8Macroblocks = 0;
	};
	static void PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations);

	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraVideoDecoderH265_Apple() = default;
};

#endif
