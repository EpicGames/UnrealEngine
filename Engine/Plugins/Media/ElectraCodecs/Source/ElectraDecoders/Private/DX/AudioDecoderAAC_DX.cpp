// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "DX/AudioDecoderAAC_DX.h"
#include "DX/DecoderErrors_DX.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"
#include "DSP/FloatArrayMath.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputAudio.h"

/*********************************************************************************************************************/
#include COMPILED_PLATFORM_HEADER(PlatformHeaders_Audio_DX.h)

#define VERIFY_HR(FNcall, Msg, Code)	\
Result = FNcall;						\
if (FAILED(Result))						\
{										\
	PostError(Result, Msg, Code);		\
	return false;						\
}

/*********************************************************************************************************************/

class FElectraAudioDecoderOutputAAC_DX : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputAAC_DX()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	int32 GetNumChannels() const override
	{ return NumChannels; }
	int32 GetSampleRate() const override
	{ return SampleRate; }
	int32 GetNumFrames() const override
	{ return NumFrames; }
	bool IsInterleaved() const override
	{ return true; }
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid; }
	ESampleFormat GetSampleFormat() const override
	{ return ESampleFormat::Float; }
	int32 GetBytesPerSample() const override
	{ return sizeof(float); }
	int32 GetBytesPerFrame() const override
	{ return GetBytesPerSample() * GetNumChannels(); }
	const void* GetData(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr; }

public:
	enum
	{
		kMaxChannels = 32		// There are at most 32 bits in the WAVEFORMATEXTENSIBLE to define the channel position
	};
	EChannelPosition ChannelPositions[kMaxChannels];
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};



class FElectraAudioDecoderAAC_DX : public IElectraAudioDecoderAAC_DX
{
public:
	FElectraAudioDecoderAAC_DX(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraAudioDecoderAAC_DX();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Audio; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override;
	void Resume() override;

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			ReleasePayload();
		}
		void ReleasePayload()
		{
			FMemory::Free(InputDataCopy);
			InputDataCopy = nullptr;
		}

		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		void* InputDataCopy = nullptr;
	};

	struct FDecoderOutputBuffer
	{
		~FDecoderOutputBuffer()
		{
			if (OutputBuffer.pSample)
			{
				OutputBuffer.pSample->Release();
			}
			ReleaseEvents();
		}
		TRefCountPtr<IMFSample> DetachOutputSample()
		{
			TRefCountPtr<IMFSample> pOutputSample;
			if (OutputBuffer.pSample)
			{
				pOutputSample = TRefCountPtr<IMFSample>(OutputBuffer.pSample, false);
				OutputBuffer.pSample = nullptr;
			}
			ReleaseEvents();
			return pOutputSample;
		}
		void PrepareForProcess()
		{
			ReleaseEvents();
			OutputBuffer.dwStatus = 0;
			OutputBuffer.dwStreamID = 0;
		}
		void ReleaseEvents()
		{
			if (OutputBuffer.pEvents)
			{
				OutputBuffer.pEvents->Release();
				OutputBuffer.pEvents = nullptr;
			}
		}
		MFT_OUTPUT_STREAM_INFO	OutputStreamInfo { 0 };
		MFT_OUTPUT_DATA_BUFFER	OutputBuffer { 0 };
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	void PostError(HRESULT ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> GetCSDFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();
	bool InternalDecoderDrain();
	bool InternalDecoderFlush();
	bool SetDecoderOutputType();
	bool CreateDecoderOutputBuffer();

	bool CreateInputSample(TRefCountPtr<IMFSample>& InputSample, const FInputAccessUnit& InInputAccessUnit);
	bool ConvertDecoderOutput();


	IElectraDecoder::FError LastError;

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> ConfigRecord;
	TUniquePtr<FDecoderOutputBuffer> CurrentDecoderOutputBuffer;
	TArray<FDecoderInput> InDecoderInput;
	TSharedPtr<FElectraAudioDecoderOutputAAC_DX, ESPMode::ThreadSafe> CurrentOutput;

	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bRequireDiscontinuity = false;

	TRefCountPtr<IMFTransform> DecoderTransform;
	TRefCountPtr<IMFMediaType> CurrentOutputMediaType;
	MFT_INPUT_STREAM_INFO DecoderInputStreamInfo;
	MFT_OUTPUT_STREAM_INFO DecoderOutputStreamInfo;


	static const IElectraDecoderAudioOutput::EChannelPosition ChannelBitsToPosition[32];
	static const GUID MFTmsAACDecoder_Audio;
	static const GUID MEDIASUBTYPE_RAW_AAC1_Audio;
	static const GUID MEDIASUBTYPE_PCM;
	static const GUID MEDIASUBTYPE_FLOAT;
};

const GUID FElectraAudioDecoderAAC_DX::MFTmsAACDecoder_Audio       = { 0x32d186a7, 0x218f, 0x4c75, { 0x88, 0x76, 0xdd, 0x77, 0x27, 0x3a, 0x89, 0x99 } };
const GUID FElectraAudioDecoderAAC_DX::MEDIASUBTYPE_RAW_AAC1_Audio = { 0x000000ff, 0x0000, 0x0010, { 0x80, 0x00, 0x00, 0xaa, 0x00, 0x38, 0x9b, 0x71 } };
const GUID FElectraAudioDecoderAAC_DX::MEDIASUBTYPE_PCM            = { 0x00000001, 0x0000, 0x0010, { 0x80, 0x00, 0x00, 0xaa, 0x00, 0x38, 0x9b, 0x71 } };
const GUID FElectraAudioDecoderAAC_DX::MEDIASUBTYPE_FLOAT          = { 0x00000003, 0x0000, 0x0010, { 0x80, 0x00, 0x00, 0xaa, 0x00, 0x38, 0x9b, 0x71 } };

const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_DX::ChannelBitsToPosition[32] =
{
	// The channel positions are named with "back left" and "side left" in that "back left" is
	// what is traditionally "left surround" in 5.1 and "left surround side" in 7.1 with the additional 2 surround
	// channels called "left surround rear".
	// As long as the AAC decoder only decodes 6 channels and the two surrounds have the channel mask set as
	// "SPEAKER_BACK_LEFT" and "SPEAKER_BACK_RIGHT" there's no ambiguity.
	IElectraDecoderAudioOutput::EChannelPosition::L,		// 0x1 - SPEAKER_FRONT_LEFT
	IElectraDecoderAudioOutput::EChannelPosition::R,		// 0x2 - SPEAKER_FRONT_RIGHT
	IElectraDecoderAudioOutput::EChannelPosition::C,		// 0x4 - SPEAKER_FRONT_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::LFE,		// 0x8 - SPEAKER_LOW_FREQUENCY
	IElectraDecoderAudioOutput::EChannelPosition::Ls,		// 0x10 - SPEAKER_BACK_LEFT
	IElectraDecoderAudioOutput::EChannelPosition::Rs,		// 0x20 - SPEAKER_BACK_RIGHT
	IElectraDecoderAudioOutput::EChannelPosition::Lc,		// 0x40 - SPEAKER_FRONT_LEFT_OF_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Rc,		// 0x80 - SPEAKER_FRONT_RIGHT_OF_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Cs,		// 0x100 - SPEAKER_BACK_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Lss,		// 0x200 - SPEAKER_SIDE_LEFT
	IElectraDecoderAudioOutput::EChannelPosition::Rss,		// 0x400 - SPEAKER_SIDE_RIGHT
	IElectraDecoderAudioOutput::EChannelPosition::Ts,		// 0x800 - SPEAKER_TOP_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Lv,		// 0x1000 - SPEAKER_TOP_FRONT_LEFT
	IElectraDecoderAudioOutput::EChannelPosition::Cv,		// 0x2000 - SPEAKER_TOP_FRONT_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Rv,		// 0x4000 - SPEAKER_TOP_FRONT_RIGHT
	IElectraDecoderAudioOutput::EChannelPosition::Lvs,		// 0x8000 - SPEAKER_TOP_BACK_LEFT
	IElectraDecoderAudioOutput::EChannelPosition::Cvr,		// 0x10000 - SPEAKER_TOP_BACK_CENTER
	IElectraDecoderAudioOutput::EChannelPosition::Rvs,		// 0x20000 - SPEAKER_TOP_BACK_RIGHT
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled,
	IElectraDecoderAudioOutput::EChannelPosition::Disabled
};


void IElectraAudioDecoderAAC_DX::GetConfigurationOptions(TMap<FString, FVariant>& OutFeatures)
{
}


TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraAudioDecoderAAC_DX::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraAudioDecoderAAC_DX, ESPMode::ThreadSafe> New = MakeShared<FElectraAudioDecoderAAC_DX>(InCodecFormat, InAdditionalOptions);
	return New;
}


FElectraAudioDecoderAAC_DX::FElectraAudioDecoderAAC_DX(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.CSD.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->ParseFrom(InCodecFormat.CSD.GetData(), InCodecFormat.CSD.Num()))
		{
			ConfigRecord = MoveTemp(NewConfigRecord);
		}
	}
}


FElectraAudioDecoderAAC_DX::~FElectraAudioDecoderAAC_DX()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}


TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_DX::GetCSDFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord;
		NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->ParseFrom(SidebandData.GetData(), SidebandData.Num()))
		{
			return NewConfigRecord;
		}
		else
		{
			LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
			LastError.Message = TEXT("Failed to parse codec specific data");
		}
	}
	return nullptr;
}


void FElectraAudioDecoderAAC_DX::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
}


IElectraDecoder::FError FElectraAudioDecoderAAC_DX::GetError() const
{
	return LastError;
}


void FElectraAudioDecoderAAC_DX::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}


bool FElectraAudioDecoderAAC_DX::ResetToCleanStart()
{
	if (DecoderTransform.IsValid())
	{
		InternalDecoderFlush();
	}

	DecoderTransform = nullptr;
	CurrentOutputMediaType = nullptr;
	FMemory::Memzero(DecoderInputStreamInfo);
	FMemory::Memzero(DecoderOutputStreamInfo);
	ConfigRecord.Reset();
	CurrentDecoderOutputBuffer.Reset();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;
	bRequireDiscontinuity = true;

	return !LastError.IsSet();
}


IElectraDecoder::ECSDCompatibility FElectraAudioDecoderAAC_DX::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_DX::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode call just provides new configuration?
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) != EElectraDecoderFlags::None)
	{
		ConfigRecord.Reset();
		InternalDecoderDestroy();
	}

	// Need a valid CSD to create a decoder.
	if (!ConfigRecord.IsValid())
	{
		ConfigRecord = GetCSDFromOptions(InAdditionalOptions);
		if (!ConfigRecord.IsValid())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Create decoder transform if necessary.
	if (!DecoderTransform.IsValid() && !InternalDecoderCreate())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Create input sample
		TRefCountPtr<IMFSample>	InputSample;
		if (!CreateInputSample(InputSample, InInputAccessUnit))
		{
			return IElectraDecoder::EDecoderError::Error;
		}

		while(true)
		{
			// Create output buffer if needed
			if (!CurrentDecoderOutputBuffer.IsValid() && !CreateDecoderOutputBuffer())
			{
				return IElectraDecoder::EDecoderError::Error;
			}

			DWORD dwStatus = 0;
			CurrentDecoderOutputBuffer->PrepareForProcess();
			HRESULT Result = DecoderTransform->ProcessOutput(0, 1, &CurrentDecoderOutputBuffer->OutputBuffer, &dwStatus);
			if (Result == MF_E_TRANSFORM_NEED_MORE_INPUT)
			{
				if (InputSample.IsValid())
				{
					Result = DecoderTransform->ProcessInput(0, InputSample.GetReference(), 0);
					if (FAILED(Result))
					{
						PostError(Result, TEXT("Failed to process audio decoder input"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_INPUT);
						return IElectraDecoder::EDecoderError::Error;
					}
					// Used this sample. Have no further input data for now, but continue processing to produce output if possible.
					InputSample = nullptr;


					FDecoderInput In;
					In.AdditionalOptions = InAdditionalOptions;
					In.AccessUnit = InInputAccessUnit;
					// If we need to hold on to the input data we need to make a local copy.
					// For safety reasons we zero out the pointer we were given in the input data copy to now accidentally
					// continue to use the pointer we were given that is not valid outside the scope of this method here.
					In.AccessUnit.Data = nullptr;
					// Add to the list of inputs passed to the decoder.
					InDecoderInput.Emplace(MoveTemp(In));
					InDecoderInput.Sort([](const FDecoderInput& e1, const FDecoderInput& e2)
					{
						return e1.AccessUnit.PTS < e2.AccessUnit.PTS;
					});
				}
				else
				{
					// Need more input but have none right now.
					return IElectraDecoder::EDecoderError::None;
				}
			}
			else if (Result == MF_E_TRANSFORM_STREAM_CHANGE)
			{
				if (!SetDecoderOutputType())
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				CurrentDecoderOutputBuffer.Reset();
			}
			else if (SUCCEEDED(Result))
			{
				if (!ConvertDecoderOutput())
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				// It is possible that we got output without having consumed the input. Check for this and return appropraitely.
				return InputSample.IsValid() ? IElectraDecoder::EDecoderError::NoBuffer : IElectraDecoder::EDecoderError::None;
			}
			else
			{
				PostError(Result, TEXT("Failed to process audio decoder output"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
				CurrentDecoderOutputBuffer.Reset();
				return IElectraDecoder::EDecoderError::Error;
			}
		}
	}

	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_DX::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (DecoderTransform.IsValid())
	{
		if (!InternalDecoderDrain())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_DX::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderTransform.IsValid())
	{
		if (!InternalDecoderFlush())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		HRESULT Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0);
		if (FAILED(Result))
		{
			PostError(Result, TEXT("Failed to flush and restart audio decoder"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Decoding;
		CurrentDecoderOutputBuffer.Reset();
		bRequireDiscontinuity = true;
		InDecoderInput.Empty();
		CurrentOutput.Reset();
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FElectraAudioDecoderAAC_DX::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	// Call ProcessOutput() until it asks for new input.
	if (DecoderTransform.IsValid())
	{
		while(true)
		{
			// Create output buffer if needed
			if (!CurrentDecoderOutputBuffer.IsValid() && !CreateDecoderOutputBuffer())
			{
				return IElectraDecoder::EOutputStatus::Error;
			}

			DWORD dwStatus = 0;
			CurrentDecoderOutputBuffer->PrepareForProcess();
			HRESULT Result = DecoderTransform->ProcessOutput(0, 1, &CurrentDecoderOutputBuffer->OutputBuffer, &dwStatus);
			if (Result == MF_E_TRANSFORM_NEED_MORE_INPUT)
			{
				if (DecodeState == EDecodeState::Draining)
				{
					Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_FLUSH, 0);
					check(SUCCEEDED(Result));
					Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0);
					if (FAILED(Result))
					{
						PostError(Result, TEXT("Failed to flush and restart audio decoder"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
						return IElectraDecoder::EOutputStatus::Error;
					}
					DecodeState = EDecodeState::Decoding;
					CurrentDecoderOutputBuffer.Reset();
					bRequireDiscontinuity = true;
					InDecoderInput.Empty();
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				else
				{
					return IElectraDecoder::EOutputStatus::NeedInput;
				}
			}
			else if (Result == MF_E_TRANSFORM_STREAM_CHANGE)
			{
				if (!SetDecoderOutputType())
				{
					return IElectraDecoder::EOutputStatus::Error;
				}
				CurrentDecoderOutputBuffer.Reset();
			}
			else if (SUCCEEDED(Result))
			{
				return ConvertDecoderOutput() ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
			}
			else
			{
				PostError(Result, TEXT("Failed to process audio decoder output"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
				CurrentDecoderOutputBuffer.Reset();
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
	}

	return IElectraDecoder::EOutputStatus::NeedInput;
}


TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_DX::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}


void FElectraAudioDecoderAAC_DX::Suspend()
{
}


void FElectraAudioDecoderAAC_DX::Resume()
{
}


void FElectraAudioDecoderAAC_DX::PostError(HRESULT ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
}



bool FElectraAudioDecoderAAC_DX::InternalDecoderCreate()
{
	const uint8 NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };

	TRefCountPtr<IMFTransform> Decoder;
	TRefCountPtr<IMFMediaType> MediaType;
	HRESULT Result;

	// Create decoder transform
	VERIFY_HR(CoCreateInstance(MFTmsAACDecoder_Audio, nullptr, CLSCTX_INPROC_SERVER, IID_IMFTransform, reinterpret_cast<void**>(&Decoder)), TEXT("CoCreateInstance failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);

	// Create input media type
	VERIFY_HR(MFCreateMediaType(MediaType.GetInitReference()), TEXT("MFCreateMediaType failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio), TEXT("Failed to set input media type for audio"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	UINT32 PayloadType = 0;	// 0=raw, 1=adts, 2=adif, 3=latm
	VERIFY_HR(MediaType->SetGUID(MF_MT_SUBTYPE, MEDIASUBTYPE_RAW_AAC1_Audio), TEXT("Failed to set input media audio type to RAW AAC"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetUINT32(MF_MT_AAC_PAYLOAD_TYPE, PayloadType), TEXT("Failed to set input media audio payload type"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, ConfigRecord->SamplingRate), FString::Printf(TEXT("Failed to set input audio sampling rate to %u"), ConfigRecord->SamplingRate), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	// ConfigRecord->ChannelConfiguration is in the range 0-15.
	check(ConfigRecord->ChannelConfiguration < 16);
	if (NumChannelsForConfig[ConfigRecord->ChannelConfiguration])
	{
		VERIFY_HR(MediaType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, NumChannelsForConfig[ConfigRecord->ChannelConfiguration]), FString::Printf(TEXT("Failed to set input audio number of channels for configuration %u"), ConfigRecord->ChannelConfiguration), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	VERIFY_HR(MediaType->SetBlob(MF_MT_USER_DATA, ConfigRecord->GetCodecSpecificData().GetData(), ConfigRecord->GetCodecSpecificData().Num()), TEXT("Failed to set input audio CSD"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	// Set input media type with decoder
	VERIFY_HR(Decoder->SetInputType(0, MediaType, 0), TEXT("Failed to set audio decoder input type"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	DecoderTransform = Decoder;

	// Set decoder output type to PCM
	if (!SetDecoderOutputType())
	{
		InternalDecoderDestroy();
		return false;
	}
	// Get input and output stream information from decoder
	VERIFY_HR(DecoderTransform->GetInputStreamInfo(0, &DecoderInputStreamInfo), TEXT("Failed to get audio decoder input stream information"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(DecoderTransform->GetOutputStreamInfo(0, &DecoderOutputStreamInfo), TEXT("Failed to get audio decoder output stream information"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);

	// Start the decoder transform
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_BEGIN_STREAMING, 0), TEXT("Failed to set audio decoder stream begin"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0), TEXT("Failed to start audio decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);

	return true;
}


bool FElectraAudioDecoderAAC_DX::InternalDecoderDrain()
{
	HRESULT Result;
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_END_OF_STREAM, 0), TEXT("Failed to set audio decoder end of stream notification"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_DRAIN, 0), TEXT("Failed to issue audio decoder drain command"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	return true;
}


bool FElectraAudioDecoderAAC_DX::InternalDecoderFlush()
{
	HRESULT Result;
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_END_OF_STREAM, 0), TEXT("Failed to set audio decoder end of stream notification"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_FLUSH, 0), TEXT("Failed to issue audio decoder flush command"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	return true;
}


void FElectraAudioDecoderAAC_DX::InternalDecoderDestroy()
{
	DecoderTransform = nullptr;
	CurrentOutputMediaType = nullptr;
	FMemory::Memzero(DecoderInputStreamInfo);
	FMemory::Memzero(DecoderOutputStreamInfo);
	CurrentDecoderOutputBuffer.Reset();
}


bool FElectraAudioDecoderAAC_DX::SetDecoderOutputType()
{
	TRefCountPtr<IMFMediaType> MediaType;
	HRESULT Result;
	uint32 TypeIndex = 0;
	while(SUCCEEDED(DecoderTransform->GetOutputAvailableType(0, TypeIndex++, MediaType.GetInitReference())))
	{
		GUID Subtype;
		Result = MediaType->GetGUID(MF_MT_SUBTYPE, &Subtype);

		// Prefer float over int
		if (SUCCEEDED(Result) && Subtype == MFAudioFormat_Float)
		{
			VERIFY_HR(DecoderTransform->SetOutputType(0, MediaType, 0), TEXT("Failed to set audio decoder output type"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
			CurrentOutputMediaType = MediaType;
			return true;
		}
		else if (SUCCEEDED(Result) && Subtype == MFAudioFormat_PCM)
		{
			VERIFY_HR(DecoderTransform->SetOutputType(0, MediaType, 0), TEXT("Failed to set audio decoder output type"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
			CurrentOutputMediaType = MediaType;
			return true;
		}
	}
	PostError(S_OK, TEXT("Failed to set audio decoder output type to PCM"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
	return false;
}


bool FElectraAudioDecoderAAC_DX::CreateDecoderOutputBuffer()
{
	HRESULT Result;
	TUniquePtr<FDecoderOutputBuffer> NewDecoderOutputBuffer(new FDecoderOutputBuffer);

	VERIFY_HR(DecoderTransform->GetOutputStreamInfo(0, &NewDecoderOutputBuffer->OutputStreamInfo), TEXT("Failed to get audio decoder output stream information"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
	// Do we need to provide the sample output buffer or does the decoder create it for us?
	if ((NewDecoderOutputBuffer->OutputStreamInfo.dwFlags & (MFT_OUTPUT_STREAM_PROVIDES_SAMPLES | MFT_OUTPUT_STREAM_CAN_PROVIDE_SAMPLES)) == 0)
	{
		// We have to provide the output sample buffer.
		TRefCountPtr<IMFSample> OutputSample;
		TRefCountPtr<IMFMediaBuffer> OutputBuffer;
		VERIFY_HR(MFCreateSample(OutputSample.GetInitReference()), TEXT("Failed to create output sample for audio decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		if (NewDecoderOutputBuffer->OutputStreamInfo.cbAlignment > 0)
		{
			VERIFY_HR(MFCreateAlignedMemoryBuffer(NewDecoderOutputBuffer->OutputStreamInfo.cbSize, NewDecoderOutputBuffer->OutputStreamInfo.cbAlignment, OutputBuffer.GetInitReference()), TEXT("Failed to create aligned output buffer for audio decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
		else
		{
			VERIFY_HR(MFCreateMemoryBuffer(NewDecoderOutputBuffer->OutputStreamInfo.cbSize, OutputBuffer.GetInitReference()), TEXT("Failed to create output buffer for audio decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
		VERIFY_HR(OutputSample->AddBuffer(OutputBuffer.GetReference()), TEXT("Failed to add sample buffer to output sample for audio decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		(NewDecoderOutputBuffer->OutputBuffer.pSample = OutputSample.GetReference())->AddRef();
		OutputSample = nullptr;
	}
	CurrentDecoderOutputBuffer = MoveTemp(NewDecoderOutputBuffer);
	return true;
}


bool FElectraAudioDecoderAAC_DX::CreateInputSample(TRefCountPtr<IMFSample>& InputSample, const FInputAccessUnit& InInputAccessUnit)
{
	// Create input sample
	TRefCountPtr<IMFMediaBuffer> InputSampleBuffer;
	BYTE* pbNewBuffer = nullptr;
	DWORD dwMaxBufferSize = 0;
	DWORD dwSize = 0;
	LONGLONG llSampleTime = 0;
	HRESULT Result;

	VERIFY_HR(MFCreateSample(InputSample.GetInitReference()), TEXT("Failed to create audio decoder input sample"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(MFCreateMemoryBuffer((DWORD)InInputAccessUnit.DataSize, InputSampleBuffer.GetInitReference()), TEXT("Failed to create audio decoder input sample memory buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSample->AddBuffer(InputSampleBuffer.GetReference()), TEXT("Failed to set audio decoder input buffer with sample"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSampleBuffer->Lock(&pbNewBuffer, &dwMaxBufferSize, &dwSize), TEXT("Failed to lock audio decoder input sample buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	FMemory::Memcpy(pbNewBuffer, InInputAccessUnit.Data, InInputAccessUnit.DataSize);
	VERIFY_HR(InputSampleBuffer->Unlock(), TEXT("Failed to unlock audio decoder input sample buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSampleBuffer->SetCurrentLength((DWORD)InInputAccessUnit.DataSize), TEXT("Failed to set audio decoder input sample buffer length"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	// Set sample attributes
	llSampleTime = InInputAccessUnit.PTS.GetTicks();
	VERIFY_HR(InputSample->SetSampleTime(llSampleTime), TEXT("Failed to set audio decoder input sample decode time"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	llSampleTime = InInputAccessUnit.Duration.GetTicks();
	VERIFY_HR(InputSample->SetSampleDuration(llSampleTime), TEXT("Failed to set audio decode input sample duration"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	if (bRequireDiscontinuity)
	{
		bRequireDiscontinuity = false;
		VERIFY_HR(InputSample->SetUINT32(MFSampleExtension_Discontinuity, 1), TEXT("Failed to set audio decoder input discontinuity"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	}

	return true;
}


bool FElectraAudioDecoderAAC_DX::ConvertDecoderOutput()
{
	TRefCountPtr<IMFSample> DecodedOutputSample = CurrentDecoderOutputBuffer->DetachOutputSample();
	CurrentDecoderOutputBuffer.Reset();
	if (!DecodedOutputSample.IsValid())
	{
		PostError(0, TEXT("There is no output sample to convert!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	// We take the frontmost entry of decoder inputs. The data is not expected to be decoded out of order.
	// We also remove the frontmost entry even if conversion of the output actually fails.
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	FDecoderInput MatchingInput = InDecoderInput[0];
	InDecoderInput.RemoveAt(0);

	TRefCountPtr<IMFMediaBuffer> DecodedLinearOutputBuffer;
	DWORD dwBufferLen;
	BYTE* pDecompressedData = nullptr;
	LONGLONG llTimeStamp = 0;
	WAVEFORMATEXTENSIBLE OutputWaveFormat;
	WAVEFORMATEX* OutputWaveFormatPtr = nullptr;
	UINT32 WaveFormatSize = 0;
	HRESULT Result;

	VERIFY_HR(DecodedOutputSample->GetSampleTime(&llTimeStamp), TEXT("Failed to get audio decoder output sample timestamp"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	VERIFY_HR(DecodedOutputSample->ConvertToContiguousBuffer(DecodedLinearOutputBuffer.GetInitReference()), TEXT("Failed to convert audio decoder output sample to contiguous buffer"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	VERIFY_HR(DecodedLinearOutputBuffer->GetCurrentLength(&dwBufferLen), TEXT("Failed to get audio decoder output buffer current length"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	VERIFY_HR(MFCreateWaveFormatExFromMFMediaType(CurrentOutputMediaType.GetReference(), &OutputWaveFormatPtr, &WaveFormatSize, MFWaveFormatExConvertFlag_ForceExtensible), TEXT("Failed to create audio decoder output buffer format info"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	if (WaveFormatSize < sizeof(OutputWaveFormat))
	{
		PostError(0, TEXT("Converted wave format struct is not at least a WAVEFORMATEXTENSIBLE"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	FMemory::Memcpy(&OutputWaveFormat, OutputWaveFormatPtr, sizeof(OutputWaveFormat));	//-V512
	if (OutputWaveFormatPtr->wFormatTag != WAVE_FORMAT_EXTENSIBLE)
	{
		PostError(0, TEXT("Converted wave format struct is not a WAVEFORMATEXTENSIBLE!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	CoTaskMemFree(OutputWaveFormatPtr);
	OutputWaveFormatPtr = nullptr;
	if (OutputWaveFormat.SubFormat != MEDIASUBTYPE_PCM && OutputWaveFormat.SubFormat != MEDIASUBTYPE_FLOAT)
	{
		PostError(0, TEXT("Converted wave format struct is neither int16 nor float!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	if (OutputWaveFormat.dwChannelMask == 0)
	{
		PostError(0, TEXT("Converted wave format struct has a channel mask of zero!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	TSharedPtr<FElectraAudioDecoderOutputAAC_DX, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraAudioDecoderOutputAAC_DX>();
	NewOutput->PTS = MatchingInput.AccessUnit.PTS;
	NewOutput->UserValue = MatchingInput.AccessUnit.UserValue;
	NewOutput->NumChannels = (int32) OutputWaveFormat.Format.nChannels;
	NewOutput->SampleRate = (int32) OutputWaveFormat.Format.nSamplesPerSec;

	int32 FrameSize = OutputWaveFormat.Format.wBitsPerSample / 8 * OutputWaveFormat.Format.nChannels;
	int32 NumFramesProduced = dwBufferLen / FrameSize;

	// Convert the channel mask
	int32 nCh = 0;
	for(int32 i=0; i<32; ++i)
	{
		if ((OutputWaveFormat.dwChannelMask & (1<<i)) != 0)
		{
			NewOutput->ChannelPositions[nCh++] = ChannelBitsToPosition[i];
		}
	}
	if (nCh != OutputWaveFormat.Format.nChannels)
	{
		PostError(0, TEXT("Converted wave format struct has a mismatch between number of channels and bits set in the channel mask!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	NewOutput->NumFrames = NumFramesProduced;
	NewOutput->Buffer = static_cast<float*>(FMemory::Malloc(sizeof(float) * NewOutput->NumChannels * NewOutput->NumFrames));
	if (!NewOutput->Buffer)
	{
		PostError(0, TEXT("Could not allocate memory for decoded samples!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	// Copy samples across if they are float already, otherwise convert to float.
	DWORD dwMaxBufferLen, dwBufferLen2;
	VERIFY_HR(DecodedLinearOutputBuffer->Lock(&pDecompressedData, &dwMaxBufferLen, &dwBufferLen2), TEXT("Failed to lock audio decoder output buffer"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	check(dwBufferLen == dwBufferLen2);
	float* Dest = NewOutput->Buffer;
	if (OutputWaveFormat.SubFormat == MEDIASUBTYPE_FLOAT)
	{
		const float* Source = reinterpret_cast<const float*>(pDecompressedData);
		FMemory::Memcpy(Dest, Source, sizeof(float) * NewOutput->NumChannels * NewOutput->NumFrames);
	}
	else
	{
		const int16* Source = reinterpret_cast<const int16*>(pDecompressedData);
		Audio::ArrayPcm16ToFloat(MakeArrayView(Source, NewOutput->NumChannels* NewOutput->NumFrames)
			, MakeArrayView(Dest, NewOutput->NumChannels* NewOutput->NumFrames));
	}
	DecodedLinearOutputBuffer->Unlock();
	CurrentOutput = MoveTemp(NewOutput);
	return true;
}

#undef VERIFY_HR

#endif
