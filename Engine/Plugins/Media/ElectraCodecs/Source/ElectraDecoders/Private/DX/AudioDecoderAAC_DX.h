// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "IElectraDecoder.h"

class IElectraAudioDecoderAAC_DX : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraAudioDecoderAAC_DX() = default;
};

#endif
