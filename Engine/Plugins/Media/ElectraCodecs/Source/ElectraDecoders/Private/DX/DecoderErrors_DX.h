// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_BAD_PARAMETER						1
#define ERRCODE_INTERNAL_ALREADY_CLOSED						2
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				3
#define ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE			5
#define ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER			6
#define ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE		7
#define ERRCODE_INTERNAL_FAILED_TO_PROCESS_INPUT			8
#define ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT			9
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	10
#define ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER			11
