// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "IElectraDecoder.h"

class IElectraVideoDecoderH264_DX : public IElectraDecoder
{
public:
	static bool PlatformStaticInitialize();

	struct FSupportedConfiguration
	{
		FSupportedConfiguration(int32 InProfile, int32 InLevel, int32 InFramesPerSecond, int32 InWidth, int32 InHeight, int32 InNum16x16Macroblocks)
			: Profile(InProfile), Level(InLevel), FramesPerSecond(InFramesPerSecond), Width(InWidth), Height(InHeight), Num16x16Macroblocks(InNum16x16Macroblocks)
		{}

		int32 Profile = 0;
		int32 Level = 0;
		int32 FramesPerSecond = 0;
		int32 Width = 0;
		int32 Height = 0;
		int32 Num16x16Macroblocks = 0;
	};
	static void PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations);

	static void PlatformGetConfigurationOptions(TMap<FString, FVariant>& OutOptions);

	class IPlatformHandle
	{
	protected:
		virtual ~IPlatformHandle() = default;
	public:
		virtual int32 GetDXVersionTimes1000() const = 0;
		virtual void* GetMFTransform() const = 0;
		virtual void* GetDXDevice() const = 0;
		virtual void* GetDXDeviceContext() const = 0;
		virtual bool IsSoftware() const = 0;
	};
	static FError PlatformCreateMFDecoderTransform(IPlatformHandle** OutPlatformHandle, const TMap<FString, FVariant>& InOptions);
	static void PlatformReleaseMFDecoderTransform(IPlatformHandle** OutPlatformHandle);


	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	
	virtual ~IElectraVideoDecoderH264_DX() = default;
};

#endif
