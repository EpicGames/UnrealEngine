// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "DX/VideoDecoderH265_DX.h"
#include "DX/DecoderErrors_DX.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersModule.h"
#include "PixelFormat.h"

#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)
#include COMPILED_PLATFORM_HEADER(PlatformHeaders_Video_DX.h)

#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "ID3D12DynamicRHI.h"

/*********************************************************************************************************************/

#define VERIFY_HR(FNcall, Msg, Code)	\
Result = FNcall;						\
if (FAILED(Result))						\
{										\
	PostError(Result, Msg, Code);		\
	return false;						\
}

/*********************************************************************************************************************/

#ifndef ELECTRA_HAVE_IMAGEBUFFERS
#define	ELECTRA_HAVE_IMAGEBUFFERS 0
#endif

/*********************************************************************************************************************/

DECLARE_CYCLE_STAT(TEXT("ElectraDecoder ConvertOutput"), STAT_ElectraDecoder_ConvertOutputH265, STATGROUP_Media);

/*********************************************************************************************************************/

static TAutoConsoleVariable<int32> CVarElectraWindowsH265UseOldOutputPath(
	TEXT("Electra.Win.H265UseOldOutputPath"),
	0,
	TEXT("Use old style CPU buffer output path for H265.\n")
	TEXT(" 0: use new upload heap / direct path (default); 1: use old CPU buffer path."),
	ECVF_Default);

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraVideoDecoderOutputH265_DX : public IElectraDecoderVideoOutput
#if ELECTRA_HAVE_IMAGEBUFFERS
	, public IElectraDecoderVideoOutputImageBuffers
#endif
{
public:
	virtual ~FElectraVideoDecoderOutputH265_DX()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width; }
	int32 GetHeight() const override
	{ return Height; }
	int32 GetDecodedWidth() const override
	{ return DecodedWidth; }
	int32 GetDecodedHeight() const override
	{ return DecodedHeight; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		switch (InTypeOfHandle)
		{
#if ELECTRA_HAVE_IMAGEBUFFERS
			case EElectraDecoderPlatformOutputHandleType::ImageBuffers:
			{
				return !MFSample.IsValid() ? static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FElectraVideoDecoderOutputH265_DX*>(this)) : nullptr;
			}
#endif
			case EElectraDecoderPlatformOutputHandleType::MFSample:
			{
				return MFSample.GetReference();
			}
			case EElectraDecoderPlatformOutputHandleType::DXDevice:
			{
				return Ctx.DxDevice.GetReference();
			}
			case EElectraDecoderPlatformOutputHandleType::DXDeviceContext:
			{
				return Ctx.DxDeviceContext.GetReference();
			}
			default:
			{
				return nullptr;
			}
		}
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

#if ELECTRA_HAVE_IMAGEBUFFERS
	//
	// IElectraDecoderVideoOutputImageBuffers
	//

	// Return the 4cc of the codec.
	uint32 GetCodec4CC() const override
	{
		return *(uint32*)"hevc";
	}

	// Returns the number of separate image buffers making up the frame.
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}

	// Returns the n'th image buffer as CPU data buffer.
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		// For now CPU side buffers use the old style MFSample way to return data
		return nullptr;
	}

	// Returns the n'th image buffer as GPU texture resource.
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer.Resource.GetReference();
		}
		return nullptr;
	}

	bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		if (InBufferIndex == 0)
		{
#if ALLOW_MFSAMPLE_WITH_DX12
			SyncObject = { Buffer.DecoderSync.GetReference(), 0, Buffer.MFSample };
			// note: a reference to the MFSample remains with this instance, but is assumed to soon be released as the instance will be destroyed shortly after this was called
#else
			SyncObject = { Buffer.Fence.GetReference(), Buffer.FenceValue, nullptr };
#endif
			return true;
		}
		return false;
	}

	// Returns the n'th image buffer format
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return (EPixelFormat)ElectraDecodersUtil::GetVariantValueSafeU64(ExtraValues, TEXT("pixfmt"), (uint64)EPixelFormat::PF_Unknown);
		}
		return EPixelFormat::PF_Unknown;
	}

	// Returns the n'th image buffer encoding
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return (EElectraTextureSamplePixelEncoding)ElectraDecodersUtil::GetVariantValueSafeU64(ExtraValues, TEXT("pixenc"), (uint64)EElectraTextureSamplePixelEncoding::Native);
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}

	// Returns the n'th image buffer pitch
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Pitch;
		}
		return 0;
	}
#endif // ELECTRA_HAVE_IMAGEBUFFERS

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;
	TRefCountPtr<IMFSample> MFSample;
	FElectraVideoDecoderDXDeviceContext Ctx;

#if ALLOW_MFSAMPLE_WITH_DX12
	struct {
		TRefCountPtr<ID3D12Resource> Resource;
		TRefCountPtr<IMFD3D12SynchronizationObjectCommands> DecoderSync;
		TRefCountPtr<IMFSample> MFSample;	// note: this reference is kept to allow proper passing on of ref keeping to the DX12 queue only - as this instance will be destroyed once the DX12 queue took over, this reference will not keep the MFSample around for too long
	} Buffer;
#else
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData Buffer;
#endif
};



class FElectraVideoDecoderH265_DX : public IElectraVideoDecoderH265_DX
{
public:
	FElectraVideoDecoderH265_DX(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraVideoDecoderH265_DX();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorH265::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			ReleasePayload();
		}
		void ReleasePayload()
		{
			FMemory::Free(InputDataCopy);
			InputDataCopy = nullptr;
		}

		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		void* InputDataCopy = nullptr;
		bool bDropOutput = false;
	};

	struct FDecoderOutputBuffer
	{
		~FDecoderOutputBuffer()
		{
			if (OutputBuffer.pSample)
			{
				OutputBuffer.pSample->Release();
			}
			ReleaseEvents();
		}
		TRefCountPtr<IMFSample> DetachOutputSample()
		{
			TRefCountPtr<IMFSample> pOutputSample;
			if (OutputBuffer.pSample)
			{
				pOutputSample = TRefCountPtr<IMFSample>(OutputBuffer.pSample, false);
				OutputBuffer.pSample = nullptr;
			}
			ReleaseEvents();
			return pOutputSample;
		}
		void PrepareForProcess()
		{
			ReleaseEvents();
			OutputBuffer.dwStatus = 0;
			OutputBuffer.dwStreamID = 0;
		}
		void ReleaseEvents()
		{
			if (OutputBuffer.pEvents)
			{
				OutputBuffer.pEvents->Release();
				OutputBuffer.pEvents = nullptr;
			}
		}
		MFT_OUTPUT_STREAM_INFO	OutputStreamInfo { 0 };
		MFT_OUTPUT_DATA_BUFFER	OutputBuffer { 0 };
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	void PostError(HRESULT ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> GetSPSFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();
	bool InternalDecoderDrain();
	bool InternalDecoderFlush();
	bool SetDecoderOutputType();
	bool CreateDecoderOutputBuffer();

	bool CreateInputSample(TRefCountPtr<IMFSample>& InputSample, const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	bool ConvertDecoderOutput();
	static void CopyBufferData(TRefCountPtr<IMF2DBuffer> Buffer2D, TRefCountPtr<ID3D12Resource> BufferResource, TRefCountPtr<ID3D12Fence> BufferFence, uint64 BufferFenceValue, uint32 BufferPitch, int32 DecodedHeight);

	IElectraDecoder::FError LastError;

	TUniquePtr<FDecoderOutputBuffer> CurrentDecoderOutputBuffer;
	TArray<FDecoderInput> InDecoderInput;
	TSharedPtr<FElectraVideoDecoderOutputH265_DX, ESPMode::ThreadSafe> CurrentOutput;

	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bRequireDiscontinuity = false;

	IPlatformHandle* DecoderPlatformHandle = nullptr;
	TRefCountPtr<IMFTransform> DecoderTransform;
	TRefCountPtr<IMFMediaType> CurrentOutputMediaType;

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;
	uint32 MaxOutputBuffers;
	uint32 MaxWidth;
	uint32 MaxHeight;
#if !ALLOW_MFSAMPLE_WITH_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif

	static const GUID MFTms_AVLowLatencyMode;
};

const GUID FElectraVideoDecoderH265_DX::MFTms_AVLowLatencyMode = { 0x9c27891a, 0xed7a, 0x40e1, { 0x88, 0xe8, 0xb2, 0x27, 0x27, 0xa0, 0x24, 0xee } };



TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraVideoDecoderH265_DX::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraVideoDecoderH265_DX, ESPMode::ThreadSafe> New = MakeShared<FElectraVideoDecoderH265_DX>(InCodecFormat, InAdditionalOptions);
	return New;
}

FElectraVideoDecoderH265_DX::FElectraVideoDecoderH265_DX(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	MaxWidth = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_width"), 3840), 16u);
	MaxHeight = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_height"), 2160), 16u);

	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
}


FElectraVideoDecoderH265_DX::~FElectraVideoDecoderH265_DX()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> FElectraVideoDecoderH265_DX::GetSPSFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
		ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, SidebandData.GetData(), SidebandData.Num());
		for(int32 i=0; i<NALUs.Num(); ++i)
		{
			if (NALUs[i].Type == 33)
			{
				TSharedPtr<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = MakeShared<ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet, ESPMode::ThreadSafe>();
				if (ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(*NewSPS, ElectraDecodersUtil::AdvancePointer(SidebandData.GetData(), NALUs[i].Offset + NALUs[i].UnitLength), NALUs[i].Size))
				{
					return NewSPS;
				}
				else
				{
					break;
				}
			}
		}
		LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
		LastError.Message = TEXT("Failed to parse codec specific data");
	}
	return nullptr;
}


void FElectraVideoDecoderH265_DX::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	PlatformGetConfigurationOptions(OutFeatures);
}


IElectraDecoder::FError FElectraVideoDecoderH265_DX::GetError() const
{
	return LastError;
}


void FElectraVideoDecoderH265_DX::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}


bool FElectraVideoDecoderH265_DX::ResetToCleanStart()
{
	if (DecoderTransform.IsValid())
	{
		InternalDecoderFlush();
	}

	InternalDecoderDestroy();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;
	bRequireDiscontinuity = true;

	return !LastError.IsSet();
}


IElectraDecoder::ECSDCompatibility FElectraVideoDecoderH265_DX::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!DecoderTransform.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::Drain;
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_DX::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Create decoder transform if necessary.
	if (!DecoderTransform.IsValid() && !InternalDecoderCreate(InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Create input sample
		TRefCountPtr<IMFSample>	InputSample;
		if (!CreateInputSample(InputSample, InInputAccessUnit, InAdditionalOptions))
		{
			return IElectraDecoder::EDecoderError::Error;
		}

		while(true)
		{
			// Create output buffer if needed
			if (!CurrentDecoderOutputBuffer.IsValid() && !CreateDecoderOutputBuffer())
			{
				return IElectraDecoder::EDecoderError::Error;
			}

			DWORD dwStatus = 0;
			CurrentDecoderOutputBuffer->PrepareForProcess();
			HRESULT Result = DecoderTransform->ProcessOutput(0, 1, &CurrentDecoderOutputBuffer->OutputBuffer, &dwStatus);
			if (Result == MF_E_TRANSFORM_NEED_MORE_INPUT)
			{
				if (InputSample.IsValid())
				{
					Result = DecoderTransform->ProcessInput(0, InputSample.GetReference(), 0);
					if (FAILED(Result))
					{
						PostError(Result, TEXT("Failed to process video decoder input"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_INPUT);
						return IElectraDecoder::EDecoderError::Error;
					}
					// Used this sample. Have no further input data for now, but continue processing to produce output if possible.
					InputSample = nullptr;

					FDecoderInput In;
					In.AdditionalOptions = InAdditionalOptions;
					In.bDropOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::DoNotOutput;
					In.AccessUnit = InInputAccessUnit;
					// If we need to hold on to the input data we need to make a local copy.
					// For safety reasons we zero out the pointer we were given in the input data copy to now accidentally
					// continue to use the pointer we were given that is not valid outside the scope of this method here.
					In.AccessUnit.Data = nullptr;
					// Add to the list of inputs passed to the decoder.
					InDecoderInput.Emplace(MoveTemp(In));
					InDecoderInput.Sort([](const FDecoderInput& e1, const FDecoderInput& e2)
					{
						return e1.AccessUnit.PTS < e2.AccessUnit.PTS;
					});
				}
				else
				{
					// Need more input but have none right now.
					return IElectraDecoder::EDecoderError::None;
				}
			}
			else if (Result == MF_E_TRANSFORM_STREAM_CHANGE)
			{
				if (!SetDecoderOutputType())
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				CurrentDecoderOutputBuffer.Reset();
			}
			else if (SUCCEEDED(Result))
			{
				if (!ConvertDecoderOutput())
				{
					return IElectraDecoder::EDecoderError::Error;
				}
				// It is possible that we got output without having consumed the input. Check for this and return appropriately.
				return InputSample.IsValid() ? IElectraDecoder::EDecoderError::NoBuffer : IElectraDecoder::EDecoderError::None;
			}
			else
			{
				PostError(Result, TEXT("Failed to process video decoder output"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
				CurrentDecoderOutputBuffer.Reset();
				return IElectraDecoder::EDecoderError::Error;
			}
		}
	}

	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_DX::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (DecoderTransform.IsValid())
	{
		if (!InternalDecoderDrain())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EDecoderError FElectraVideoDecoderH265_DX::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderTransform.IsValid())
	{
		if (!InternalDecoderFlush())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		HRESULT Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0);
		if (FAILED(Result))
		{
			PostError(Result, TEXT("Failed to flush and restart video decoder"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		DecodeState = EDecodeState::Decoding;
		CurrentDecoderOutputBuffer.Reset();
		bRequireDiscontinuity = true;
		InDecoderInput.Empty();
		CurrentOutput.Reset();
	}
	return IElectraDecoder::EDecoderError::None;
}


IElectraDecoder::EOutputStatus FElectraVideoDecoderH265_DX::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	// Call ProcessOutput() until it asks for new input.
	if (DecoderTransform.IsValid())
	{
		while(true)
		{
			// Create output buffer if needed
			if (!CurrentDecoderOutputBuffer.IsValid() && !CreateDecoderOutputBuffer())
			{
				return IElectraDecoder::EOutputStatus::Error;
			}

			DWORD dwStatus = 0;
			CurrentDecoderOutputBuffer->PrepareForProcess();
			HRESULT Result = DecoderTransform->ProcessOutput(0, 1, &CurrentDecoderOutputBuffer->OutputBuffer, &dwStatus);
			if (Result == MF_E_TRANSFORM_NEED_MORE_INPUT)
			{
				if (DecodeState == EDecodeState::Draining)
				{
					Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_FLUSH, 0);
					check(SUCCEEDED(Result));
					Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0);
					if (FAILED(Result))
					{
						PostError(Result, TEXT("Failed to flush and restart video decoder"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
						return IElectraDecoder::EOutputStatus::Error;
					}
					DecodeState = EDecodeState::Decoding;
					CurrentDecoderOutputBuffer.Reset();
					bRequireDiscontinuity = true;
					InDecoderInput.Empty();
					return IElectraDecoder::EOutputStatus::EndOfData;
				}
				else
				{
					return IElectraDecoder::EOutputStatus::NeedInput;
				}
			}
			else if (Result == MF_E_TRANSFORM_STREAM_CHANGE)
			{
				if (!SetDecoderOutputType())
				{
					return IElectraDecoder::EOutputStatus::Error;
				}
				CurrentDecoderOutputBuffer.Reset();
			}
			else if (SUCCEEDED(Result))
			{
				return ConvertDecoderOutput() ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
			}
			else
			{
				PostError(Result, TEXT("Failed to process video decoder output"), ERRCODE_INTERNAL_FAILED_TO_PROCESS_OUTPUT);
				CurrentDecoderOutputBuffer.Reset();
				return IElectraDecoder::EOutputStatus::Error;
			}
		}
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}


TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraVideoDecoderH265_DX::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}


void FElectraVideoDecoderH265_DX::Suspend()
{
}


void FElectraVideoDecoderH265_DX::Resume()
{
}


void FElectraVideoDecoderH265_DX::PostError(HRESULT ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
}


bool FElectraVideoDecoderH265_DX::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	TRefCountPtr<IMFAttributes>	Attributes;
	TRefCountPtr<IMFMediaType> MediaType;
	HRESULT Result;

	// Create decoder transform
	TMap<FString, FVariant> NoOptions;
	LastError = PlatformCreateMFDecoderTransform(&DecoderPlatformHandle, NoOptions);
	if (LastError.IsSet())
	{
		return false;
	}
	DecoderTransform = reinterpret_cast<IMFTransform*>(DecoderPlatformHandle->GetMFTransform());

	// Try to enable low-latency mode.
	VERIFY_HR(DecoderTransform->GetAttributes(Attributes.GetInitReference()), "Failed to get video decoder transform attributes", ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	if (FAILED(Result = Attributes->SetUINT32(MFTms_AVLowLatencyMode, 1)))
	{
		// If this did not work it's unfortunate but not a real problem.
	}

	// Create input media type
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("dcr"));
	if (SidebandData.Num() == 0)
	{
		PostError(0, TEXT("No CSD provided to create video decoder with"),ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
		return false;
	}

	VERIFY_HR(MFCreateMediaType(MediaType.GetInitReference()), TEXT("MFCreateMediaType failed"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video), TEXT("Failed to set input media type for video"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_HEVC), TEXT("Failed to set input media video type"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(MediaType->SetBlob(MF_MT_USER_DATA, SidebandData.GetData(), SidebandData.Num()), TEXT("Failed to set input video CSD"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	// We need to set the resolution of the to-be-decoded stream, otherwise setting the output type will fail for some reason.
	// If possible we get the actual resolution from the CSD, otherwise we set a common value. The exact size doesn't seem to be important.
	ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord dcr;
	if (dcr.Parse(SidebandData) && !dcr.GetParsedSequenceParameterSets().IsEmpty())
	{
		VERIFY_HR(MFSetAttributeSize(MediaType, MF_MT_FRAME_SIZE, dcr.GetParsedSequenceParameterSets()[0].GetWidth(), dcr.GetParsedSequenceParameterSets()[0].GetHeight()), "Failed to set video decoder input media type resolution", ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}
	else
	{
		VERIFY_HR(MFSetAttributeSize(MediaType, MF_MT_FRAME_SIZE, 3840, 2160), "Failed to set video decoder input media type resolution", ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	}

	// Set input media type with decoder
	VERIFY_HR(DecoderTransform->SetInputType(0, MediaType, 0), TEXT("Failed to set video decoder input type"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);

	// Set initial decoder output type
	if (!SetDecoderOutputType())
	{
		InternalDecoderDestroy();
		return false;
	}

	// Start the decoder transform
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_BEGIN_STREAMING, 0), TEXT("Failed to set video decoder stream begin"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0), TEXT("Failed to start video decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_DECODER);

	return true;
}


bool FElectraVideoDecoderH265_DX::InternalDecoderDrain()
{
	HRESULT Result;
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_END_OF_STREAM, 0), TEXT("Failed to set video decoder end of stream notification"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_DRAIN, 0), TEXT("Failed to issue video decoder drain command"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	return true;
}


bool FElectraVideoDecoderH265_DX::InternalDecoderFlush()
{
	HRESULT Result;
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_END_OF_STREAM, 0), TEXT("Failed to set video decoder end of stream notification"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	VERIFY_HR(DecoderTransform->ProcessMessage(MFT_MESSAGE_COMMAND_FLUSH, 0), TEXT("Failed to issue video decoder flush command"), ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER);
	return true;
}


void FElectraVideoDecoderH265_DX::InternalDecoderDestroy()
{
	DecoderTransform = nullptr;
	CurrentOutputMediaType = nullptr;
	CurrentDecoderOutputBuffer.Reset();
	PlatformReleaseMFDecoderTransform(&DecoderPlatformHandle);
}


bool FElectraVideoDecoderH265_DX::SetDecoderOutputType()
{
	TRefCountPtr<IMFMediaType> MediaType;
	HRESULT Result;
	uint32 TypeIndex = 0;
	// Supposedly the HEVC decoder returns MFVideoFormat_NV12 or MFVideoFormat_P010, but
	// for >8 bit streams MFVideoFormat_P010 will be returned only after decoding the first access unit!
	while(SUCCEEDED(DecoderTransform->GetOutputAvailableType(0, TypeIndex++, MediaType.GetInitReference())))
	{
		GUID MajorType, Subtype;
		VERIFY_HR(MediaType->GetGUID(MF_MT_MAJOR_TYPE, &MajorType), TEXT("Failed to set video decoder output type"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
		VERIFY_HR(MediaType->GetGUID(MF_MT_SUBTYPE, &Subtype), TEXT("Failed to set video decoder output type"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);

		if (MajorType == MFMediaType_Video && (Subtype == MFVideoFormat_P010 || Subtype == MFVideoFormat_NV12))
		{
			VERIFY_HR(DecoderTransform->SetOutputType(0, MediaType, 0), TEXT("Failed to set video decoder output type"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
			CurrentOutputMediaType = MediaType;
			return true;
		}
	}
	PostError(S_OK, TEXT("Failed to set video decoder output type to desired format"), ERRCODE_INTERNAL_FAILED_TO_SET_OUTPUT_TYPE);
	return false;
}


bool FElectraVideoDecoderH265_DX::CreateDecoderOutputBuffer()
{
	HRESULT Result;
	TUniquePtr<FDecoderOutputBuffer> NewDecoderOutputBuffer(new FDecoderOutputBuffer);

	VERIFY_HR(DecoderTransform->GetOutputStreamInfo(0, &NewDecoderOutputBuffer->OutputStreamInfo), TEXT("Failed to get video decoder output stream information"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
	// Do we need to provide the sample output buffer or does the decoder create it for us?
	if ((NewDecoderOutputBuffer->OutputStreamInfo.dwFlags & (MFT_OUTPUT_STREAM_PROVIDES_SAMPLES | MFT_OUTPUT_STREAM_CAN_PROVIDE_SAMPLES)) == 0)
	{
		// We have to provide the output sample buffer.
		TRefCountPtr<IMFSample> OutputSample;
		TRefCountPtr<IMFMediaBuffer> OutputBuffer;
		VERIFY_HR(MFCreateSample(OutputSample.GetInitReference()), TEXT("Failed to create output sample for video decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		if (NewDecoderOutputBuffer->OutputStreamInfo.cbAlignment > 0)
		{
			VERIFY_HR(MFCreateAlignedMemoryBuffer(NewDecoderOutputBuffer->OutputStreamInfo.cbSize, NewDecoderOutputBuffer->OutputStreamInfo.cbAlignment, OutputBuffer.GetInitReference()), TEXT("Failed to create aligned output buffer for video decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
		else
		{
			VERIFY_HR(MFCreateMemoryBuffer(NewDecoderOutputBuffer->OutputStreamInfo.cbSize, OutputBuffer.GetInitReference()), TEXT("Failed to create output buffer for video decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		}
		VERIFY_HR(OutputSample->AddBuffer(OutputBuffer.GetReference()), TEXT("Failed to add sample buffer to output sample for video decoder"), ERRCODE_INTERNAL_FAILED_TO_CREATE_BUFFER);
		// TRefCountPtr<> does not have a Release() method to disconnect the object, so we have to manually increment the reference
		// counter before `OutputSample` goes out of scope / is reset.
		(NewDecoderOutputBuffer->OutputBuffer.pSample = OutputSample.GetReference())->AddRef();
		OutputSample = nullptr;
	}
	CurrentDecoderOutputBuffer = MoveTemp(NewDecoderOutputBuffer);
	return true;
}


bool FElectraVideoDecoderH265_DX::CreateInputSample(TRefCountPtr<IMFSample>& InputSample, const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// Create input sample
	TRefCountPtr<IMFMediaBuffer> InputSampleBuffer;
	BYTE* pbNewBuffer = nullptr;
	DWORD dwMaxBufferSize = 0;
	DWORD dwSize = 0;
	LONGLONG llSampleTime = 0;
	HRESULT Result;

	DWORD InputDataSize = 0;
	TArray<uint8> CSD;
	// If this is a sync sample we prepend the CSD. While this is not necessary on a running stream we need to have the CSD
	// on the first frame and it is easier to prepend it to all IDR frames when seeking etc.
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
	{
		CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
	}
	InputDataSize = CSD.Num() + InInputAccessUnit.DataSize;
	VERIFY_HR(MFCreateSample(InputSample.GetInitReference()), TEXT("Failed to create video decoder input sample"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(MFCreateMemoryBuffer(InputDataSize, InputSampleBuffer.GetInitReference()), TEXT("Failed to create video decoder input sample memory buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSample->AddBuffer(InputSampleBuffer.GetReference()), TEXT("Failed to set video decoder input buffer with sample"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSampleBuffer->Lock(&pbNewBuffer, &dwMaxBufferSize, &dwSize), TEXT("Failed to lock video decoder input sample buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	FMemory::Memcpy(pbNewBuffer, CSD.GetData(), CSD.Num());
	FMemory::Memcpy(pbNewBuffer + CSD.Num(), InInputAccessUnit.Data, InInputAccessUnit.DataSize);
	VERIFY_HR(InputSampleBuffer->Unlock(), TEXT("Failed to unlock video decoder input sample buffer"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSampleBuffer->SetCurrentLength(InputDataSize), TEXT("Failed to set video decoder input sample buffer length"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	// Set sample attributes
	llSampleTime = InInputAccessUnit.PTS.GetTicks();
	VERIFY_HR(InputSample->SetSampleTime(llSampleTime), TEXT("Failed to set video decoder input sample decode time"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	llSampleTime = InInputAccessUnit.DTS.GetTicks();
	VERIFY_HR(InputSample->SetUINT64(MFSampleExtension_DecodeTimestamp, llSampleTime), TEXT("Failed to set video decoder input sample decode time"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	llSampleTime = InInputAccessUnit.Duration.GetTicks();
	VERIFY_HR(InputSample->SetSampleDuration(llSampleTime), TEXT("Failed to set video decode input sample duration"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	VERIFY_HR(InputSample->SetUINT32(MFSampleExtension_CleanPoint, ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None) ? 1 : 0), TEXT("Failed to set video decoder input sample clean point"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	if (bRequireDiscontinuity)
	{
		bRequireDiscontinuity = false;
		VERIFY_HR(InputSample->SetUINT32(MFSampleExtension_Discontinuity, 1), TEXT("Failed to set video decoder input discontinuity"), ERRCODE_INTERNAL_FAILED_TO_CREATE_INPUT_SAMPLE);
	}

	return true;
}

#if !ALLOW_MFSAMPLE_WITH_DX12
void FElectraVideoDecoderH265_DX::CopyBufferData(TRefCountPtr<IMF2DBuffer> Buffer2D, TRefCountPtr<ID3D12Resource> BufferResource, TRefCountPtr<ID3D12Fence> BufferFence, uint64 BufferFenceValue, uint32 BufferPitch, int32 DecodedHeight)
{
	uint8* Data = nullptr;
	LONG Pitch = 0;
	TRefCountPtr<IMF2DBuffer2> Buffer2D2;
	HRESULT Res = Buffer2D->QueryInterface(IID_PPV_ARGS(Buffer2D2.GetInitReference()));
	if (SUCCEEDED(Res))
	{
		uint8* Start = nullptr;
		DWORD Length = 0;
		Res = Buffer2D2->Lock2DSize(MF2DBuffer_LockFlags_Read, &Data, &Pitch, &Start, &Length);
		if (SUCCEEDED(Res))
		{
			check(Length >= static_cast<DWORD>(Pitch * DecodedHeight * 3 / 2));
		}
		else
		{
			checkNoEntry();
		}
	}
	else
	{
		Res = Buffer2D->Lock2D(&Data, &Pitch);
		check(SUCCEEDED(Res));
	}

	ID3D12DynamicRHI* const D3D12RHI = GetID3D12DynamicRHI();
	ID3D12Device* const Device = D3D12RHI->RHIGetDevice_NoMGPU();

	void* BufferAddr;
	Res = BufferResource->Map(0, nullptr, &BufferAddr);
	if (FAILED(Res))
	{
		D3D12RHI->RHIVerifyResult(Device, Res, "FElectraVideoDecoderH265_DX::CopyBufferData Map", __FILE__, __LINE__);
	}

	FElectraMediaDecoderOutputBufferPool_DX12::CopyWithPitchAdjust((uint8*)BufferAddr, BufferPitch, Data, Pitch, DecodedHeight * 3 / 2);

	BufferResource->Unmap(0, nullptr);

	Buffer2D->Unlock2D();

	Res = BufferFence->Signal(BufferFenceValue);
	if (FAILED(Res))
	{
		D3D12RHI->RHIVerifyResult(Device, Res, "FElectraVideoDecoderH265_DX::CopyBufferData Signal", __FILE__, __LINE__);
	}
};
#endif

bool FElectraVideoDecoderH265_DX::ConvertDecoderOutput()
{
	SCOPE_CYCLE_COUNTER(STAT_ElectraDecoder_ConvertOutputH265);

	TRefCountPtr<IMFSample> DecodedOutputSample = CurrentDecoderOutputBuffer->DetachOutputSample();
	CurrentDecoderOutputBuffer.Reset();
	if (!DecodedOutputSample.IsValid())
	{
		PostError(0, TEXT("There is no output sample to convert!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}

	HRESULT Result;
	FString ResultMsg;
	LONGLONG llTimeStamp = 0;
	VERIFY_HR(DecodedOutputSample->GetSampleTime(&llTimeStamp), TEXT("Failed to get video decoder output sample timestamp"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	FTimespan TimeStamp((int64) llTimeStamp);

	// Find the input corresponding to this output.
	FDecoderInput MatchingInput;
	bool bFoundMatch = false;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec].AccessUnit.PTS == TimeStamp)
		{
			MatchingInput = InDecoderInput[nInDec];
			bFoundMatch = true;
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}

	// If no exact match is found then the decoder transform may have interpolated the PTS given the individual
	// frame durations or perhaps due to SEI messages.
	if (!bFoundMatch)
	{
		MatchingInput = InDecoderInput[0];
		bFoundMatch = true;
		InDecoderInput.RemoveAt(0);
	}
	/*
	if (!bFoundMatch)
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	*/

	if (MatchingInput.bDropOutput)
	{
		return true;
	}

	TSharedPtr<FElectraVideoDecoderOutputH265_DX, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraVideoDecoderOutputH265_DX>();
	NewOutput->PTS = MatchingInput.AccessUnit.PTS;
	NewOutput->UserValue = MatchingInput.AccessUnit.UserValue;

	// Get information from the decoded sample. It is best to use the values as provided over those we have from the source
	// since the image may have undergone cropping or other transformations.
	UINT32	dwInputWidth = 0;
	UINT32	dwInputHeight = 0;
	VERIFY_HR(MFGetAttributeSize(CurrentOutputMediaType.GetReference(), MF_MT_FRAME_SIZE, &dwInputWidth, &dwInputHeight), TEXT("Failed to get decoded image dimensions"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	MFVideoArea videoArea = {};
	Result = CurrentOutputMediaType->GetBlob(MF_MT_MINIMUM_DISPLAY_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
	if (FAILED(Result))
	{
		Result = CurrentOutputMediaType->GetBlob(MF_MT_GEOMETRIC_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
		if (FAILED(Result))
		{
			videoArea.OffsetX.fract = 0;
			videoArea.OffsetX.value = 0;
			videoArea.OffsetY.fract = 0;
			videoArea.OffsetY.value = 0;
			videoArea.Area.cx = dwInputWidth;
			videoArea.Area.cy = dwInputHeight;
		}
	}

	// Width and height of the final output image
	NewOutput->Width = (int32)videoArea.Area.cx;
	NewOutput->Height = (int32)videoArea.Area.cy;

	// Width and height as "decoded dims" (which are featuring format specific scales & no cropping)
	NewOutput->DecodedWidth = (int32)dwInputWidth;
	NewOutput->DecodedHeight = (int32)dwInputHeight;
	// If we deliver the data in a CPU side buffer, we need to patch the decoded height to reflect the oddness of these planar formats
	// (DirectX12 and higher (unless we can use the render device to decode), SW decoder or a non-DirectX API will trigger this)
	if (DecoderPlatformHandle->IsSoftware() || (DecoderPlatformHandle->GetDXVersionTimes1000() >= 12000 && DecoderPlatformHandle->GetDXDevice() != nullptr) || DecoderPlatformHandle->GetDXVersionTimes1000() == 0)
	{
#if !ALLOW_MFSAMPLE_WITH_DX12
		if (DecoderPlatformHandle->GetDXVersionTimes1000() < 12000 || CVarElectraWindowsH265UseOldOutputPath.GetValueOnAnyThread() != 0)
#endif
		{
			// We are returning a CPU side buffer (software decode OR DX12 render device) - adjust the height so it can be interpreted as a single plane texture
			NewOutput->DecodedHeight = NewOutput->DecodedHeight * 3 / 2;
		}
	}

	// Note: Decoder crops at the lower right border.
	NewOutput->Crop.Left = 0;
	NewOutput->Crop.Top = 0;
	NewOutput->Crop.Right = dwInputWidth - videoArea.Area.cx;
	NewOutput->Crop.Bottom = dwInputHeight - videoArea.Area.cy;

	// Try to get the stride. Defaults to 0 should it not be obtainable.
	UINT32 stride = MFGetAttributeUINT32(CurrentOutputMediaType, MF_MT_DEFAULT_STRIDE, 0);

	// Try to get the frame rate ratio.
	UINT32 num = 0, denom = 0;
	MFGetAttributeRatio(CurrentOutputMediaType, MF_MT_FRAME_RATE, &num, &denom);
	NewOutput->FrameRateN = num;
	NewOutput->FrameRateD = denom;

	// Try to get the pixel aspect ratio.
	num = 0, denom = 0;
	MFGetAttributeRatio(CurrentOutputMediaType, MF_MT_PIXEL_ASPECT_RATIO, &num, &denom);
	if (!num || !denom)
	{
		num = 1;
		denom = 1;
	}
	NewOutput->AspectW = num;
	NewOutput->AspectH = denom;

	// Get actual output format used right now.
	GUID CurrentVideoFormatGUID;
	EPixelFormat PixFmt;
	if (SUCCEEDED(CurrentOutputMediaType->GetGUID(MF_MT_SUBTYPE, &CurrentVideoFormatGUID)))
	{
		check(CurrentVideoFormatGUID == MFVideoFormat_NV12 || CurrentVideoFormatGUID == MFVideoFormat_P010);
	}
	else
	{
		CurrentVideoFormatGUID = MFVideoFormat_NV12;
	}
	if (CurrentVideoFormatGUID == MFVideoFormat_NV12)
	{
		NewOutput->NumBits = 8;
		NewOutput->PixelFormat = 0;
		PixFmt = EPixelFormat::PF_NV12;
	}
	else
	{
		NewOutput->NumBits = 10;
		NewOutput->PixelFormat = 1;
		PixFmt = EPixelFormat::PF_P010;
	}
	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("dx")));
	NewOutput->ExtraValues.Emplace(TEXT("dxversion"), FVariant((int64)DecoderPlatformHandle->GetDXVersionTimes1000()));
	NewOutput->ExtraValues.Emplace(TEXT("sw"), FVariant(DecoderPlatformHandle->IsSoftware()));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("hevc")));
	NewOutput->ExtraValues.Emplace(TEXT("pixfmt"), FVariant((int64)PixFmt));
	NewOutput->ExtraValues.Emplace(TEXT("pixenc"), FVariant((int64)EElectraTextureSamplePixelEncoding::Native));

#if ELECTRA_HAVE_IMAGEBUFFERS
	void* PlatformDevice = nullptr;
	int32 PlatformDeviceVersion = 0;
	bool bUseGPUBuffers = false;
	if (CVarElectraWindowsH265UseOldOutputPath.GetValueOnAnyThread() == 0)
	{
		FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
		bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
	}

	// If we don't have support for DX12 direct to video decoding, yet, we bring the CPU buffer data into a DX12 resource now
	if (bUseGPUBuffers)
	{
		DWORD BuffersNum = 0;
		if ((Result = DecodedOutputSample->GetBufferCount(&BuffersNum)) != S_OK)
		{
			PostError(Result, TEXT("Failed to get buffer count"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}
		if (BuffersNum != 1)
		{
			PostError(0, TEXT("Invalid buffer count"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}
		TRefCountPtr<IMFMediaBuffer> Buffer;
		if ((Result = DecodedOutputSample->GetBufferByIndex(0, Buffer.GetInitReference())) != S_OK)
		{
			PostError(Result, TEXT("Failed to get buffer by index"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}

#if !ALLOW_MFSAMPLE_WITH_DX12
		//
		// Pre SDK 22621: use CPU copy to get data from MFSample to DX12 resource
		//
		TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));

		// Create the resource pool as needed...
		if (!D3D12ResourcePool || !D3D12ResourcePool->IsCompatibleAsBuffer(MaxOutputBuffers, MaxWidth, MaxHeight * 3 / 2, (PixFmt == EPixelFormat::PF_NV12) ? 1 : 2))
		{
			D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, MaxWidth, MaxHeight * 3 / 2, (PixFmt == EPixelFormat::PF_NV12) ? 1 : 2);
		}

		// Get data from MFSample...
		TRefCountPtr<IMF2DBuffer> Buffer2D;
		if ((Result = Buffer->QueryInterface(__uuidof(IMF2DBuffer), (void**)Buffer2D.GetInitReference())) != S_OK)
		{
			PostError(Result, TEXT("Failed to get 2d buffer from buffer"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}

		// Request resource and fence...
		uint32 BufferPitch;
		if (!D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, NewOutput->Buffer, BufferPitch))
		{
			PostError(Result, FString::Printf(TEXT("Failed to allocate D3D12 data buffer (%s)"), *ResultMsg), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return false;
		}

		NewOutput->Pitch = BufferPitch;
		CopyBufferData(Buffer2D, NewOutput->Buffer.Resource, NewOutput->Buffer.Fence, NewOutput->Buffer.FenceValue, BufferPitch, NewOutput->DecodedHeight);
#else
		//
		// Post SDK 22621: direct link from MFSample to DX12 resource
		//

		TRefCountPtr<IMFDXGIBuffer> DXGIBuffer;
		if ((Result = Buffer->QueryInterface(__uuidof(IMFDXGIBuffer), (void**)DXGIBuffer.GetInitReference())) != S_OK)
		{
			return false;
		}

		TRefCountPtr<ID3D12Resource> Resource;
		Result = DXGIBuffer->GetResource(__uuidof(ID3D12Resource), (void**)Resource.GetInitReference());
		if (Result == S_OK)
		{
			// Yes, set this up...
			TRefCountPtr<IMFD3D12SynchronizationObjectCommands> DecoderSync;
			if ((Result = DXGIBuffer->GetUnknown(MF_D3D12_SYNCHRONIZATION_OBJECT, IID_PPV_ARGS(DecoderSync.GetInitReference()))) != S_OK)
			{
				return false;
			}

			NewOutput->Buffer.Resource = Resource;
			NewOutput->Buffer.DecoderSync = DecoderSync;
			NewOutput->Buffer.MFSample = DecodedOutputSample;	// we keep a reference around until we actually submit the sync object to a DX12 queue
		}
#endif // !ALLOW_MFSAMPLE_WITH_DX12
	}
	else
#endif // ELECTRA_HAVE_IMAGEBUFFERS
	{
		//
		// Fallback output path for pre- / none-DX12 graphics APIs
		//

		// Retain the IMFSample in the output. It is needed later in converting it for display.
		NewOutput->Pitch = stride;
		NewOutput->MFSample = DecodedOutputSample;
	}

	// Do the same for the HW acceleration device.
	NewOutput->Ctx.SetDeviceAndContext(DecoderPlatformHandle->GetDXDevice(), DecoderPlatformHandle->GetDXDeviceContext());

	CurrentOutput = MoveTemp(NewOutput);
	return true;
}

#undef VERIFY_HR

#endif
