// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "IElectraDecoder.h"

class IElectraVideoDecoderH265_DX : public IElectraDecoder
{
public:
	static bool PlatformStaticInitialize();

	struct FSupportedConfiguration
	{
		FSupportedConfiguration(int32 InTier, int32 InProfile, uint32 InProfileCompatibility, int32 InLevel, int32 InFramesPerSecond, int32 InWidth, int32 InHeight, int32 InNum8x8Macroblocks)
			: Tier(InTier), Profile(InProfile), ProfileCompatibility(InProfileCompatibility), Level(InLevel), FramesPerSecond(InFramesPerSecond), Width(InWidth), Height(InHeight), Num8x8Macroblocks(InNum8x8Macroblocks)
		{}

		int32 Tier = 0;
		int32 Profile = 0;
		int32 ProfileCompatibility = 0;
		int32 Level = 0;
		int32 FramesPerSecond = 0;
		int32 Width = 0;
		int32 Height = 0;
		int32 Num8x8Macroblocks = 0;
	};
	static void PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations);

	static void PlatformGetConfigurationOptions(TMap<FString, FVariant>& OutOptions);

	class IPlatformHandle
	{
	protected:
		virtual ~IPlatformHandle() = default;
	public:
		virtual int32 GetDXVersionTimes1000() const = 0;
		virtual void* GetMFTransform() const = 0;
		virtual void* GetDXDevice() const = 0;
		virtual void* GetDXDeviceContext() const = 0;
		virtual bool IsSoftware() const = 0;
	};
	static FError PlatformCreateMFDecoderTransform(IPlatformHandle** OutPlatformHandle, const TMap<FString, FVariant>& InOptions);
	static void PlatformReleaseMFDecoderTransform(IPlatformHandle** OutPlatformHandle);


	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraVideoDecoderH265_DX() = default;
};

#endif
