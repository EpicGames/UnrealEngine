// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraDecodersModule.h"

#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

#include "IElectraDecodersModule.h"
#include "PlatformElectraDecoders.h"

#include "Utils/ElectraCodecFormatUtils.h"

// Common codecs
#include "ElectraMediaMP3Decoder.h"

#define LOCTEXT_NAMESPACE "ElectraDecodersModule"

DEFINE_LOG_CATEGORY(LogElectraDecoders);


class FElectraDecodersModule : public IElectraDecodersModule
{
public:
	void StartupModule() override
	{
		FElectraCodecFormatUtilsModularFeature::Startup();
		FPlatformElectraDecoders::Startup();
		FElectraMediaMP3Decoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaMP3Decoder::Shutdown();
		FPlatformElectraDecoders::Shutdown();
		FElectraCodecFormatUtilsModularFeature::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
	void RegisterDecodersWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith) override
	{
		FPlatformElectraDecoders::RegisterWithCodecFactory(InCodecFactoryToRegisterWith);
	}
};

IMPLEMENT_MODULE(FElectraDecodersModule, ElectraDecoders);

#undef LOCTEXT_NAMESPACE
