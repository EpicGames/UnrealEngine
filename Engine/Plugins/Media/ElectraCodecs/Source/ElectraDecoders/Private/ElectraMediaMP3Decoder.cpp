// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaMP3Decoder.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputAudio.h"
#include "ElectraDecodersModule.h"
#include "ElectraDecodersUtils.h"
#include "Utilities/ElectraBitstream.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#ifndef WITH_DR_LIBS_MP3
#define WITH_DR_LIBS_MP3 0
#endif

#if WITH_DR_LIBS_MP3
#include "dr_libs_mp3decoder.h"
#endif

#define ERRCODE_MP3DEC_INTERNAL_NO_ERROR						0
#define ERRCODE_MP3DEC_INTERNAL_ALREADY_CLOSED					1
#define ERRCODE_MP3DEC_INTERNAL_FAILED_TO_PARSE_CSD				2
#define ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT			3
#define ERRCODE_MP3DEC_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT		4

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraAudioDecoderOutputMP3_Common : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputMP3_Common()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	int32 GetNumChannels() const override
	{
		return NumChannels;
	}
	int32 GetSampleRate() const override
	{
		return SampleRate;
	}
	int32 GetNumFrames() const override
	{
		return NumFrames;
	}
	bool IsInterleaved() const override
	{
		return true;
	}
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < ChannelPositions.Num() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid;
	}
	ESampleFormat GetSampleFormat() const override
	{
		return ESampleFormat::Float;
	}
	int32 GetBytesPerSample() const override
	{
		return sizeof(float);
	}
	int32 GetBytesPerFrame() const override
	{
		return GetBytesPerSample() * GetNumChannels();
	}
	const void* GetData(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr;
	}

public:
	TArray<EChannelPosition> ChannelPositions;
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};


class FElectraMP3Decoder : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
	}

	FElectraMP3Decoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraMP3Decoder();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Audio;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FMpegAudioConfig
	{
		int32 SampleRate = 0;
		int32 NumberOfChannels = 0;
		int32 SamplesPerFrame = 0;
		int32 Bitrate = 0;
		void Reset()
		{
			SampleRate = 0;
			NumberOfChannels = 0;
		}
		bool SameAs(const FMpegAudioConfig& rhs)
		{
			return SampleRate == rhs.SampleRate && NumberOfChannels == rhs.NumberOfChannels;
		}
	};

	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	bool ParseMpegAudioHeader(FMpegAudioConfig& OutConfig, const void* InData, int64 InDataSize);

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	bool SetupChannelMap();

	IElectraDecoder::FError LastError;

#if WITH_DR_LIBS_MP3
	FMP3Decoder* DecoderHandle = nullptr;
#else
	void* DecoderHandle = nullptr;
#endif

	TSharedPtr<FElectraAudioDecoderOutputMP3_Common, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;

	// Input configuration
	FMpegAudioConfig MpegConfig;
	bool bHaveParsedMpegHeader = false;

	// Output
	TArray<IElectraDecoderAudioOutput::EChannelPosition> OutputChannelMap;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraCommonAudioMP3DecoderFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FElectraCommonAudioMP3DecoderFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FElectraCommonAudioMP3DecoderFactory()
	{ }

	FElectraCommonAudioMP3DecoderFactory()
	{
#if WITH_DR_LIBS_MP3 && defined(DR_LIBS_VERSION_NUMBER)
		LibVersion = ANSI_TO_TCHAR(DR_LIBS_VERSION_NUMBER);
#endif
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FElectraMP3Decoder::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Audio || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			return 0;
		}
		if (!Permitted4CCs.Contains(InCodecFormat.RFC6381))
		{
			return 0;
		}
		return 5;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>() ? MakeShared<FElectraMP3Decoder, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("dr_libs")); }
	FString GetVersion() const override
	{ return LibVersion; }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }

	FString LibVersion;
	static TSharedPtr<FElectraCommonAudioMP3DecoderFactory, ESPMode::ThreadSafe> Self;
	static TArray<FString> Permitted4CCs;
};
TSharedPtr<FElectraCommonAudioMP3DecoderFactory, ESPMode::ThreadSafe> FElectraCommonAudioMP3DecoderFactory::Self;
TArray<FString> FElectraCommonAudioMP3DecoderFactory::Permitted4CCs = { TEXT("mp4a.6b"), TEXT("mp4a.40.34"), TEXT(".mp3") };

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaMP3Decoder::Startup()
{
#if WITH_DR_LIBS_MP3
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FElectraCommonAudioMP3DecoderFactory::Self.IsValid());
	FElectraCommonAudioMP3DecoderFactory::Self = MakeShared<FElectraCommonAudioMP3DecoderFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioMP3DecoderFactory::Self.Get());
#endif
}

void FElectraMediaMP3Decoder::Shutdown()
{
#if WITH_DR_LIBS_MP3
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioMP3DecoderFactory::Self.Get());
	FElectraCommonAudioMP3DecoderFactory::Self.Reset();
#endif
}

TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FElectraMediaMP3Decoder::CreateFactory()
{
	return MakeShared<FElectraCommonAudioMP3DecoderFactory, ESPMode::ThreadSafe>();;
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FElectraMP3Decoder::FElectraMP3Decoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
}


FElectraMP3Decoder::~FElectraMP3Decoder()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_MP3DEC_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

bool FElectraMP3Decoder::ParseMpegAudioHeader(FMpegAudioConfig& OutConfig, const void* InData, int64 InDataSize)
{
	if (InDataSize >= 4)
	{
		const uint8* Data = reinterpret_cast<const uint8*>(InData);
		const uint32 HeaderValue = (static_cast<uint32>(Data[0]) << 24) | (static_cast<uint32>(Data[1]) << 16) | (static_cast<uint32>(Data[2]) << 8) | static_cast<uint32>(Data[3]);
		if (ElectraDecodersUtil::MPEG::UtilsMPEG123::HasValidSync(HeaderValue))
		{
			OutConfig.SampleRate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplingRate(HeaderValue);
			OutConfig.NumberOfChannels = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetChannelCount(HeaderValue);
			OutConfig.Bitrate = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetBitrate(HeaderValue);
			OutConfig.SamplesPerFrame = ElectraDecodersUtil::MPEG::UtilsMPEG123::GetSamplesPerFrame(HeaderValue);
			return true;
		}
	}
	return false;
}



bool FElectraMP3Decoder::InternalDecoderCreate()
{
#if WITH_DR_LIBS_MP3
	if (!DecoderHandle)
	{
		DecoderHandle = new FMP3Decoder;
	}
	return true;
#else
	return false;
#endif
}

void FElectraMP3Decoder::InternalDecoderDestroy()
{
#if WITH_DR_LIBS_MP3
	if (DecoderHandle)
	{
		delete DecoderHandle;
		DecoderHandle = nullptr;
	}
#endif
}


void FElectraMP3Decoder::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraMP3Decoder::GetError() const
{
	return LastError;
}

bool FElectraMP3Decoder::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FElectraMP3Decoder::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_MP3DEC_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FElectraMP3Decoder::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// No configuration parsed yet, so this is deemed compatible.
	if (!bHaveParsedMpegHeader)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	// There is no CSD for MPEG audio, so we can't assume compatibility
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraMP3Decoder::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();

	bHaveParsedMpegHeader = false;
	MpegConfig.Reset();
	OutputChannelMap.Empty();
	InternalDecoderDestroy();
	return true;
}

bool FElectraMP3Decoder::SetupChannelMap()
{
	if (OutputChannelMap.Num())
	{
		return true;
	}
	// Pre-init with all channels disabled.
	OutputChannelMap.Empty();
	OutputChannelMap.Init(IElectraDecoderAudioOutput::EChannelPosition::Disabled, MpegConfig.NumberOfChannels);
	if (MpegConfig.NumberOfChannels == 1)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::C;
	}
	else if (MpegConfig.NumberOfChannels == 2)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
	}
	else
	{
		// 2 channels at most.
		return false;
	}
	return true;
}




IElectraDecoder::EDecoderError FElectraMP3Decoder::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if WITH_DR_LIBS_MP3
	// Decode data.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Parse the codec specific information
		if (!bHaveParsedMpegHeader)
		{
			if (!ParseMpegAudioHeader(MpegConfig, InInputAccessUnit.Data, InInputAccessUnit.DataSize))
			{
				return IElectraDecoder::EDecoderError::Error;
			}
			bHaveParsedMpegHeader = true;
		}
		// Set up the channel map accordingly.
		if (!SetupChannelMap())
		{
			// Error was already posted.
			return IElectraDecoder::EDecoderError::Error;
		}
		// Create decoder if necessary.
		if (!DecoderHandle && !InternalDecoderCreate())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		// Decode
		TSharedPtr<FElectraAudioDecoderOutputMP3_Common, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraAudioDecoderOutputMP3_Common>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;
		int32 AllocSize = sizeof(float) * MpegConfig.SamplesPerFrame * MpegConfig.NumberOfChannels;
		NewOutput->Buffer = (float*)FMemory::Malloc(AllocSize);
		NewOutput->NumChannels = MpegConfig.NumberOfChannels;
		NewOutput->SampleRate = MpegConfig.SampleRate;
		FMP3Decoder::FFrameInfo fi;
		int Result = DecoderHandle->Decode(&fi, NewOutput->Buffer, AllocSize, (const unsigned char*) InInputAccessUnit.Data, (int) InInputAccessUnit.DataSize);
		if (Result < 0)
		{
			PostError(Result, TEXT("FElectraMP3Decoder decoding failed"), ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if (NewOutput->NumChannels != fi.NumChannels)
		{
			PostError(0, TEXT("Mismatching number of decoded channels during decode sequence!"), ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if (NewOutput->SampleRate != fi.SampleRate)
		{
			PostError(0, TEXT("Mismatching sample rate during decode sequence!"), ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if (Result > 0 && Result != MpegConfig.SamplesPerFrame)
		{
			PostError(0, TEXT("Mismatching samples per frame count during decode sequence!"), ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if ((int32) InInputAccessUnit.DataSize != fi.NumFrameBytes)
		{
			PostError(Result, TEXT("FElectraMP3Decoder did not consume the entire input"), ERRCODE_MP3DEC_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		NewOutput->NumFrames = Result;
		NewOutput->ChannelPositions = OutputChannelMap;
		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
#else
	return IElectraDecoder::EDecoderError::Error;
#endif
}

IElectraDecoder::EDecoderError FElectraMP3Decoder::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraMP3Decoder::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraMP3Decoder::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraMP3Decoder::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
