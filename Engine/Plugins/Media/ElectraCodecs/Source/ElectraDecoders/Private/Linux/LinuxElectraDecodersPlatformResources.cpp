// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxElectraDecodersPlatformResources.h"
#include "ElectraDecodersModule.h"

#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MediaDecoderOutput.h"
#include "MediaVideoDecoderOutput.h"
#include "ElectraTextureSample.h"


#if PLATFORM_LINUX || PLATFORM_UNIX

#if WITH_LIBAV
#include "libav_Decoder_Common_Video.h"
#else
class ILibavDecoderDecodedImage;
#endif



struct FElectraDecodersPlatformResourcesLinux::IDecoderPlatformResource
{
	uint32 MaxWidth = 0;
	uint32 MaxHeight = 0;
	uint32 MaxOutputBuffers = 0;
};

FElectraDecodersPlatformResourcesLinux::IDecoderPlatformResource* FElectraDecodersPlatformResourcesLinux::CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions)
{
	IDecoderPlatformResource* pr = new IDecoderPlatformResource;
	pr->MaxWidth  = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_width"), 1920), 16);
	pr->MaxHeight = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_height"), 1080), 16);
	pr->MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_output_buffers"), 5);
	return pr;
}

void FElectraDecodersPlatformResourcesLinux::ReleasePlatformVideoResource(void* InOwnerHandle, FElectraDecodersPlatformResourcesLinux::IDecoderPlatformResource* InHandleToRelease)
{
	if (InHandleToRelease)
	{
		delete InHandleToRelease;
	}
}


#if WITH_LIBAV
namespace LibavcodecConversion
{
static bool ConvertDecodedImageToNV12(TArray64<uint8>& OutNV12Buffer, FIntPoint OutBufDim, int32 NumBits, ILibavDecoderDecodedImage* InImage)
{
	if (!InImage || !OutNV12Buffer.GetData())
	{
		return false;
	}
	const ILibavDecoderVideoCommon::FOutputInfo& ImageInfo = InImage->GetOutputInfo();
	if (ImageInfo.NumPlanes == 2 &&
		ImageInfo.Planes[0].Content == ILibavDecoderVideoCommon::FPlaneInfo::EContent::Luma &&
		ImageInfo.Planes[1].Content == ILibavDecoderVideoCommon::FPlaneInfo::EContent::ChromaUV)
	{
		if (ImageInfo.Planes[0].BytesPerPixel == 1)
		{
			const int32 w = ImageInfo.Planes[0].Width;
			const int32 h = ImageInfo.Planes[0].Height;
			const int32 aw = ((w + 1) / 2) * 2;
			const int32 ah = ((h + 1) / 2) * 2;
			uint8* DstY = OutNV12Buffer.GetData();
			uint8* DstUV = DstY + aw * ah;
			const uint8* SrcY = (const uint8*)ImageInfo.Planes[0].Address;
			const uint8* SrcUV = (const uint8*)ImageInfo.Planes[1].Address;
			// To simplify the conversion we require the output buffer to have the dimension of the planes.
			check(OutBufDim.X == aw && OutBufDim.Y == ah);
			if (!SrcY || !SrcUV || OutBufDim.X != aw || OutBufDim.Y != ah)
			{
				return false;
			}
			if ((w & 1) == 0)
			{
				FMemory::Memcpy(DstY, SrcY, w*h);
				FMemory::Memcpy(DstUV, SrcUV, w*h/2);
			}
			else
			{
				for(int32 y=0; y<h; ++y)
				{
					FMemory::Memcpy(DstY, SrcY, w);
					DstY += aw;
					SrcY += w;
				}
				for(int32 y=0; y<h/2; ++y)
				{
					FMemory::Memcpy(DstUV, SrcUV, w);
					DstUV += aw;
					SrcUV += w;
				}
			}
		}
		else if (ImageInfo.Planes[0].BytesPerPixel == 2)
		{
			const int32 w = ImageInfo.Planes[0].Width;
			const int32 h = ImageInfo.Planes[0].Height;
			const int32 aw = ((w + 1) / 2) * 2;
			const int32 ah = ((h + 1) / 2) * 2;
			uint16* DstY = (uint16*)OutNV12Buffer.GetData();
			uint16* DstUV = DstY + aw * ah;
			const uint16* SrcY = (const uint16*)ImageInfo.Planes[0].Address;
			const uint16* SrcUV = (const uint16*)ImageInfo.Planes[1].Address;
			// To simplify the conversion we require the output buffer to have the dimension of the planes.
			check(OutBufDim.X == aw && OutBufDim.Y == ah);
			if (!SrcY || !SrcUV || OutBufDim.X != aw || OutBufDim.Y != ah)
			{
				return false;
			}
#if 0
			// Copy as 16 bits
			if ((w & 1) == 0)
			{
				FMemory::Memcpy(DstY, SrcY, w*h*2);
				FMemory::Memcpy(DstUV, SrcUV, w*h);
			}
			else
			{
				for(int32 y=0; y<h; ++y)
				{
					FMemory::Memcpy(DstY, SrcY, w*2);
					DstY += aw;
					SrcY += w;
				}
				for(int32 y=0; y<h/2; ++y)
				{
					FMemory::Memcpy(DstUV, SrcUV, w*2);
					DstUV += aw;
					SrcUV += w;
				}
			}
#else
			// Convert down to 8 bits
			uint8* Dst8Y = OutNV12Buffer.GetData();
			uint8* Dst8UV = Dst8Y + aw * ah;
			for(int32 y=0; y<h; ++y)
			{
				for(int32 x=0; x<w; ++x)
				{
					Dst8Y[x] = (*SrcY++) >> 8;
				}
				Dst8Y += aw;
			}
			for(int32 y=0; y<h/2; ++y)
			{
				for(int32 u=0; u<w/2; ++u)
				{
					Dst8UV[u*2+0] = (*SrcUV++) >> 8;
					Dst8UV[u*2+1] = (*SrcUV++) >> 8;
				}
				Dst8UV += aw;
			}
#endif
		}
	}
	else if (ImageInfo.NumPlanes == 3 &&
				ImageInfo.Planes[0].Content == ILibavDecoderVideoCommon::FPlaneInfo::EContent::Luma &&
				ImageInfo.Planes[1].Content == ILibavDecoderVideoCommon::FPlaneInfo::EContent::ChromaU &&
				ImageInfo.Planes[2].Content == ILibavDecoderVideoCommon::FPlaneInfo::EContent::ChromaV)
	{
		if (ImageInfo.Planes[0].BytesPerPixel == 1)
		{
			const int32 w = ImageInfo.Planes[0].Width;
			const int32 h = ImageInfo.Planes[0].Height;
			const int32 aw = ((w + 1) / 2) * 2;
			const int32 ah = ((h + 1) / 2) * 2;
			uint8* DstY = OutNV12Buffer.GetData();
			uint8* DstUV = DstY + aw * ah;
			const uint8* SrcY = (const uint8*)ImageInfo.Planes[0].Address;
			const uint8* SrcU = (const uint8*)ImageInfo.Planes[1].Address;
			const uint8* SrcV = (const uint8*)ImageInfo.Planes[2].Address;
			// To simplify the conversion we require the output buffer to have the dimension of the planes.
			check(OutBufDim.X == aw && OutBufDim.Y == ah);
			if (!SrcY || !SrcU || !SrcV || OutBufDim.X != aw || OutBufDim.Y != ah)
			{
				return false;
			}
			if ((w & 1) == 0)
			{
				FMemory::Memcpy(DstY, SrcY, w*h);
				for(int32 i=0, iMax=w*h/4; i<iMax; ++i)
				{
					*DstUV++ = *SrcU++;
					*DstUV++ = *SrcV++;
				}
			}
			else
			{
				for(int32 y=0; y<h; ++y)
				{
					FMemory::Memcpy(DstY, SrcY, w);
					DstY += aw;
					SrcY += w;
				}
				int32 padUV = (aw - w) * 2;
				for(int32 v=0; v<h/2; ++v)
				{
					for(int32 u=0; u<w/2; ++u)
					{
						*DstUV++ = *SrcU++;
						*DstUV++ = *SrcV++;
					}
					DstUV += padUV;
				}
			}
		}
		else if (ImageInfo.Planes[0].BytesPerPixel == 2)
		{
			const int32 w = ImageInfo.Planes[0].Width;
			const int32 h = ImageInfo.Planes[0].Height;
			const int32 aw = ((w + 1) / 2) * 2;
			const int32 ah = ((h + 1) / 2) * 2;
		// The destination is 8 bits only!
			uint8* DstY = OutNV12Buffer.GetData();
			uint8* DstUV = DstY + aw * ah;
			const uint16* SrcY = (const uint16*)ImageInfo.Planes[0].Address;
			const uint16* SrcU = (const uint16*)ImageInfo.Planes[1].Address;
			const uint16* SrcV = (const uint16*)ImageInfo.Planes[2].Address;
			// To simplify the conversion we require the output buffer to have the dimension of the planes.
			check(OutBufDim.X == aw && OutBufDim.Y == ah);
			if (!SrcY || !SrcU || !SrcV || OutBufDim.X != aw || OutBufDim.Y != ah)
			{
				return false;
			}
			for(int32 y=0; y<h; ++y)
			{
				for(int32 x=0; x<w; ++x)
				{
					DstY[x] = SrcY[x] >> 2;
				}
				DstY += aw;
				SrcY += w;
			}
			int32 padUV = (aw - w) * 2;
			for(int32 v=0; v<h/2; ++v)
			{
				for(int32 u=0; u<w/2; ++u)
				{
					*DstUV++ = (*SrcU++) >> 2;
					*DstUV++ = (*SrcV++) >> 2;
				}
				DstUV += padUV;
			}
		}
	}
	return true;
}
} // LibavcodecConversion
#endif




bool FElectraDecodersPlatformResourcesLinux::SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, FElectraDecodersPlatformResourcesLinux::IDecoderPlatformResource* InPlatformResource)
{
	if (!InOutTextureSampleToSetup.IsValid() || !InDecoderOutput.IsValid() || !InOutBufferPropertes.IsValid() || !InPlatformResource)
	{
		OutErrorMessage = TEXT("Bad parameter");
		return false;
	}

	TMap<FString, FVariant> ExtraValues;
	InDecoderOutput->GetExtraValues(ExtraValues);

	FElectraVideoDecoderOutputCropValues Crop = InDecoderOutput->GetCropValues();
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Width, Electra::FVariantValue((int64)InDecoderOutput->GetWidth()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Height, Electra::FVariantValue((int64)InDecoderOutput->GetHeight()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropLeft, Electra::FVariantValue((int64)Crop.Left));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropRight, Electra::FVariantValue((int64)Crop.Right));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropTop, Electra::FVariantValue((int64)Crop.Top));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropBottom, Electra::FVariantValue((int64)Crop.Bottom));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectRatio, Electra::FVariantValue((double)InDecoderOutput->GetAspectRatioW() / (double)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectW, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioW()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectH, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSNumerator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateNumerator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSDenominator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateDenominator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelDataScale, Electra::FVariantValue((double)ElectraDecodersUtil::GetVariantValueSafeDouble(ExtraValues, TEXT("pix_datascale"), 1.0)));

	ILibavDecoderDecodedImage* DecodedImage = reinterpret_cast<ILibavDecoderDecodedImage*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::LibavDecoderDecodedImage));
	IElectraDecoderVideoOutputImageBuffers* ImageBuffers = !DecodedImage ? reinterpret_cast<IElectraDecoderVideoOutputImageBuffers*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::ImageBuffers)) : nullptr;
	if (DecodedImage || ImageBuffers)
	{
		if (DecodedImage)
		{
#if WITH_LIBAV
			// We are currently converting output to 8 bit NV12 format!
			int32 NumBits = 8;	// InDecoderOutput->GetNumberOfBits();
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64) NumBits));
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)((NumBits <= 8) ? EPixelFormat::PF_NV12 : EPixelFormat::PF_P010)));
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)InDecoderOutput->GetDecodedWidth() * ((NumBits <= 8) ? 1 : 2)));

			const ILibavDecoderVideoCommon::FOutputInfo& DecodedImageInfo = DecodedImage->GetOutputInfo();
			FIntPoint ImageDim(DecodedImageInfo.Planes[0].Width, DecodedImageInfo.Planes[0].Height);
			FIntPoint AlignedDim(ImageDim);

			InOutTextureSampleToSetup->InitializeCommon(*InOutBufferPropertes);

			// Round up to multiple of 2.
			AlignedDim.X = ((AlignedDim.X + 1) / 2) * 2;
			AlignedDim.Y = ((AlignedDim.Y + 1) / 2) * 2;
			// The vertical sample dimension encompasses the height of the UV plane.
			AlignedDim.Y = AlignedDim.Y * 3 / 2;

			InOutTextureSampleToSetup->SetDim(AlignedDim);
			// Stride is the width of the buffer for now (since we converted down to 8 bits)
			InOutTextureSampleToSetup->Stride = AlignedDim.X;
			
			// Allocate the buffer
			InOutTextureSampleToSetup->Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
			int32 AllocSize = AlignedDim.X * AlignedDim.Y * (NumBits > 8 ? 2 : 1);
			InOutTextureSampleToSetup->Buffer->SetNum(AllocSize);

			LibavcodecConversion::ConvertDecodedImageToNV12(*InOutTextureSampleToSetup->Buffer, ImageDim, NumBits, DecodedImage);

			// Finish initialization.
			if (!InOutTextureSampleToSetup->FinishInitialization())
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Unsupported pixel format encoding"));
				return false;
			}
			return true;
#endif
		}
		else if (ImageBuffers && ImageBuffers->GetNumberOfBuffers() == 1)
		{
			// Compatible format?
			if ((ImageBuffers->GetBufferFormatByIndex(0) == EPixelFormat::PF_NV12 || ImageBuffers->GetBufferFormatByIndex(0) == EPixelFormat::PF_P010) &&
				 ImageBuffers->GetBufferEncodingByIndex(0) == EElectraTextureSamplePixelEncoding::Native)
			{
				// We are currently converting output to 8 bit NV12 format!
				int32 NumBits = InDecoderOutput->GetNumberOfBits();
				int32 Pitch = ImageBuffers->GetBufferPitchByIndex(0);
				InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64) NumBits));
				InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)((NumBits <= 8) ? EPixelFormat::PF_NV12 : EPixelFormat::PF_P010)));
				InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));
				InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelEncoding, Electra::FVariantValue((int64)EElectraTextureSamplePixelEncoding::Native));

				FIntPoint AlignedDim((int32) InDecoderOutput->GetWidth(), (int32) InDecoderOutput->GetHeight());

				InOutTextureSampleToSetup->InitializeCommon(*InOutBufferPropertes);

				// Round up to multiple of 2.
				AlignedDim.X = ((AlignedDim.X + 1) / 2) * 2;
				AlignedDim.Y = ((AlignedDim.Y + 1) / 2) * 2;
				// The vertical sample dimension encompasses the height of the UV plane.
				AlignedDim.Y = AlignedDim.Y * 3 / 2;

				InOutTextureSampleToSetup->SetDim(AlignedDim);
				
				// Share the buffer
				InOutTextureSampleToSetup->Buffer = ImageBuffers->GetBufferDataByIndex(0);

				// Stride is the width of the buffer
				InOutTextureSampleToSetup->Stride = Pitch;

				// Finish initialization.
				if (!InOutTextureSampleToSetup->FinishInitialization())
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Unsupported pixel format encoding"));
					return false;
				}
				return true;
			}
		}
	}
	OutErrorMessage = TEXT("Unhandled decoder output format");
	return false;
}

#endif // PLATFORM_LINUX || PLATFORM_UNIX
