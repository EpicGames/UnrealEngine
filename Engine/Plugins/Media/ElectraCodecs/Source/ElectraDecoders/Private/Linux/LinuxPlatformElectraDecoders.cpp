// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxPlatformElectraDecoders.h"
#include "IElectraCodecRegistry.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX

#include "libav_Decoder_Common.h"

#include "h264/H264_VideoDecoder_Linux.h"
#include "h265/H265_VideoDecoder_Linux.h"
#include "vpx/VPx_VideoDecoder_Linux.h"
#include "aac/AAC_AudioDecoder_Linux.h"

namespace PlatformElectraDecodersLinux
{
	static TArray<TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> Factories;
}

void FPlatformElectraDecodersLinux::Startup()
{
	ILibavDecoder::Startup();
}

void FPlatformElectraDecodersLinux::Shutdown()
{
	PlatformElectraDecodersLinux::Factories.Empty();
	ILibavDecoder::Shutdown();
}

void FPlatformElectraDecodersLinux::RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith)
{
	check(InCodecFactoryToRegisterWith);
	if (InCodecFactoryToRegisterWith)
	{
		auto RegisterFactory = [&](TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InFactory) -> void
		{
			if (InFactory.IsValid())
			{
				PlatformElectraDecodersLinux::Factories.Emplace(InFactory);
				InCodecFactoryToRegisterWith->AddCodecFactory(InFactory);
			}
		};

		if (ILibavDecoder::IsLibAvAvailable())
		{
			// H.264 video decoder
			RegisterFactory(FH264VideoDecoderLinux::CreateFactory());
			// H.265 video decoder
			RegisterFactory(FH265VideoDecoderLinux::CreateFactory());
			// VP8/VP9 video decoder
			RegisterFactory(FVPxVideoDecoderLinux::CreateFactory());
			// AAC audio decoder
			RegisterFactory(FAACAudioDecoderLinux::CreateFactory());
		}
	}
}

#endif
