// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX

class IElectraCodecRegistry;

class FPlatformElectraDecodersLinux
{
public:
	static void Startup();
	static void Shutdown();
	static void RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith);
};

using FPlatformElectraDecoders = FPlatformElectraDecodersLinux;

#endif
