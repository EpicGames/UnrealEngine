// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

#if PLATFORM_LINUX || PLATFORM_UNIX

#include "IElectraCodecFactory.h"

class FAACAudioDecoderLinux
{
public:
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();

};

#endif
