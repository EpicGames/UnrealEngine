// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDecoderAAC_Linux.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX

#include "DecoderErrors_Linux.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputAudio.h"

/*********************************************************************************************************************/

#include "libav_Decoder_Common.h"
#include "libav_Decoder_AAC.h"

/*********************************************************************************************************************/

class FElectraAudioDecoderOutputAAC_Linux : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputAAC_Linux()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	int32 GetNumChannels() const override
	{ return NumChannels; }
	int32 GetSampleRate() const override
	{ return SampleRate; }
	int32 GetNumFrames() const override
	{ return NumFrames; }
	bool IsInterleaved() const override
	{ return true; }
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid; }
	ESampleFormat GetSampleFormat() const override
	{ return ESampleFormat::Float; }
	int32 GetBytesPerSample() const override
	{ return sizeof(float); }
	int32 GetBytesPerFrame() const override
	{ return GetBytesPerSample() * GetNumChannels(); }
	const void* GetData(int32 InChannelNumber) const override
	{ return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr; }

public:
	enum
	{
		kMaxChannels = 8
	};
	EChannelPosition ChannelPositions[kMaxChannels];
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};



class FElectraAudioDecoderAAC_Linux : public IElectraAudioDecoderAAC_Linux
{
public:
	FElectraAudioDecoderAAC_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraAudioDecoderAAC_Linux();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Audio; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override;
	void Resume() override;

private:
	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> GetCSDFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	bool ConvertDecoderOutput(const ILibavDecoderAAC::FOutputInfo& OutInfo);

	IElectraDecoder::FError LastError;

	TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> ConfigRecord;
	TArray<FDecoderInput> InDecoderInput;
	TSharedPtr<FElectraAudioDecoderOutputAAC_Linux, ESPMode::ThreadSafe> CurrentOutput;

	TSharedPtr<ILibavDecoderAAC, ESPMode::ThreadSafe> DecoderInstance;
	EDecodeState DecodeState = EDecodeState::Decoding;

	static const uint8 NumChannelsForConfig[16];
	static const IElectraDecoderAudioOutput::EChannelPosition _1[1];
	static const IElectraDecoderAudioOutput::EChannelPosition _2[2];
	static const IElectraDecoderAudioOutput::EChannelPosition _3[3];
	static const IElectraDecoderAudioOutput::EChannelPosition _4[4];
	static const IElectraDecoderAudioOutput::EChannelPosition _5[5];
	static const IElectraDecoderAudioOutput::EChannelPosition _6[6];
	static const IElectraDecoderAudioOutput::EChannelPosition _7[7];
	static const IElectraDecoderAudioOutput::EChannelPosition _8[8];
	static const IElectraDecoderAudioOutput::EChannelPosition * const MostCommonChannelMapping[8];
};
const uint8 FElectraAudioDecoderAAC_Linux::NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_1[1] = { IElectraDecoderAudioOutput::EChannelPosition::C };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_2[2] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_3[3] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_4[4] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::Cs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_5[5] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_6[6] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_7[7] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Cs };
const IElectraDecoderAudioOutput::EChannelPosition FElectraAudioDecoderAAC_Linux::_8[8] = { IElectraDecoderAudioOutput::EChannelPosition::L, IElectraDecoderAudioOutput::EChannelPosition::R, IElectraDecoderAudioOutput::EChannelPosition::C, IElectraDecoderAudioOutput::EChannelPosition::LFE, IElectraDecoderAudioOutput::EChannelPosition::Ls, IElectraDecoderAudioOutput::EChannelPosition::Rs, IElectraDecoderAudioOutput::EChannelPosition::Lsr, IElectraDecoderAudioOutput::EChannelPosition::Rsr };
const IElectraDecoderAudioOutput::EChannelPosition * const FElectraAudioDecoderAAC_Linux::MostCommonChannelMapping[8] = { FElectraAudioDecoderAAC_Linux::_1, FElectraAudioDecoderAAC_Linux::_2, FElectraAudioDecoderAAC_Linux::_3, FElectraAudioDecoderAAC_Linux::_4, FElectraAudioDecoderAAC_Linux::_5, FElectraAudioDecoderAAC_Linux::_6, FElectraAudioDecoderAAC_Linux::_7, FElectraAudioDecoderAAC_Linux::_8 };


void IElectraAudioDecoderAAC_Linux::GetConfigurationOptions(TMap<FString, FVariant>& OutFeatures)
{
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraAudioDecoderAAC_Linux::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraAudioDecoderAAC_Linux, ESPMode::ThreadSafe> New = MakeShared<FElectraAudioDecoderAAC_Linux>(InCodecFormat, InAdditionalOptions);
	return New;
}

FElectraAudioDecoderAAC_Linux::FElectraAudioDecoderAAC_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.CSD.Num())
	{ 
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->Parse(InCodecFormat.CSD))
		{
			ConfigRecord = MoveTemp(NewConfigRecord);
		}
	}
}

FElectraAudioDecoderAAC_Linux::~FElectraAudioDecoderAAC_Linux()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Linux::GetCSDFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TSharedPtr<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe> NewConfigRecord;
		NewConfigRecord = MakeShared<ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord, ESPMode::ThreadSafe>();
		if (NewConfigRecord->ParseFrom(SidebandData.GetData(), SidebandData.Num()))
		{
			return NewConfigRecord;
		}
		else
		{
			LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
			LastError.Message = TEXT("Failed to parse codec specific data");
		}
	}
	return nullptr;
}

void FElectraAudioDecoderAAC_Linux::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
}

IElectraDecoder::FError FElectraAudioDecoderAAC_Linux::GetError() const
{
	return LastError;
}

void FElectraAudioDecoderAAC_Linux::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FElectraAudioDecoderAAC_Linux::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraAudioDecoderAAC_Linux::ResetToCleanStart()
{
	InternalDecoderDestroy();
	ConfigRecord.Reset();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Linux::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode call just provides new configuration?
	if ((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) != EElectraDecoderFlags::None)
	{
		ConfigRecord.Reset();
		InternalDecoderDestroy();
	}

	// Need a valid CSD to create a decoder.
	if (!ConfigRecord.IsValid())
	{
		ConfigRecord = GetCSDFromOptions(InAdditionalOptions);
		if (!ConfigRecord.IsValid())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Create decoder transform if necessary.
	if (!DecoderInstance.IsValid() && !InternalDecoderCreate())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		ILibavDecoderAAC::FInputAU DecAU;
		DecAU.Data = InInputAccessUnit.Data;
		DecAU.DataSize = (int32) InInputAccessUnit.DataSize;
		DecAU.DTS = InInputAccessUnit.DTS.GetTicks();
		DecAU.PTS = InInputAccessUnit.PTS.GetTicks();
		DecAU.Duration = InInputAccessUnit.Duration.GetTicks();
		DecAU.UserValue = InInputAccessUnit.PTS.GetTicks();
		ILibavDecoder::EDecoderError DecErr = DecoderInstance->DecodeAccessUnit(DecAU);
		if (DecErr != ILibavDecoder::EDecoderError::None)
		{
			PostError((int32)DecErr, "Failed to decode access unit", ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		// Add to the list of inputs passed to the decoder.
		FDecoderInput In;
		In.AdditionalOptions = InAdditionalOptions;
		In.AccessUnit = InInputAccessUnit;
		InDecoderInput.Emplace(MoveTemp(In));
		InDecoderInput.Sort([](const FDecoderInput& e1, const FDecoderInput& e2)
		{
			return e1.AccessUnit.PTS < e2.AccessUnit.PTS;
		});
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Linux::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->SendEndOfData();
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraAudioDecoderAAC_Linux::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderInstance.IsValid())
	{
		InternalDecoderDestroy();
		ConfigRecord.Reset();
		DecodeState = EDecodeState::Decoding;
		InDecoderInput.Empty();
		CurrentOutput.Reset();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraAudioDecoderAAC_Linux::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (!DecoderInstance.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	ILibavDecoderAAC::FOutputInfo Out;
	switch(DecoderInstance->HaveOutput(Out))
	{
		case ILibavDecoder::EOutputStatus::Available:
		{
			return ConvertDecoderOutput(Out) ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
		}
		case ILibavDecoder::EOutputStatus::EndOfData:
		{
			InDecoderInput.Empty();
			InternalDecoderDestroy();
			DecodeState = EDecodeState::Decoding;
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		default:
		case ILibavDecoder::EOutputStatus::NeedInput:
		{
			return IElectraDecoder::EOutputStatus::NeedInput;
		}
	}
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraAudioDecoderAAC_Linux::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

void FElectraAudioDecoderAAC_Linux::Suspend()
{
}


void FElectraAudioDecoderAAC_Linux::Resume()
{
}

bool FElectraAudioDecoderAAC_Linux::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

bool FElectraAudioDecoderAAC_Linux::InternalDecoderCreate()
{
	if (!ILibavDecoderAAC::IsAvailable())
	{
		return PostError(0, "libavcodec does not support this audio format", ERRCODE_INTERNAL_DECODER_NOT_SUPPORTED);
	}
	DecoderInstance = ILibavDecoderAAC::Create(ConfigRecord->GetCodecSpecificData());
	int32 ErrorCode = 0;
	if (DecoderInstance.IsValid() && ((ErrorCode = DecoderInstance->GetLastLibraryError())==0))
	{
		return true;
	}
	InternalDecoderDestroy();
	PostError(ErrorCode, "libavcodec failed to open audio decoder", ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	return false;
}

void FElectraAudioDecoderAAC_Linux::InternalDecoderDestroy()
{
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Reset();
		DecoderInstance.Reset();
	}
}

bool FElectraAudioDecoderAAC_Linux::ConvertDecoderOutput(const ILibavDecoderAAC::FOutputInfo& OutInfo)
{
	// We take the frontmost entry of decoder inputs. The data is not expected to be decoded out of order.
	// We also remove the frontmost entry even if conversion of the output actually fails.
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return false;
	}
	FDecoderInput MatchingInput = InDecoderInput[0];
	InDecoderInput.RemoveAt(0);

	TSharedPtr<FElectraAudioDecoderOutputAAC_Linux, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraAudioDecoderOutputAAC_Linux>();
	NewOutput->PTS = MatchingInput.AccessUnit.PTS;
	NewOutput->UserValue = MatchingInput.AccessUnit.UserValue;
	NewOutput->SampleRate = OutInfo.SampleRate;
	NewOutput->NumFrames = OutInfo.NumSamples;
	NewOutput->NumChannels = OutInfo.NumChannels;
	for(uint32 i=0; i<NewOutput->NumChannels; ++i)
	{
		NewOutput->ChannelPositions[i] = MostCommonChannelMapping[NewOutput->NumChannels-1][i];
	}

	int32 BufferSize = NewOutput->NumFrames * NewOutput->NumChannels * sizeof(float);
	NewOutput->Buffer = (float*)FMemory::Malloc(BufferSize);
	if (!DecoderInstance->GetOutputAsF32(NewOutput->Buffer, BufferSize))
	{
		return PostError(0, "Could not get decoded output due to decoded format being unsupported", ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	}

	CurrentOutput = MoveTemp(NewOutput);
	return true;
}


#endif // ELECTRA_DECODERS_ENABLE_LINUX
