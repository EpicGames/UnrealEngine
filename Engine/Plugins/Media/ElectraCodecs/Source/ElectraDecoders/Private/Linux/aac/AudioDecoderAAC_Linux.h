// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX

class IElectraAudioDecoderAAC_Linux : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraAudioDecoderAAC_Linux() = default;
};

#endif
