// Copyright Epic Games, Inc. All Rights Reserved.

#include "h264/H264_VideoDecoder_Linux.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX
#include "h264/VideoDecoderH264_Linux.h"

#include "ElectraDecodersUtils.h"

#include "libav_Decoder_Common.h"
#include "libav_Decoder_H264.h"

class FH264VideoDecoderFactoryLinux : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FH264VideoDecoderFactoryLinux()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderH264_Linux::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		// Codec available?
		if (!ILibavDecoderH264::IsAvailable())
		{
			return 0;
		}
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','1') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','3'))
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vid = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		const uint32 Width = (uint32)vid.Width;
		const uint32 Height = (uint32)vid.Height;
		const double fps = vid.FrameRate.IsValid() ? vid.FrameRate.AsDecimal() : 0.0;
		// Check if supported.
		TArray<IElectraVideoDecoderH264_Linux::FSupportedConfiguration> Configs;
		IElectraVideoDecoderH264_Linux::PlatformGetSupportedConfigurations(Configs);
		bool bSupported = false;
		for(int32 i=0; i<Configs.Num(); ++i)
		{
			const IElectraVideoDecoderH264_Linux::FSupportedConfiguration& Cfg = Configs[i];
			if (Cfg.Profile == vid.Profile.Profile)
			{
				if (vid.Profile.Level > Cfg.Level)
				{
					continue;
				}
				if (ElectraDecodersUtil::ExceedsMaxDimensions(Width, Height, Cfg.Width, Cfg.Height))
				{
					continue;
				}
				if (fps > 0.0 && Cfg.FramesPerSecond && (int32)fps > Cfg.FramesPerSecond)
				{
					continue;
				}
				if (Cfg.Num16x16Macroblocks && ((Align(Width, 16) * Align(Height, 16)) / 256) > Cfg.Num16x16Macroblocks)
				{
					continue;
				}
				bSupported = true;
				break;
			}
		}
		return bSupported ? 1 : 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderH264_Linux::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libav")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FH264VideoDecoderLinux::CreateFactory()
{
	return MakeShared<FH264VideoDecoderFactoryLinux, ESPMode::ThreadSafe>();
}

#endif
