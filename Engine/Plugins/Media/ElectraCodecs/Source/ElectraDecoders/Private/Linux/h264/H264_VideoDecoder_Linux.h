// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IElectraCodecFactory.h"

class FH264VideoDecoderLinux
{
public:
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();

};
