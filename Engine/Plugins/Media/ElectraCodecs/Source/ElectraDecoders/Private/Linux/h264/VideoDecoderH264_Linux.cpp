// Copyright Epic Games, Inc. All Rights Reserved.

#include "h264/VideoDecoderH264_Linux.h"
#include "DecoderErrors_Linux.h"
#include "ElectraDecodersUtils.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraBitstreamProcessor_H264.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersModule.h"

/*********************************************************************************************************************/

#include "libav_Decoder_Common.h"
#include "libav_Decoder_H264.h"

/*********************************************************************************************************************/

class FElectraVideoDecoderH264_Linux;


class FElectraVideoDecoderOutputH264_Linux : public IElectraDecoderVideoOutput
{
public:
	virtual ~FElectraVideoDecoderOutputH264_Linux()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return Width; }
	int32 GetDecodedHeight() const override
	{ return Height; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::LibavDecoderDecodedImage)
		{
			return DecodedImage.Get();
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;

	TSharedPtr<ILibavDecoderDecodedImage, ESPMode::ThreadSafe> DecodedImage;
};



class FElectraVideoDecoderH264_Linux : public IElectraVideoDecoderH264_Linux, public ILibavDecoderVideoResourceAllocator
{
public:
	FElectraVideoDecoderH264_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraVideoDecoderH264_Linux();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorH264::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	// Add methods from ILibavDecoderVideoResourceAllocator here
	// ...

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> SPS;
		bool bDropOutput = false;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	bool PostError(int32_t ApiReturnValue, FString Message, int32 Code);

	TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> GetSPSFromOptions(const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();

	enum class EConvertResult
	{
		Success,
		Failure
	};
	EConvertResult ConvertDecoderOutput(const ILibavDecoderVideoCommon::FOutputInfo& OutInfo);

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	IElectraDecoder::FError LastError;

	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;
	TSharedPtr<FElectraVideoDecoderOutputH264_Linux, ESPMode::ThreadSafe> CurrentOutput;
	TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> CurrentSPS;

	TSharedPtr<ILibavDecoderVideoCommon, ESPMode::ThreadSafe> DecoderInstance;
	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bNewDecoderRequired = false;
};


namespace IElectraVideoDecoderH264_Linux_Platform
{
	static TArray<IElectraVideoDecoderH264_Linux::FSupportedConfiguration> DecoderConfigurations;
	static bool bDecoderConfigurationsDirty = true;
}

void IElectraVideoDecoderH264_Linux::PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations)
{
	if (IElectraVideoDecoderH264_Linux_Platform::bDecoderConfigurationsDirty)
	{
		IElectraVideoDecoderH264_Linux_Platform::DecoderConfigurations.Empty();

		// Perhaps these can be determined dynamically?

		// Baseline
		IElectraVideoDecoderH264_Linux_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_Linux::FSupportedConfiguration(66, 52, 60, 3840, 2160, 0));
		// Main
		IElectraVideoDecoderH264_Linux_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_Linux::FSupportedConfiguration(77, 52, 60, 3840, 2160, 0));
		// High
		IElectraVideoDecoderH264_Linux_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_Linux::FSupportedConfiguration(100, 52, 60, 3840, 2160, 0));

		IElectraVideoDecoderH264_Linux_Platform::bDecoderConfigurationsDirty = false;
	}
	OutSupportedConfigurations = IElectraVideoDecoderH264_Linux_Platform::DecoderConfigurations;
}

void IElectraVideoDecoderH264_Linux::GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
{
	OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
	// The decoder is adaptive. There is no need to call IsCompatibleWith() on stream switches.
	OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraVideoDecoderH264_Linux::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraVideoDecoderH264_Linux, ESPMode::ThreadSafe> New = MakeShared<FElectraVideoDecoderH264_Linux>(InCodecFormat, InAdditionalOptions);
	return New;
}

FElectraVideoDecoderH264_Linux::FElectraVideoDecoderH264_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
}


FElectraVideoDecoderH264_Linux::~FElectraVideoDecoderH264_Linux()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> FElectraVideoDecoderH264_Linux::GetSPSFromOptions(const TMap<FString, FVariant>& InOptions)
{
	TArray<uint8> SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InOptions, TEXT("csd"));
	if (SidebandData.Num())
	{
		TArray<ElectraDecodersUtil::MPEG::H264::FNaluInfo> NALUs;
		ElectraDecodersUtil::MPEG::H264::ParseBitstreamForNALUs(NALUs, SidebandData.GetData(), SidebandData.Num());
		for(int32 i=0; i<NALUs.Num(); ++i)
		{
			if (NALUs[i].Type == 7)
			{
				TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = MakeShared<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe>();
				if (ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(*NewSPS, ElectraDecodersUtil::AdvancePointer(SidebandData.GetData(), NALUs[i].Offset + NALUs[i].UnitLength), NALUs[i].Size))
				{
					return NewSPS;
				}
				else
				{
					break;
				}
			}
		}
		LastError.Code = ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD;
		LastError.Message = TEXT("Failed to parse codec specific data");
	}
	return nullptr;
}

void FElectraVideoDecoderH264_Linux::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraVideoDecoderH264_Linux::GetError() const
{
	return LastError;
}

void FElectraVideoDecoderH264_Linux::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

bool FElectraVideoDecoderH264_Linux::ResetToCleanStart()
{
	InternalDecoderDestroy();

	InDecoderInput.Empty();
	CurrentOutput.Reset();
	CurrentSPS.Reset();
	bNewDecoderRequired = false;
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

IElectraDecoder::ECSDCompatibility FElectraVideoDecoderH264_Linux::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// NOTE: Since we set IElectraDecoderFeature::IsAdaptive to be true this method here will not be invoked on stream switches.

	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!DecoderInstance.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}

	TSharedPtr<ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet, ESPMode::ThreadSafe> NewSPS = GetSPSFromOptions(CSDAndAdditionalOptions);
	if (!NewSPS.IsValid())
	{
		UE_LOGF(LogElectraDecoders, Warning, "No CSD provided to IsCompatibleWith(), returning to drain and reset.");
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}

	// The decoder is adaptive, so it is compatible.
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH264_Linux::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// If a new decoder is needed, destroy the current one.
	if (bNewDecoderRequired)
	{
		InternalDecoderDestroy();
	}

	// Create decoder transform if necessary.
	if (!DecoderInstance.IsValid() && !InternalDecoderCreate(InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		uint8* Temp = nullptr;
		ILibavDecoderVideoCommon::FInputAU DecAU;
		// If this is a sync sample we prepend the CSD. While this is not necessary on a running stream we need to have the CSD
		// on the first frame and it is easier to prepend it to all IDR frames when seeking etc.
		DecAU.DataSize = InInputAccessUnit.DataSize;
		if ((InInputAccessUnit.Flags & EElectraDecoderFlags::IsSyncSample) != EElectraDecoderFlags::None)
		{
			CurrentSPS = GetSPSFromOptions(InAdditionalOptions);
			if (!CurrentSPS.IsValid())
			{
				PostError(0, TEXT("Failed to parse video SPS"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}

			TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("csd"));
			Temp = (uint8*)FMemory::Malloc(DecAU.DataSize + CSD.Num());
			FMemory::Memcpy(Temp, CSD.GetData(), CSD.Num());
			FMemory::Memcpy(Temp + CSD.Num(), InInputAccessUnit.Data, InInputAccessUnit.DataSize);
			DecAU.DataSize += CSD.Num();
			DecAU.Data = Temp;
		}
		else
		{
			DecAU.Data = InInputAccessUnit.Data;
		}

		DecAU.DTS = InInputAccessUnit.DTS.GetTicks();
		DecAU.UserValue = DecAU.PTS = InInputAccessUnit.PTS.GetTicks();
		DecAU.Duration = InInputAccessUnit.Duration.GetTicks();
		DecAU.Flags = 0;
		ILibavDecoder::EDecoderError Result = DecoderInstance->DecodeAccessUnit(DecAU);
		FMemory::Free(Temp);
		if (Result == ILibavDecoder::EDecoderError::None)
		{
			// Add to the list of inputs passed to the decoder.
			TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
			In->AdditionalOptions = InAdditionalOptions;
			In->AccessUnit = InInputAccessUnit;
			In->bDropOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::DoNotOutput;
			In->SPS = CurrentSPS;
			// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
			// values here that would lead us to think that we do.
			In->AccessUnit.Data = nullptr;
			In->AccessUnit.DataSize = 0;
			InDecoderInput.Emplace(MoveTemp(In));
			InDecoderInput.Sort([](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e1, const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e2)
			{
				return e1->AccessUnit.PTS < e2->AccessUnit.PTS;
			});
			return IElectraDecoder::EDecoderError::None;
		}
		else
		{
			PostError(DecoderInstance->GetLastLibraryError(), TEXT("Failed to decode video frame"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH264_Linux::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->SendEndOfData();
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderH264_Linux::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderInstance.IsValid())
	{
		InternalDecoderDestroy();
	}
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	CurrentSPS.Reset();
	bNewDecoderRequired = false;
	DecodeState = EDecodeState::Decoding;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraVideoDecoderH264_Linux::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !DecoderInstance.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	ILibavDecoderVideoCommon::FOutputInfo Out;
	switch(DecoderInstance->HaveOutput(Out))
	{
		case ILibavDecoder::EOutputStatus::Available:
		{
			return ConvertDecoderOutput(Out) == EConvertResult::Success ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
		}
		case ILibavDecoder::EOutputStatus::EndOfData:
		{
			DecodeState = EDecodeState::Decoding;
			InDecoderInput.Empty();
			bNewDecoderRequired = true;
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		default:
		case ILibavDecoder::EOutputStatus::NeedInput:
		{
			return IElectraDecoder::EOutputStatus::NeedInput;
		}
	}
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraVideoDecoderH264_Linux::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

void FElectraVideoDecoderH264_Linux::Suspend()
{
}

void FElectraVideoDecoderH264_Linux::Resume()
{
}

bool FElectraVideoDecoderH264_Linux::PostError(int32_t ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

bool FElectraVideoDecoderH264_Linux::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	CurrentSPS.Reset();
	CurrentOutput.Reset();
	bNewDecoderRequired = false;

	if (!ILibavDecoderH264::IsAvailable())
	{
		return PostError(0, "libavcodec does not support this video format", ERRCODE_INTERNAL_DECODER_NOT_SUPPORTED);
	}
	TMap<FString, FVariant> Options;
	Options.Add(TEXT("hw_priority"), FVariant(FString("cuda;vdpau;vaapi")));
	DecoderInstance = ILibavDecoderH264::Create(this, Options);
	int32 ErrorCode = 0;
	if (DecoderInstance.IsValid() && ((ErrorCode = DecoderInstance->GetLastLibraryError())==0))
	{
		return true;
	}
	InternalDecoderDestroy();
	PostError(ErrorCode, "libavcodec failed to open video decoder", ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	return false;
}

void FElectraVideoDecoderH264_Linux::InternalDecoderDestroy()
{
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Reset();
		DecoderInstance.Reset();
	}
}

FElectraVideoDecoderH264_Linux::EConvertResult FElectraVideoDecoderH264_Linux::ConvertDecoderOutput(const ILibavDecoderVideoCommon::FOutputInfo& OutInfo)
{
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	// Find the input corresponding to this output.
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec]->AccessUnit.PTS.GetTicks() == OutInfo.UserValue)
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}
	if (!In->SPS.IsValid())
	{
		PostError(0, TEXT("There is no SPS associated with the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	TSharedPtr<FElectraVideoDecoderOutputH264_Linux, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraVideoDecoderOutputH264_Linux>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;
	NewOutput->DecodedImage = DecoderInstance->GetOutput();

	NewOutput->Width = In->SPS->GetWidth();
	NewOutput->Height = In->SPS->GetHeight();
	NewOutput->Pitch = NewOutput->Width;
	NewOutput->NumBits = 8 + In->SPS->bit_depth_luma_minus8;
	NewOutput->PixelFormat = 0;
	In->SPS->GetCrop(NewOutput->Crop.Left, NewOutput->Crop.Right, NewOutput->Crop.Top, NewOutput->Crop.Bottom);
	In->SPS->GetAspect(NewOutput->AspectW, NewOutput->AspectH);
	if (NewOutput->AspectW == 0 || NewOutput->AspectH == 0)
	{
		NewOutput->AspectW = 1;
		NewOutput->AspectH = 1;
	}
	if (In->SPS->timing_info_present_flag && In->SPS->vui_parameters_present_flag)
	{
		NewOutput->FrameRateN = In->SPS->time_scale;
		NewOutput->FrameRateD = In->SPS->num_units_in_tick * 2;
	}

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("linux")));
	NewOutput->ExtraValues.Emplace(TEXT("decoder"), FVariant(TEXT("libavcodec")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("avc")));

	// Take this output on if it is not to be dropped.
	// We need to have pulled it out of libav before deciding this though.
	if (!In->bDropOutput)
	{
		CurrentOutput = MoveTemp(NewOutput);
	}
	return EConvertResult::Success;
}
