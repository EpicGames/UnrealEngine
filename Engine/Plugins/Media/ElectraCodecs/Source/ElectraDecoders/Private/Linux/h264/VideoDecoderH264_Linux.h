// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"

class IElectraVideoDecoderH264_Linux : public IElectraDecoder
{
public:
	struct FSupportedConfiguration
	{
		FSupportedConfiguration(int32 InProfile, int32 InLevel, int32 InFramesPerSecond, int32 InWidth, int32 InHeight, int32 InNum16x16Macroblocks)
			: Profile(InProfile), Level(InLevel), FramesPerSecond(InFramesPerSecond), Width(InWidth), Height(InHeight), Num16x16Macroblocks(InNum16x16Macroblocks)
		{}

		int32 Profile = 0;
		int32 Level = 0;
		int32 FramesPerSecond = 0;
		int32 Width = 0;
		int32 Height = 0;
		int32 Num16x16Macroblocks = 0;
	};
	static void PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations);

	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraVideoDecoderH264_Linux() = default;
};
