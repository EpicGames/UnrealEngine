// Copyright Epic Games, Inc. All Rights Reserved.

#include "h265/H265_VideoDecoder_Linux.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX
#include "h265/VideoDecoderH265_Linux.h"

#include "ElectraDecodersUtils.h"

#include "libav_Decoder_Common.h"
#include "libav_Decoder_H265.h"

class FH265VideoDecoderFactoryLinux : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FH265VideoDecoderFactoryLinux()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderH265_Linux::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		// Codec available?
		if (!ILibavDecoderH265::IsAvailable())
		{
			return 0;
		}
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('h','v','c','1') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('h','e','v','1'))
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vid = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		const uint32 Width = (uint32)vid.Width;
		const uint32 Height = (uint32)vid.Height;
		const double fps = vid.FrameRate.IsValid() ? vid.FrameRate.AsDecimal() : 0.0;
		// Check if supported.
		TArray<IElectraVideoDecoderH265_Linux::FSupportedConfiguration> Configs;
		IElectraVideoDecoderH265_Linux::PlatformGetSupportedConfigurations(Configs);
		bool bSupported = false;
		for(int32 i=0; i<Configs.Num(); ++i)
		{
			const IElectraVideoDecoderH265_Linux::FSupportedConfiguration& Cfg = Configs[i];
			if (Cfg.Profile == vid.Profile.Profile)
			{
				if (vid.Profile.Level > Cfg.Level)
				{
					continue;
				}
				if (ElectraDecodersUtil::ExceedsMaxDimensions(Width, Height, Cfg.Width, Cfg.Height))
				{
					continue;
				}
				if (fps > 0.0 && Cfg.FramesPerSecond && (int32)fps > Cfg.FramesPerSecond)
				{
					continue;
				}
				if (Cfg.Num8x8Macroblocks && (((Align(Width, 8) * Align(Height, 8)) / 64) * (fps > 0.0 ? fps : 1.0)) > Cfg.Num8x8Macroblocks)
				{
					continue;
				}
				bSupported = true;
				break;
			}
		}
		return bSupported ? 1 : 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderH265_Linux::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libav")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FH265VideoDecoderLinux::CreateFactory()
{
	return MakeShared<FH265VideoDecoderFactoryLinux, ESPMode::ThreadSafe>();
}

#endif
