// Copyright Epic Games, Inc. All Rights Reserved.

#include "vpx/VPx_VideoDecoder_Linux.h"

#ifdef ELECTRA_DECODERS_ENABLE_LINUX
#include "vpx/VideoDecoderVPx_Linux.h"

#include "ElectraDecodersUtils.h"

#include "libav_Decoder_Common.h"
#include "libav_Decoder_VP8.h"
#include "libav_Decoder_VP9.h"

class FVPxVideoDecoderFactoryLinux : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FVPxVideoDecoderFactoryLinux()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderVPx_Linux::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		// Codec available?
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if ((InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('v','p','0','8') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('v','p','0','9')) ||
			(InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','8') && !ILibavDecoderVP8::IsAvailable()) ||
			(InCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','9') && !ILibavDecoderVP9::IsAvailable()))
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vid = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		const uint32 Width = vid.Width;
		const uint32 Height = vid.Height;
		// See if there are values provided we can check against.
		if ((Width && Width < 128) || (Height && Height < 128))
		{
			return 0;
		}
		else if ((Width && Width > 8192) || (Height && Height > 8192))
		{
			return 0;
		}
		return 5;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderVPx_Linux::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libav")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FVPxVideoDecoderLinux::CreateFactory()
{
	return MakeShared<FVPxVideoDecoderFactoryLinux, ESPMode::ThreadSafe>();
}

#endif
