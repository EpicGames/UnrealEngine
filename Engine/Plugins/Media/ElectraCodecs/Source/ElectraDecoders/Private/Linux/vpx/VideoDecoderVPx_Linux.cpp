// Copyright Epic Games, Inc. All Rights Reserved.

#include "vpx/VideoDecoderVPx_Linux.h"
#include "DecoderErrors_Linux.h"
#include "ElectraDecodersUtils.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/Google/ElectraBitstreamProcessorVPx.h"

#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoderOutputVideo.h"

#include "ElectraDecodersModule.h"

/*********************************************************************************************************************/

#include "libav_Decoder_Common.h"
#include "libav_Decoder_VP8.h"
#include "libav_Decoder_VP9.h"

/*********************************************************************************************************************/

class FElectraVideoDecoderVPx_Linux;


class FElectraVideoDecoderOutputVPx_Linux : public IElectraDecoderVideoOutput
{
public:
	virtual ~FElectraVideoDecoderOutputVPx_Linux()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return Width; }
	int32 GetDecodedHeight() const override
	{ return Height; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::LibavDecoderDecodedImage)
		{
			return DecodedImage.Get();
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;

	TSharedPtr<ILibavDecoderDecodedImage, ESPMode::ThreadSafe> DecodedImage;
};



class FElectraVideoDecoderVPx_Linux : public IElectraVideoDecoderVPx_Linux, public ILibavDecoderVideoResourceAllocator
{
public:
	FElectraVideoDecoderVPx_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraVideoDecoderVPx_Linux();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorVPx::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	// Add methods from ILibavDecoderVideoResourceAllocator here
	// ...

	enum class ECodec
	{
		VP8,
		VP9
	};

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		int32 NumBits = 8;
		bool bDropOutput = false;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	bool PostError(int32_t ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();

	enum class EConvertResult
	{
		Success,
		Failure
	};
	EConvertResult ConvertDecoderOutput(const ILibavDecoderVideoCommon::FOutputInfo& OutInfo);

	ECodec ActiveCodec = ECodec::VP9;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	IElectraDecoder::FError LastError;

	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;
	TSharedPtr<FElectraVideoDecoderOutputVPx_Linux, ESPMode::ThreadSafe> CurrentOutput;

	TSharedPtr<ILibavDecoderVideoCommon, ESPMode::ThreadSafe> DecoderInstance;
	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bNewDecoderRequired = false;
};


void IElectraVideoDecoderVPx_Linux::GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
{
	OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
	// The decoder is adaptive. There is no need to call IsCompatibleWith() on stream switches.
	OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
}

TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> IElectraVideoDecoderVPx_Linux::Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FElectraVideoDecoderVPx_Linux, ESPMode::ThreadSafe> New = MakeShared<FElectraVideoDecoderVPx_Linux>(InCodecFormat, InAdditionalOptions);
	return New;
}

FElectraVideoDecoderVPx_Linux::FElectraVideoDecoderVPx_Linux(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	ActiveCodec = InitialCodecFormat.FourCC == ElectraDecodersUtil::Make4CC('v','p','0','9') ? ECodec::VP9 : ECodec::VP8;
}

FElectraVideoDecoderVPx_Linux::~FElectraVideoDecoderVPx_Linux()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FElectraVideoDecoderVPx_Linux::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraVideoDecoderVPx_Linux::GetError() const
{
	return LastError;
}

void FElectraVideoDecoderVPx_Linux::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

bool FElectraVideoDecoderVPx_Linux::ResetToCleanStart()
{
	InternalDecoderDestroy();

	InDecoderInput.Empty();
	CurrentOutput.Reset();
	bNewDecoderRequired = false;
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

IElectraDecoder::ECSDCompatibility FElectraVideoDecoderVPx_Linux::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// NOTE: Since we set IElectraDecoderFeature::IsAdaptive to be true this method here will not be invoked on stream switches.

	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!DecoderInstance.IsValid())
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}

//	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	// The decoder is adaptive, so it is compatible.
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderVPx_Linux::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If this is discardable and won't be output we do not need to handle it at all.
	if ((InInputAccessUnit.Flags & (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable)) == (EElectraDecoderFlags::DoNotOutput | EElectraDecoderFlags::IsDiscardable))
	{
		return IElectraDecoder::EDecoderError::None;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// If a new decoder is needed, destroy the current one.
	if (bNewDecoderRequired)
	{
		InternalDecoderDestroy();
	}

	// Create decoder transform if necessary.
	if (!DecoderInstance.IsValid() && !InternalDecoderCreate(InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		int32 NumBits = 8;
		if (ActiveCodec == ECodec::VP9)
		{
			ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader Hdr;
			if (!ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Hdr, InInputAccessUnit.Data, InInputAccessUnit.DataSize))
			{
				PostError(0, TEXT("Failed to validate VP9 bitstream header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			NumBits = Hdr.GetBitDepth();
		}
		else //if (ActiveCodec == ECodec::VP8)
		{
			ElectraDecodersUtil::VPxVideo::FVP8UncompressedHeader Hdr;
			if (!ElectraDecodersUtil::VPxVideo::ParseVP8UncompressedHeader(Hdr, InInputAccessUnit.Data, InInputAccessUnit.DataSize))
			{
				PostError(0, TEXT("Failed to validate VP8 bitstream header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}

		uint8* Temp = nullptr;
		ILibavDecoderVideoCommon::FInputAU DecAU;
		DecAU.DataSize = InInputAccessUnit.DataSize;
		DecAU.Data = InInputAccessUnit.Data;
		DecAU.DTS = InInputAccessUnit.DTS.GetTicks();
		DecAU.UserValue = DecAU.PTS = InInputAccessUnit.PTS.GetTicks();
		DecAU.Duration = InInputAccessUnit.Duration.GetTicks();
		DecAU.Flags = 0;
		ILibavDecoder::EDecoderError Result = DecoderInstance->DecodeAccessUnit(DecAU);
		FMemory::Free(Temp);
		if (Result == ILibavDecoder::EDecoderError::None)
		{
			// Add to the list of inputs passed to the decoder.
			TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
			In->AdditionalOptions = InAdditionalOptions;
			In->AccessUnit = InInputAccessUnit;
			In->bDropOutput = (InInputAccessUnit.Flags & EElectraDecoderFlags::DoNotOutput) == EElectraDecoderFlags::DoNotOutput;
			In->NumBits = NumBits;
			// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
			// values here that would lead us to think that we do.
			In->AccessUnit.Data = nullptr;
			In->AccessUnit.DataSize = 0;
			InDecoderInput.Emplace(MoveTemp(In));
			InDecoderInput.Sort([](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e1, const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e2)
			{
				return e1->AccessUnit.PTS < e2->AccessUnit.PTS;
			});
			return IElectraDecoder::EDecoderError::None;
		}
		else
		{
			PostError(DecoderInstance->GetLastLibraryError(), TEXT("Failed to decode video frame"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderVPx_Linux::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->SendEndOfData();
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraVideoDecoderVPx_Linux::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (DecoderInstance.IsValid())
	{
		InternalDecoderDestroy();
	}
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	bNewDecoderRequired = false;
	DecodeState = EDecodeState::Decoding;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraVideoDecoderVPx_Linux::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !DecoderInstance.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	ILibavDecoderVideoCommon::FOutputInfo Out;
	switch(DecoderInstance->HaveOutput(Out))
	{
		case ILibavDecoder::EOutputStatus::Available:
		{
			return ConvertDecoderOutput(Out) == EConvertResult::Success ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::Error;
		}
		case ILibavDecoder::EOutputStatus::EndOfData:
		{
			DecodeState = EDecodeState::Decoding;
			InDecoderInput.Empty();
			bNewDecoderRequired = true;
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		default:
		case ILibavDecoder::EOutputStatus::NeedInput:
		{
			return IElectraDecoder::EOutputStatus::NeedInput;
		}
	}
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraVideoDecoderVPx_Linux::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

void FElectraVideoDecoderVPx_Linux::Suspend()
{
}

void FElectraVideoDecoderVPx_Linux::Resume()
{
}

bool FElectraVideoDecoderVPx_Linux::PostError(int32_t ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

bool FElectraVideoDecoderVPx_Linux::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	CurrentOutput.Reset();
	bNewDecoderRequired = false;

	TMap<FString, FVariant> Options;
	Options.Add(TEXT("hw_priority"), FVariant(FString("cuda;vdpau;vaapi")));
	if (ActiveCodec == ECodec::VP8)
	{
		if (!ILibavDecoderVP8::IsAvailable())
		{
			return PostError(0, "libavcodec does not support this video format", ERRCODE_INTERNAL_DECODER_NOT_SUPPORTED);
		}
		DecoderInstance = ILibavDecoderVP8::Create(this, Options);
	}
	else
	{
		if (!ILibavDecoderVP9::IsAvailable())
		{
			return PostError(0, "libavcodec does not support this video format", ERRCODE_INTERNAL_DECODER_NOT_SUPPORTED);
		}
		DecoderInstance = ILibavDecoderVP9::Create(this, Options);
	}
	int32 ErrorCode = 0;
	if (DecoderInstance.IsValid() && ((ErrorCode = DecoderInstance->GetLastLibraryError())==0))
	{
		return true;
	}
	InternalDecoderDestroy();
	PostError(ErrorCode, "libavcodec failed to open video decoder", ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	return false;
}

void FElectraVideoDecoderVPx_Linux::InternalDecoderDestroy()
{
	if (DecoderInstance.IsValid())
	{
		DecoderInstance->Reset();
		DecoderInstance.Reset();
	}
}

FElectraVideoDecoderVPx_Linux::EConvertResult FElectraVideoDecoderVPx_Linux::ConvertDecoderOutput(const ILibavDecoderVideoCommon::FOutputInfo& OutInfo)
{
	if (!InDecoderInput.Num())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	// Find the input corresponding to this output.
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec]->AccessUnit.PTS.GetTicks() == OutInfo.UserValue)
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	TSharedPtr<FElectraVideoDecoderOutputVPx_Linux, ESPMode::ThreadSafe> NewOutput = MakeShared<FElectraVideoDecoderOutputVPx_Linux>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;
	NewOutput->DecodedImage = DecoderInstance->GetOutput();

	NewOutput->Width = OutInfo.Width;
	NewOutput->Height = OutInfo.Height;
	NewOutput->Pitch = NewOutput->Width;
	NewOutput->NumBits = In->NumBits;
	NewOutput->PixelFormat = 0;
	NewOutput->AspectW = 1;
	NewOutput->AspectH = 1;

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("linux")));
	NewOutput->ExtraValues.Emplace(TEXT("decoder"), FVariant(TEXT("libavcodec")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("vpx")));

	// Take this output on if it is not to be dropped.
	// We need to have pulled it out of libav before deciding this though.
	if (!In->bDropOutput)
	{
		CurrentOutput = MoveTemp(NewOutput);
	}
	return EConvertResult::Success;
}
