// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "IElectraDecoder.h"

class IElectraVideoDecoderVPx_Linux : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions);
	static TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> Create(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);

	virtual ~IElectraVideoDecoderVPx_Linux() = default;
};
