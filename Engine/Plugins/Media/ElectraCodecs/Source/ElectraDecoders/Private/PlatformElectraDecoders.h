// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if ELECTRA_DECODERS_HAVE_PLATFORM_DEFAULTS
#include COMPILED_PLATFORM_HEADER(PlatformElectraDecoders.h)
#else

class IElectraCodecRegistry;
class FPlatformElectraDecodersNull
{
public:
	static void Startup()
	{
	}
	static void Shutdown()
	{
	}
	static void RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith)
	{
	}
};

using FPlatformElectraDecoders = FPlatformElectraDecodersNull;

#endif
