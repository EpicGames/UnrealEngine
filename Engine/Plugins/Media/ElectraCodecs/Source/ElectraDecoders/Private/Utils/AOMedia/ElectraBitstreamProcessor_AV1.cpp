// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/AOMedia/ElectraBitstreamProcessor_AV1.h"
#include "Utils/AOMedia/ElectraUtilsAV1Video.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"


class FElectraDecoderBitstreamProcessorAV1::FImpl
{
public:
	class FBitstreamInfo : public IElectraDecoderBitstreamInfo
	{
	public:
		virtual ~FBitstreamInfo() = default;
	};

	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties)
	{
		if (CurrentColorimetry.IsValid() && bColorimetryChanged)
		{
			bColorimetryChanged = false;
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}

		if (CurrentMDCV.IsValid() && bMDCVChanged)
		{
			bMDCVChanged = false;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentMDCV.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume, TArray<uint8>(Params));
		}

		if (CurrentCLLI.IsValid() && bCLLIChanged)
		{
			bCLLIChanged = false;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentCLLI.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo, TArray<uint8>(Params));
		}
	}

	void UpdateColorimetry(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
	{
		if (!CurrentColorimetry.IsValid() ||
			CurrentColorimetry->colour_primaries != colour_primaries ||
			CurrentColorimetry->transfer_characteristics != transfer_characteristics ||
			CurrentColorimetry->matrix_coeffs != matrix_coeffs ||
			CurrentColorimetry->video_full_range_flag != video_full_range_flag ||
			CurrentColorimetry->video_format != video_format)
		{
			CurrentColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
			CurrentColorimetry->colour_primaries = colour_primaries;
			CurrentColorimetry->transfer_characteristics = transfer_characteristics;
			CurrentColorimetry->matrix_coeffs = matrix_coeffs;
			CurrentColorimetry->video_full_range_flag = video_full_range_flag;
			CurrentColorimetry->video_format = video_format;
			bColorimetryChanged = true;
		}
	}

	void UpdateMDCV(const ElectraDecodersUtil::AV1Video::FMetadata_hdr_mdvc& InMDCV)
	{
		ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume mdcv;
		mdcv.display_primaries_x[0] = InMDCV.primary_chromaticity_x[0];
		mdcv.display_primaries_y[0] = InMDCV.primary_chromaticity_y[0];
		mdcv.display_primaries_x[1] = InMDCV.primary_chromaticity_x[1];
		mdcv.display_primaries_y[1] = InMDCV.primary_chromaticity_y[1];
		mdcv.display_primaries_x[2] = InMDCV.primary_chromaticity_x[2];
		mdcv.display_primaries_y[2] = InMDCV.primary_chromaticity_y[2];
		mdcv.white_point_x = InMDCV.white_point_chromaticity_x;
		mdcv.white_point_y = InMDCV.white_point_chromaticity_y;
		mdcv.max_display_mastering_luminance = InMDCV.luminance_max;
		mdcv.min_display_mastering_luminance = InMDCV.luminance_min;

		if (!CurrentMDCV || *CurrentMDCV != mdcv)
		{
			TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> NewMDCV = MakeShared<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe>(mdcv);
			CurrentMDCV = MoveTemp(NewMDCV);
			bMDCVChanged = true;
		}
	}

	void UpdateCLL(const ElectraDecodersUtil::AV1Video::FMetadata_hdr_cll& InCLL)
	{
		ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info cll;
		cll.max_content_light_level = InCLL.max_cll;
		cll.max_pic_average_light_level = InCLL.max_fall;
		if (!CurrentCLLI || *CurrentCLLI != cll)
		{
			TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> NewCLL = MakeShared<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe>(cll);
			CurrentCLLI = MoveTemp(NewCLL);
			bCLLIChanged = true;
		}
	}

	void Clear()
	{
		CurrentColorimetry.Reset();
		CurrentMDCV.Reset();
		CurrentCLLI.Reset();
		bColorimetryChanged = false;
		bMDCVChanged = false;
		bCLLIChanged = false;
		LastErrorMessage.Empty();
	}

	static FString DCRName;
	static FString CSDName;

	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> CurrentMDCV;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> CurrentCLLI;

	bool bColorimetryChanged = false;
	bool bMDCVChanged = false;
	bool bCLLIChanged = false;

	FString LastErrorMessage;
};
FString FElectraDecoderBitstreamProcessorAV1::FImpl::DCRName(TEXT("dcr"));
FString FElectraDecoderBitstreamProcessorAV1::FImpl::CSDName(TEXT("csd"));


FElectraDecoderBitstreamProcessorAV1::~FElectraDecoderBitstreamProcessorAV1()
{
}

FElectraDecoderBitstreamProcessorAV1::FElectraDecoderBitstreamProcessorAV1(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	Impl = MakePimpl<FImpl>();
}

bool FElectraDecoderBitstreamProcessorAV1::WillModifyBitstreamInPlace()
{
	return false;
}

void FElectraDecoderBitstreamProcessorAV1::Clear()
{
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorAV1::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	// Already processed?
	if ((InOutAccessUnit.Flags & EElectraDecoderFlags::InputIsProcessed) != EElectraDecoderFlags::None)
	{
		return EProcessResult::Ok;
	}

	// Set to processed even if we fail somewhere now.
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;

	// Go over the data and look at the OBUs
	const uint8* CurrentOBU = (const uint8*)InOutAccessUnit.Data;
	const uint8* EndOfOBUs = CurrentOBU + InOutAccessUnit.DataSize;
	while(CurrentOBU < EndOfOBUs)
	{
		const uint8 OBUheaderByte = *CurrentOBU;
		if ((OBUheaderByte & 0x80) != 0)
		{
			Impl->LastErrorMessage = FString::Printf(TEXT("obu_forbidden_bit not zero"));
			return EProcessResult::Error;
		}
		const ElectraDecodersUtil::AV1Video::EOBUType OBUtype = (const ElectraDecodersUtil::AV1Video::EOBUType)(OBUheaderByte >> 3);
		const uint8 obu_extension_flag = (OBUheaderByte >> 2) & 1;
		const uint8 obu_has_size_field = (OBUheaderByte >> 1) & 1;
		const uint8 obu_reserved_1bit = OBUheaderByte & 1;
		CurrentOBU += obu_extension_flag ? 2 : 1;
		uint32 obu_size;
		uint32 nr = 0;
		if (obu_has_size_field)
		{
			ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, EndOfOBUs - CurrentOBU);
			obu_size = bs.leb128(nr);
			CurrentOBU += nr;
		}
		else
		{
			obu_size = EndOfOBUs - CurrentOBU;
		}

		if (OBUtype == ElectraDecodersUtil::AV1Video::EOBUType::OBU_SEQUENCE_HEADER)
		{
			ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, obu_size);
			ElectraDecodersUtil::AV1Video::FSequenceHeader sh;
			if (!sh.ParseFrom(bs))
			{
				Impl->LastErrorMessage = FString::Printf(TEXT("Failed to parse sequence_header_obu"));
				return EProcessResult::Error;
			}
			Impl->UpdateColorimetry((uint8)sh.color_config.color_primaries, (uint8)sh.color_config.transfer_characteristics, (uint8)sh.color_config.matrix_coefficients, sh.color_config.color_range, 5);
		}
		// HDR metadata?
		else if (OBUtype == ElectraDecodersUtil::AV1Video::EOBUType::OBU_METADATA)
		{
			ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, obu_size);
			const ElectraDecodersUtil::AV1Video::EMetadataType MetadataType = (const ElectraDecodersUtil::AV1Video::EMetadataType)(bs.leb128(nr));
			switch(MetadataType)
			{
				case ElectraDecodersUtil::AV1Video::EMetadataType::METADATA_TYPE_HDR_CLL:
				{
					ElectraDecodersUtil::AV1Video::FMetadata_hdr_cll cll;
					if (cll.ParseFrom(bs))
					{
						Impl->UpdateCLL(cll);
					}
					break;
				}
				case ElectraDecodersUtil::AV1Video::EMetadataType::METADATA_TYPE_HDR_MDCV:
				{
					ElectraDecodersUtil::AV1Video::FMetadata_hdr_mdvc mdcv;
					if (mdcv.ParseFrom(bs))
					{
						Impl->UpdateMDCV(mdcv);
					}
					break;
				}
			}
		}
		CurrentOBU += obu_size;
	}

	return EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorAV1::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	Impl->SetPropertiesOnOutput(InOutProperties);
//	uint8 num_bits = 8;
//	InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::NumBitsLuma, FVariant(num_bits));
}

FString FElectraDecoderBitstreamProcessorAV1::GetLastError()
{
	return Impl->LastErrorMessage;
}

