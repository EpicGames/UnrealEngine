// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/AOMedia/ElectraUtilsAV1Video.h"
#include "Utilities/ElectraBitstream.h"


namespace ElectraDecodersUtil
{
	namespace AV1Video
	{

		FString FAV1CodecConfigurationRecord::GetFormatInfo(uint32 InProfile, uint32 InLevel)
		{
			FString fi;
			if (InProfile == 0)
			{
				fi = TEXT("Main");
			}
			else if (InProfile == 1)
			{
				fi = TEXT("High");
			}
			else if (InProfile == 2)
			{
				fi = TEXT("Professional");
			}
			else
			{
				fi = TEXT("<Unknown>");
			}
			fi.Append(TEXT(" Profile"));
			if (InLevel <= 23)
			{
				fi.Append(FString::Printf(TEXT(", level %d.%d"), 2 + (InLevel >> 2), InLevel & 3));
			}
			else
			{
				fi.Append(TEXT(" at unknown level"));
			}
			return fi;
		}

		FString FAV1CodecConfigurationRecord::GetFormatInfo()
		{
			return FString::Printf(TEXT("AV1, %s"), *GetFormatInfo(Profile, Level));
		}


		bool FAV1CodecConfigurationRecord::Parse(const TConstArrayView<uint8>& InDCR)
		{
			CSD.Empty();
			RawData.Empty();
			if (InDCR.Num() < 4)
			{
				return false;
			}
			RawData = InDCR;
			const uint8* Inav1CBox = InDCR.GetData();

			Profile = Inav1CBox[1] >> 5;
			Level = Inav1CBox[1] & 31;
			Tier = Inav1CBox[2] >> 7;
			const uint8 high_bitdepth = !!(Inav1CBox[2] & 0x40);
			const uint8 twelve_bit = !!(Inav1CBox[2] & 0x20);
			BitDepth = 8;
			if (Profile == 2 && high_bitdepth)
			{
				BitDepth = twelve_bit ? 12 : 10;
			}
			else if (Profile <= 2)
			{
				BitDepth = high_bitdepth ? 10 : 8;
			}
			ColorInfo.monochrome = (Inav1CBox[2] >> 4) & 1;
			ColorInfo.chromaSubsamplingX = (Inav1CBox[2] >> 3) & 1;
			ColorInfo.chromaSubsamplingY = (Inav1CBox[2] >> 2) & 1;
			ColorInfo.chromaSamplingPosition = Inav1CBox[2] & 3;

			initial_presentation_delay_present = !!(Inav1CBox[3] & 0x10);
			if (initial_presentation_delay_present)
			{
				initial_presentation_delay_minus_one = Inav1CBox[3] & 0x0f;
			}

			if (InDCR.Num() > 4)
			{
				CSD = InDCR;
				CSD.RemoveAt(0, 4);
			}
			return true;
		}



		bool FMetadata_hdr_cll::ParseFrom(FBitstreamReaderAV1& InReader)
		{
			if (InReader.GetRemainingBits() < 32)
			{
				return false;
			}
			max_cll = InReader.GetBits(16);
			max_fall = InReader.GetBits(16);
			return true;
		}

		bool FMetadata_hdr_mdvc::ParseFrom(FBitstreamReaderAV1& InReader)
		{
			if (InReader.GetRemainingBits() < 192)
			{
				return false;
			}
			primary_chromaticity_x[0] = InReader.GetBits(16);
			primary_chromaticity_y[0] = InReader.GetBits(16);
			primary_chromaticity_x[1] = InReader.GetBits(16);
			primary_chromaticity_y[1] = InReader.GetBits(16);
			primary_chromaticity_x[2] = InReader.GetBits(16);
			primary_chromaticity_y[2] = InReader.GetBits(16);
			white_point_chromaticity_x = InReader.GetBits(16);
			white_point_chromaticity_y = InReader.GetBits(16);
			luminance_max = InReader.GetBits(32);
			luminance_min = InReader.GetBits(32);
			return true;
		}



		bool FSequenceHeader::ParseFrom(FBitstreamReaderAV1& InReader)
		{
			seq_profile = (uint8) InReader.GetBits(3);
			if (seq_profile > 2)
			{
				return false;
			}
			still_picture = InReader.GetBits(1);
			reduced_still_picture_header = InReader.GetBits(1);
			if (reduced_still_picture_header && !still_picture)
			{
				return false;
			}
			decoder_model_info_present_flag = 0;
			initial_display_delay_present_flag = 0;
			operating_points_cnt_minus_1 = 0;
			if (!reduced_still_picture_header)
			{
				timing_info_present_flag = InReader.GetBits(1);
				if (timing_info_present_flag)
				{
					// timing_info
					timing_info.num_units_in_display_tick = InReader.GetBits(32);
					timing_info.time_scale = InReader.GetBits(32);
					timing_info.equal_picture_interval = InReader.GetBits(1);
					if (timing_info.equal_picture_interval)
					{
						timing_info.num_ticks_per_picture_minus_1 = InReader.uvlc();
						if (timing_info.num_ticks_per_picture_minus_1 == 0xffffffffU)
						{
							return false;
						}
					}
					decoder_model_info_present_flag = InReader.GetBits(1);
					if (decoder_model_info_present_flag)
					{
						// decoder_model_info
						decoder_model_info.buffer_delay_length_minus_1 = InReader.GetBits(5);
						decoder_model_info.num_units_in_decoding_tick = InReader.GetBits(32);
						decoder_model_info.buffer_removal_time_length_minus_1 = InReader.GetBits(5);
						decoder_model_info.frame_presentation_time_length_minus_1 = InReader.GetBits(5);
					}
				}

				initial_display_delay_present_flag = InReader.GetBits(1);
				operating_points_cnt_minus_1 = InReader.GetBits(5);
				for(int32 i=0; i<=operating_points_cnt_minus_1; ++i)
				{
					operating_points[i].operating_point_idc = InReader.GetBits(12);
					operating_points[i].seq_level_idx = InReader.GetBits(5);
					operating_points[i].seq_tier = 0;
					if (operating_points[i].seq_level_idx > 7)
					{
						operating_points[i].seq_tier = InReader.GetBits(1);
					}
					if (decoder_model_info_present_flag)
					{
						operating_points[i].decoder_model_present_for_this_op = InReader.GetBits(1);
						if (operating_points[i].decoder_model_present_for_this_op)
						{
							// operating_parameters_info
							uint32 nb = decoder_model_info.buffer_delay_length_minus_1 + 1;
							operating_parameters_info[i].decoder_buffer_delay = InReader.GetBits(nb);
							operating_parameters_info[i].encoder_buffer_delay = InReader.GetBits(nb);
							operating_parameters_info[i].low_delay_mode_flag = InReader.GetBits(1);
						}
					}
					else
					{
						operating_points[i].decoder_model_present_for_this_op = 0;
					}
					operating_points[i].initial_display_delay_present_for_this_op = 0;
					operating_points[i].initial_display_delay_minus_1 = 0;
					if (initial_display_delay_present_flag)
					{
						operating_points[i].initial_display_delay_present_for_this_op = InReader.GetBits(1);
						if (operating_points[i].initial_display_delay_present_for_this_op)
						{
							operating_points[i].initial_display_delay_minus_1 = InReader.GetBits(4);
						}
					}
				}
			}
			else
			{
				timing_info_present_flag = 0;
				operating_points[0].operating_point_idc = 0;
				operating_points[0].seq_level_idx = InReader.GetBits(5);
				operating_points[0].seq_tier = 0;
				operating_points[0].decoder_model_present_for_this_op = 0;
				operating_points[0].initial_display_delay_present_for_this_op = 0;
			}

			//operatingPoint = choose_operating_point();
			//OperatingPointIdc = operating_point_idc[operatingPoint];

			frame_width_bits_minus_1 = InReader.GetBits(4);
			frame_height_bits_minus_1 = InReader.GetBits(4);
			max_frame_width_minus_1 = InReader.GetBits(frame_width_bits_minus_1 + 1);
			max_frame_height_minus_1 = InReader.GetBits(frame_height_bits_minus_1 + 1);
			frame_id_numbers_present_flag = 0;
			if (!reduced_still_picture_header)
			{
				frame_id_numbers_present_flag = InReader.GetBits(1);
			}
			if (frame_id_numbers_present_flag)
			{
				delta_frame_id_length_minus_2 = InReader.GetBits(4);
				additional_frame_id_length_minus_1 = InReader.GetBits(3);
			}
			use_128x128_superblock = InReader.GetBits(1);
			enable_filter_intra = InReader.GetBits(1);
			enable_intra_edge_filter = InReader.GetBits(1);

			const uint8 SELECT_SCREEN_CONTENT_TOOLS = 2;
			const uint8 SELECT_INTEGER_MV = 2;

			enable_interintra_compound = 0;
			enable_masked_compound = 0;
			enable_warped_motion = 0;
			enable_dual_filter = 0;
			enable_order_hint = 0;
			enable_jnt_comp = 0;
			enable_ref_frame_mvs = 0;
			seq_force_screen_content_tools = SELECT_SCREEN_CONTENT_TOOLS;
			seq_force_integer_mv = SELECT_INTEGER_MV;
			OrderHintBits = 0;
			if (!reduced_still_picture_header)
			{
				enable_interintra_compound = InReader.GetBits(1);
				enable_masked_compound = InReader.GetBits(1);
				enable_warped_motion = InReader.GetBits(1);
				enable_dual_filter = InReader.GetBits(1);
				enable_order_hint = InReader.GetBits(1);
				enable_jnt_comp = 0;
				enable_ref_frame_mvs = 0;
				if (enable_order_hint)
				{
					enable_jnt_comp = InReader.GetBits(1);
					enable_ref_frame_mvs = InReader.GetBits(1);
				}
				seq_choose_screen_content_tools = InReader.GetBits(1);
				seq_force_screen_content_tools = SELECT_SCREEN_CONTENT_TOOLS;
				if (!seq_choose_screen_content_tools)
				{
					seq_force_screen_content_tools = InReader.GetBits(1);
				}
				seq_force_integer_mv = SELECT_INTEGER_MV;
				if (seq_force_screen_content_tools)
				{
					seq_choose_integer_mv = InReader.GetBits(1);
					if (!seq_choose_integer_mv)
					{
						seq_force_integer_mv = InReader.GetBits(1);
					}
				}
				if (enable_order_hint)
				{
					order_hint_bits_minus_1 = InReader.GetBits(3);
				}
			}

			enable_superres = InReader.GetBits(1);
			enable_cdef = InReader.GetBits(1);
			enable_restoration = InReader.GetBits(1);

			// color_config
			int32 BitDepth = 8;
			color_config.high_bitdepth = InReader.GetBits(1);
			if (seq_profile == 2 && color_config.high_bitdepth)
			{
				color_config.twelve_bit = InReader.GetBits(1);
				BitDepth = color_config.twelve_bit ? 12 : 10;
			}
			else if (seq_profile <= 2)
			{
				BitDepth = color_config.high_bitdepth ? 10 : 8;
			}
			color_config.mono_chrome = seq_profile == 1 ? 0 : InReader.GetBits(1);
			color_config.color_description_present_flag = InReader.GetBits(1);
			if (color_config.color_description_present_flag)
			{
				color_config.color_primaries = (EColorPrimaries) InReader.GetBits(8);
				color_config.transfer_characteristics = (ETransferCharacteristics) InReader.GetBits(8);
				color_config.matrix_coefficients = (EMatrixCoefficients) InReader.GetBits(8);
			}
			else
			{
				color_config.color_primaries = EColorPrimaries::CP_UNSPECIFIED;
				color_config.transfer_characteristics = ETransferCharacteristics::TC_UNSPECIFIED;
				color_config.matrix_coefficients = EMatrixCoefficients::MC_UNSPECIFIED;
			}
			if (color_config.mono_chrome)
			{
				color_config.color_range = InReader.GetBits(1);
				color_config.subsampling_x = 1;
				color_config.subsampling_y = 1;
				color_config.chroma_sample_position = EChromaSamplePosition::CSP_UNKNOWN;
				color_config.separate_uv_delta_q = 0;
			}
			else if (color_config.color_primaries == EColorPrimaries::CP_BT_709 && color_config.transfer_characteristics == ETransferCharacteristics::TC_SRGB && color_config.matrix_coefficients == EMatrixCoefficients::MC_IDENTITY)
			{
				color_config.color_range = 1;
				color_config.subsampling_x = 0;
				color_config.subsampling_y = 0;
			}
			else
			{
				color_config.color_range = InReader.GetBits(1);
				if (seq_profile == 0)
				{
					color_config.subsampling_x = 1;
					color_config.subsampling_y = 1;
				}
				else if (seq_profile == 1)
				{
					color_config.subsampling_x = 0;
					color_config.subsampling_y = 0;
				}
				else
				{
					if (BitDepth == 12)
					{
						color_config.subsampling_x = InReader.GetBits(1);
						color_config.subsampling_y = color_config.subsampling_x ? InReader.GetBits(1) : 0;
					}
					else
					{
						color_config.subsampling_x = 1;
						color_config.subsampling_y = 0;
					}
				}
				if (color_config.subsampling_x && color_config.subsampling_y)
				{
					color_config.chroma_sample_position = (EChromaSamplePosition) InReader.GetBits(2);
				}
			}
			if (color_config.mono_chrome == 0)
			{
				color_config.separate_uv_delta_q = InReader.GetBits(1);
			}

			film_grain_params_present = InReader.GetBits(1);
			return true;
		}


	} // namespace AV1Video

} // namespace ElectraDecodersUtil
