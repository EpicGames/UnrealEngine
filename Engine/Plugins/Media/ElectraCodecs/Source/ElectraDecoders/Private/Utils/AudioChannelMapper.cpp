// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/AudioChannelMapper.h"
#include "ElectraDecodersUtils.h"

namespace Electra
{

	namespace DirectlyCopyable
	{
		// This defines the order in which FMediaAudioResampler expects the source channels.
		// See comments in MediaAudioResampler.cpp regarding the row order.
		#define CP IElectraDecoderAudioOutput::EChannelPosition
		static const CP _1[] = { CP::C };
		static const CP _2[] = { CP::L, CP::R };
		static const CP _3[] = { CP::L, CP::R, CP::C };
		static const CP _4[] = { CP::L, CP::R, CP::Ls, CP::Rs };
		static const CP _5[] = { CP::L, CP::R, CP::C, CP::Ls, CP::Rs };
		static const CP _6[] = { CP::L, CP::R, CP::C, CP::LFE, CP::Ls, CP::Rs };
		static const CP _7[] = { CP::L, CP::R, CP::Lsr, CP::LFE, CP::Rsr, CP::Ls, CP::Rs };
		static const CP _8[] = { CP::L, CP::R, CP::C, CP::LFE, CP::Ls, CP::Rs, CP::Lsr, CP::Rsr };
		static const CP * const ResamplerOrderMap[] = { _1, _2, _3, _4, _5, _6, _7, _8 };
		#undef CP
	}

	FAudioChannelMapper::FAudioChannelMapper()
	{
	}

	FAudioChannelMapper::~FAudioChannelMapper()
	{
	}


	bool FAudioChannelMapper::Initialize(TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> InSampleBlock)
	{
		Reset();

		check(InSampleBlock->GetSampleFormat() == IElectraDecoderAudioOutput::ESampleFormat::Int16 ||
			  InSampleBlock->GetSampleFormat() == IElectraDecoderAudioOutput::ESampleFormat::Float);
		if (InSampleBlock->GetSampleFormat() != IElectraDecoderAudioOutput::ESampleFormat::Int16 && InSampleBlock->GetSampleFormat() != IElectraDecoderAudioOutput::ESampleFormat::Float)
		{
			return false;
		}

		const int32 Stride = InSampleBlock->GetBytesPerFrame();
		const int32 NumBytesPerSample = InSampleBlock->GetBytesPerSample();
		BytesPerSample = NumBytesPerSample;

		bCanCopyDirectly = CanBeCopiedDirectly(InSampleBlock);
		if (bCanCopyDirectly)
		{
			// Even though we can copy the source over directly we still set up the target source layout
			// for later use. GetNumTargetChannels() requires this.
			for(int32 i=0; i<InSampleBlock->GetNumChannels(); ++i)
			{
				FTargetSource& ts = TargetSources.AddDefaulted_GetRef();
				ts.ChannelPosition = InSampleBlock->GetChannelPosition(i);
				ts.FirstOffset = BytesPerSample * i;
				ts.Stride = Stride;
			}
		}
		else
		{
			// Setup the mapping from the input channels as a mutable array.
			TArray<FTargetSource> InputSources;
			for(int32 i=0; i<InSampleBlock->GetNumChannels(); ++i)
			{
				FTargetSource& ts = InputSources.AddDefaulted_GetRef();
				ts.ChannelPosition = InSampleBlock->GetChannelPosition(i);
				ts.FirstOffset = NumBytesPerSample * i;
				ts.Stride = Stride;
			}
			// Now remove all the inputs the resampler does not support.
			for(int32 i=0; i<InputSources.Num(); ++i)
			{
				if (InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::C &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::L &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::R &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::LFE &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Ls &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Rs &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Lsr &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Rsr)
				{
					InputSources.RemoveAt(i);
					--i;
				}
			}

			// Find the smallest resampler layout that contains all the required channels in any order.
			int32 ResamplerChannelLayout = MatchesResamplerLayout(InputSources);
			// At the very least the last 8 channel layout has to match since we removed all other channels.
			// If no channels remain we can't do anything.
			if (ResamplerChannelLayout == 0)
			{
				return false;
			}

			// Create a channel mapping matching the channel layout of the resampler.
			// Channels that the resampler uses but are not present are filled with empty
			// mappings that create silence for that channel.
			CreateResamplerChannelMapping(ResamplerChannelLayout-1, InputSources);
		}
		return true;
	}


	bool FAudioChannelMapper::Initialize(int32 NumBytesPerSample, uint32 ChannelConfiguration)
	{
		Reset();
		#define CP(ch) { IElectraDecoderAudioOutput::EChannelPosition::ch }
		switch(ChannelConfiguration)
		{
			default:
			{
				break;
			}
			case 1:
			{
				FSourceLayout Layout[] = { CP(C) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 2:
			case 8:	// handle unspecified channel1+channel2 configuration as L+R
			{
				FSourceLayout Layout[] = { CP(L), CP(R) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 3:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 4:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Cs) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 5:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Ls), CP(Rs) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 6:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Ls), CP(Rs), CP(LFE) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 7:
			{
				FSourceLayout Layout[] = { CP(C), CP(Lc), CP(Rc), CP(L), CP(R), CP(Ls), CP(Rs), CP(LFE) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 9:
			{
				FSourceLayout Layout[] = { CP(L), CP(R), CP(Cs) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 10:
			{
				FSourceLayout Layout[] = { CP(L), CP(R), CP(Ls), CP(Rs) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 11:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Ls), CP(Rs), CP(Cs), CP(LFE) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 12:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Ls), CP(Rs), CP(Lsr), CP(Rsr), CP(LFE) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 13:
			{
				FSourceLayout Layout[] = { CP(C), CP(Lc), CP(Rc), CP(L), CP(R), CP(Lss), CP(Rss), CP(Lsr), CP(Rsr), CP(Cs), CP(LFE), CP(LFE2),
										   CP(Cv), CP(Lv), CP(Rv), CP(Lvss), CP(Rvss), CP(Ts), CP(Lvr), CP(Rvr), CP(Cvr), CP(Cb), CP(Lb), CP(Rb) };
				return Initialize(NumBytesPerSample, Layout);
			}
			case 14:
			{
				FSourceLayout Layout[] = { CP(C), CP(L), CP(R), CP(Ls), CP(Rs), CP(LFE), CP(Lv), CP(Rv) };
				return Initialize(NumBytesPerSample, Layout);
			}
		}
		#undef CP
		return false;
	}

	bool FAudioChannelMapper::Initialize(int32 NumBytesPerSample, TArrayView<const FSourceLayout> ChannelPositions)
	{
		check(NumBytesPerSample == 2 || NumBytesPerSample == 4);
		Reset();

		bCanCopyDirectly = CanBeCopiedDirectly(ChannelPositions);
		if (bCanCopyDirectly)
		{
			// Even though we can copy the source over directly we still set up the target source layout
			// for later use. GetNumTargetChannels() requires this.
			for(int32 i=0; i<ChannelPositions.Num(); ++i)
			{
				FTargetSource& ts = TargetSources.AddDefaulted_GetRef();
				ts.ChannelPosition = ChannelPositions[i].ChannelPosition;
				ts.FirstOffset = NumBytesPerSample * i;
				ts.Stride = NumBytesPerSample * ChannelPositions.Num();
			}
		}
		else
		{
			int32 Stride = NumBytesPerSample * ChannelPositions.Num();
			// Setup the mapping from the input channels as a mutable array.
			TArray<FTargetSource> InputSources;
			for(int32 i=0; i<ChannelPositions.Num(); ++i)
			{
				FTargetSource& ts = InputSources.AddDefaulted_GetRef();
				ts.ChannelPosition = ChannelPositions[i].ChannelPosition;
				ts.FirstOffset = NumBytesPerSample * i;
				ts.Stride = Stride;
			}
			// Now remove all the inputs we do not support.
			for(int32 i=0; i<InputSources.Num(); ++i)
			{
				if (InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::C &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::L &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::R &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::LFE &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Ls &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Rs &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Lsr &&
					InputSources[i].ChannelPosition != IElectraDecoderAudioOutput::EChannelPosition::Rsr)
				{
					InputSources.RemoveAt(i);
					--i;
				}
			}

			// Do the current sources match one of the channel layouts used by the resampler?
			int32 ResamplerChannelLayout = MatchesResamplerLayout(InputSources);
			// At the very least the last 8 channel layout has to match.
			check(ResamplerChannelLayout);
			if (ResamplerChannelLayout == 0)
			{
				return false;
			}

			// Create a channel mapping matching the channel layout of the resampler.
			// Channels that the resampler uses but are not present are filled with empty
			// mappings that create silence for that channel.
			CreateResamplerChannelMapping(ResamplerChannelLayout-1, InputSources);
		}

		BytesPerSample = NumBytesPerSample;
		return true;
	}

	bool FAudioChannelMapper::IsInitialized() const
	{
		return BytesPerSample != 0;
	}


	void FAudioChannelMapper::Reset()
	{
		TargetSources.Empty();
		BytesPerSample = 0;
		bCanCopyDirectly = false;
	}

	int32 FAudioChannelMapper::GetNumTargetChannels() const
	{
		return TargetSources.Num();
	}

	void FAudioChannelMapper::MapChannels(void* OutputBuffer, uint32 OutputBufferSize, const void* InputBuffer, uint32 InputBufferSize, int32 NumSamplesPerChannel) const
	{
		// Must have been initialized!
		check(IsInitialized());
		if (IsInitialized())
		{
			if (bCanCopyDirectly)
			{
				uint32 nc = InputBufferSize < OutputBufferSize ? InputBufferSize : OutputBufferSize;
				FMemory::Memcpy(OutputBuffer, InputBuffer, nc);
			}
			else
			{
				int32 OutStride = BytesPerSample * TargetSources.Num();
				// Copy channels in output order
				for(int32 Ch=0,ChMax=TargetSources.Num(); Ch<ChMax; ++Ch)
				{
					void* OutCh = ElectraDecodersUtil::AdvancePointer(OutputBuffer, Ch * BytesPerSample);
					const void* InCh = ElectraDecodersUtil::AdvancePointer(InputBuffer, TargetSources[Ch].FirstOffset);
					int32 InStride = TargetSources[Ch].Stride;
					if (BytesPerSample == 2)
					{
						if (TargetSources[Ch].ChannelPosition == IElectraDecoderAudioOutput::EChannelPosition::Disabled)
						{
							for(int32 i=0; i<NumSamplesPerChannel; ++i)
							{
								*reinterpret_cast<uint16*>(OutCh) = 0;
								OutCh = ElectraDecodersUtil::AdvancePointer(OutCh, OutStride);
							}
						}
						else
						{
							for(int32 i=0; i<NumSamplesPerChannel; ++i)
							{
								*reinterpret_cast<uint16*>(OutCh) = *reinterpret_cast<const uint16*>(InCh);
								OutCh = ElectraDecodersUtil::AdvancePointer(OutCh, OutStride);
								InCh = ElectraDecodersUtil::AdvancePointer(InCh, InStride);
							}
						}
					}
					else if (BytesPerSample == 4)
					{
						if (TargetSources[Ch].ChannelPosition == IElectraDecoderAudioOutput::EChannelPosition::Disabled)
						{
							for(int32 i=0; i<NumSamplesPerChannel; ++i)
							{
								*reinterpret_cast<uint32*>(OutCh) = 0;
								OutCh = ElectraDecodersUtil::AdvancePointer(OutCh, OutStride);
							}
						}
						else
						{
							for(int32 i=0; i<NumSamplesPerChannel; ++i)
							{
								*reinterpret_cast<uint32*>(OutCh) = *reinterpret_cast<const uint32*>(InCh);
								OutCh = ElectraDecodersUtil::AdvancePointer(OutCh, OutStride);
								InCh = ElectraDecodersUtil::AdvancePointer(InCh, InStride);
							}
						}
					}
				}
			}
		}
		else
		{
			FMemory::Memzero(OutputBuffer, OutputBufferSize);
		}
	}


	bool FAudioChannelMapper::CanBeCopiedDirectly(TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> InSampleBlock)
	{
		int32 nc = InSampleBlock->GetNumChannels();
		int32 NumUnspecifiedChannels = 0;
		for(int32 i=0; i<nc; ++i)
		{
			IElectraDecoderAudioOutput::EChannelPosition cp = InSampleBlock->GetChannelPosition(i);
			if (cp >= IElectraDecoderAudioOutput::EChannelPosition::Unspec0 && cp <= IElectraDecoderAudioOutput::EChannelPosition::Unspec31)
			{
				++NumUnspecifiedChannels;
			}
		}
		// If all channel positions are unspecified then we can copy directly. There are no known
		// positions, so it could be anything really.
		if (NumUnspecifiedChannels == nc)
		{
			return true;
		}

		if (nc >= 1 && nc <= 8)
		{
			const IElectraDecoderAudioOutput::EChannelPosition * const ResamplerOrder = DirectlyCopyable::ResamplerOrderMap[nc - 1];
			if (ResamplerOrder)
			{
				for(int32 i=0; i<nc; ++i)
				{
					if (InSampleBlock->GetChannelPosition(i) != ResamplerOrder[i])
					{
						return false;
					}
				}
				return true;
			}
		}
		return false;
	}

	bool FAudioChannelMapper::CanBeCopiedDirectly(TArrayView<const FSourceLayout> ChannelPositions)
	{
		if (ChannelPositions.Num() >= 1 && ChannelPositions.Num() <= 8)
		{
			const IElectraDecoderAudioOutput::EChannelPosition * const ResamplerOrder = DirectlyCopyable::ResamplerOrderMap[ChannelPositions.Num() - 1];
			if (ResamplerOrder)
			{
				for(int32 i=0; i<ChannelPositions.Num(); ++i)
				{
					if (ResamplerOrder[i] != ChannelPositions[i].ChannelPosition)
					{
						return false;
					}
				}
				return true;
			}
		}
		return false;
	}

	int32 FAudioChannelMapper::MatchesResamplerLayout(TArray<FTargetSource>& InputSources)
	{
		if (InputSources.Num())
		{
			for(int32 i=InputSources.Num(); i<=UE_ARRAY_COUNT(DirectlyCopyable::ResamplerOrderMap); ++i)
			{
				TArrayView<const IElectraDecoderAudioOutput::EChannelPosition> ResamplerChannels(DirectlyCopyable::ResamplerOrderMap[i-1], i);
				bool bAllChannelsPresent = true;
				for(int32 j=0; j<InputSources.Num(); ++j)
				{
					if (!ResamplerChannels.Contains(InputSources[j].ChannelPosition))
					{
						bAllChannelsPresent = false;
						break;
					}
				}
				if (bAllChannelsPresent)
				{
					return i;
				}
			}
		}
		return 0;
	}

	void FAudioChannelMapper::CreateResamplerChannelMapping(int32 ResamplerChannelIndex, const TArray<FTargetSource>& InSources)
	{
		check(ResamplerChannelIndex >= 0 && ResamplerChannelIndex < UE_ARRAY_COUNT(DirectlyCopyable::ResamplerOrderMap));
		if (ResamplerChannelIndex >= 0 && ResamplerChannelIndex < UE_ARRAY_COUNT(DirectlyCopyable::ResamplerOrderMap))
		{
			TArrayView<const IElectraDecoderAudioOutput::EChannelPosition> ResamplerChannels(DirectlyCopyable::ResamplerOrderMap[ResamplerChannelIndex], ResamplerChannelIndex+1);
			for(auto &ResamplerChannel : ResamplerChannels)
			{
				FTargetSource ts;
				int32 SourceIndex = InSources.IndexOfByPredicate([ResamplerChannel](const FTargetSource& In){return In.ChannelPosition == ResamplerChannel;});
				if (SourceIndex != INDEX_NONE)
				{
					TargetSources.Emplace(InSources[SourceIndex]);
				}
				else
				{
					TargetSources.AddDefaulted();
				}
			}
		}
		else
		{
			// Add at least one empty entry to keep everything happy.
			TargetSources.AddDefaulted();
		}
	}


} // namespace Electra
