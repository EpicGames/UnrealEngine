// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/ElectraBitstreamProcessorDefault.h"


/*
class FElectraDecoderBitstreamProcessorDefault::FImpl
{
public:
};
*/

FElectraDecoderBitstreamProcessorDefault::~FElectraDecoderBitstreamProcessorDefault()
{
}

FElectraDecoderBitstreamProcessorDefault::FElectraDecoderBitstreamProcessorDefault()
{
//	Impl = MakePimpl<FImpl>();
}

bool FElectraDecoderBitstreamProcessorDefault::WillModifyBitstreamInPlace()
{
	return false;
}

void FElectraDecoderBitstreamProcessorDefault::Clear()
{
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorDefault::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;
	return EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorDefault::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
}

FString FElectraDecoderBitstreamProcessorDefault::GetLastError()
{
	return FString();
}
