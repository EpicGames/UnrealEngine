// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/ElectraBitstreamProcessorGenericVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"



class FElectraDecoderBitstreamProcessorGenericVideo::FImpl
{
public:
	void Clear()
	{
		bSentColorimetry = false;
		bSentMDCV = false;
		bSentCLLI = false;
		// Do not clear out any error message!
	}

	FString GetLastError()
	{
		return LastErrorMessage;
	}

	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties)
	{
		if (CurrentColorimetry.IsValid() && !bSentColorimetry)
		{
			bSentColorimetry = true;
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}

		if (CurrentMDCV.IsValid() && !bSentMDCV)
		{
			bSentMDCV = true;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentMDCV.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume, TArray<uint8>(Params));
		}

		if (CurrentCLLI.IsValid() && !bSentCLLI)
		{
			bSentCLLI = true;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentCLLI.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo, TArray<uint8>(Params));
		}
	}

	void SetColorimetryFromCOLRBox(const TArray<uint8>& InCOLRBox)
	{
		COLRBox = InCOLRBox;
		UpdateCOLR();
	}
	void SetMasteringDisplayColorVolumeFromMDCVBox(const TArray<uint8>& InMDCVBox)
	{
		MDCVBox = InMDCVBox;
		UpdateMDCV();
	}
	void SetContentLightLevelInfoFromCLLIBox(const TArray<uint8>& InCLLIBox)
	{
		CLLIBox = InCLLIBox;
		UpdateCLLI();
	}
	void SetContentLightLevelFromCOLLBox(const TArray<uint8>& InCOLLBox)
	{
		if (InCOLLBox.Num() > 4)
		{
			uint8 BoxVersion = InCOLLBox[0];
			if (BoxVersion == 0)
			{
				// 'clli' box is the same as a version 0 'COLL 'coll' box.
				CLLIBox = TArray<uint8>(InCOLLBox.GetData() + 4, InCOLLBox.Num() - 4);
			}
		}
		UpdateCLLI();
	}
private:
	void UpdateCOLR()
	{
		if (COLRBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> NewColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromCOLRBox(*NewColorimetry, COLRBox))
		{
			CurrentColorimetry = MoveTemp(NewColorimetry);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `colr` box data");
		}
	}

	void UpdateMDCV()
	{
		if (MDCVBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> NewMDCV = MakeShared<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromMDCVBox(*NewMDCV, MDCVBox))
		{
			CurrentMDCV = MoveTemp(NewMDCV);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `mdcv` box data");
		}
	}

	void UpdateCLLI()
	{
		if (CLLIBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> NewCLLI = MakeShared<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromCLLIBox(*NewCLLI, CLLIBox))
		{
			CurrentCLLI = MoveTemp(NewCLLI);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `coll`/`clli` box data");
		}
	}

	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> CurrentMDCV;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> CurrentCLLI;
	TArray<uint8> COLRBox;
	TArray<uint8> MDCVBox;
	TArray<uint8> CLLIBox;
	FString LastErrorMessage;
	bool bSentColorimetry = false;
	bool bSentMDCV = false;
	bool bSentCLLI = false;
};


FElectraDecoderBitstreamProcessorGenericVideo::~FElectraDecoderBitstreamProcessorGenericVideo()
{
}

FElectraDecoderBitstreamProcessorGenericVideo::FElectraDecoderBitstreamProcessorGenericVideo(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	Impl = MakePimpl<FImpl>();

	const TArray<uint8>* colr_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','o','l','r'));
	const TArray<uint8>* mdcv_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('m','d','c','v'));
	const TArray<uint8>* coll_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','o','l','l'));
	const TArray<uint8>* clli_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','l','l','i'));
	if (colr_Box)
	{
		Impl->SetColorimetryFromCOLRBox(*colr_Box);
	}
	if (mdcv_Box)
	{
		Impl->SetMasteringDisplayColorVolumeFromMDCVBox(*mdcv_Box);
	}
	if (coll_Box)
	{
		Impl->SetContentLightLevelFromCOLLBox(*coll_Box);
	}
	if (clli_Box)
	{
		Impl->SetContentLightLevelInfoFromCLLIBox(*clli_Box);
	}
}

bool FElectraDecoderBitstreamProcessorGenericVideo::WillModifyBitstreamInPlace()
{
	return false;
}

void FElectraDecoderBitstreamProcessorGenericVideo::Clear()
{
	// This only clears the flags that we did not send the color parameters yet.
	// The actual parameters are left unchanged as they are one-time init on construction only.
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorGenericVideo::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;
	return EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorGenericVideo::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	Impl->SetPropertiesOnOutput(InOutProperties);
}

FString FElectraDecoderBitstreamProcessorGenericVideo::GetLastError()
{
	return Impl->GetLastError();
}
