// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraCodecFormatUtils.h"
#include "Features/IModularFeatures.h"
#include "Logging/LogMacros.h"

#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/AOMedia/ElectraUtilsAV1Video.h"

#include "MP4Utilities.h"
#include "MP4Boxes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogElectraFormatUtils, Log, All);
DEFINE_LOG_CATEGORY(LogElectraFormatUtils);


FElectraCodecFormatUtilsModularFeature* FElectraCodecFormatUtilsModularFeature::Self = nullptr;

void FElectraCodecFormatUtilsModularFeature::Startup()
{
	Self = new FElectraCodecFormatUtilsModularFeature;
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), Self);
}

void FElectraCodecFormatUtilsModularFeature::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), Self);
	delete Self;
	Self = nullptr;
}


namespace
{
static const FString MimeType_Video(TEXT("video"));
static const FString MimeType_Audio(TEXT("audio"));
static const FString MimeType_Text(TEXT("text"));
static const FString MimeType_Application(TEXT("application"));
static const FString MimeType_CodecsPrefix(TEXT("codecs="));

static const FString MimeSubType_TTMLXML(TEXT("ttml+xml"));
static const FString MimeSubType_MP4(TEXT("mp4"));
static const FString MimeSubType_VTT(TEXT("vtt"));

static const FString MimeSubType_AVC(TEXT("avc"));
static const FString MimeSubType_HEVC(TEXT("hevc"));
static const FString MimeSubType_H264(TEXT("H264"));
static const FString MimeSubType_H265(TEXT("H265"));
static const FString MimeSubType_VP8(TEXT("VP8"));
static const FString MimeSubType_VP9(TEXT("VP9"));
static const FString MimeSubType_AV1(TEXT("AV1"));

static const FString MimeSubType_MPA(TEXT("MPA"));
static const FString MimeSubType_MPEG(TEXT("mpeg"));
static const FString MimeSubType_MP4A(TEXT("mp4a"));
static const FString MimeSubType_MP3(TEXT("mp3"));

static const FString CodecPrefix_AVC(TEXT("avc"));
static const FString CodecPrefix_HVC1(TEXT("hvc1"));
static const FString CodecPrefix_HEV1(TEXT("hev1"));
static const FString CodecPrefix_VP8(TEXT("vp8"));
static const FString CodecPrefix_VP08(TEXT("vp08"));
static const FString CodecPrefix_VP9(TEXT("vp9"));
static const FString CodecPrefix_VP09(TEXT("vp09"));
static const FString CodecPrefix_AV1(TEXT("av01"));

static const FString CodecPrefix_MP4A(TEXT("mp4a"));
static const FString CodecPrefix_Opus(TEXT("opus"));
static const FString CodecPrefix_Flac(TEXT("flac"));

static const FString CodecPrefix_WebVTT(TEXT("wvtt"));
static const FString CodecPrefix_TTML(TEXT("stpp"));
static const FString CodecPrefix_TX3G(TEXT("tx3g"));


static inline uint32 BitReverse32(uint32 InValue)
{
	uint32 rev = 0;
	for (int32 i = 0; i < 32; ++i)
	{
		rev = (rev << 1) | (InValue & 1);
		InValue >>= 1;
	}
	return rev;
}

static inline constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
{
	return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
}

static FString MakePrintable4CC(uint32 InFourCC)
{
	TCHAR tc[4];
	tc[0] = (TCHAR) ((InFourCC >> 24) & 255);
	tc[1] = (TCHAR) ((InFourCC >> 16) & 255);
	tc[2] = (TCHAR) ((InFourCC >>  8) & 255);
	tc[3] = (TCHAR) ((InFourCC >>  0) & 255);
	for(int32 i=0;i<4; ++i)
	{
		tc[i] = tc[i] >= 32 && tc[i] <= 127 ? tc[i] : TCHAR(' ');
	}
	return FString::ConstructFromPtrSize(tc, 4);
}

static inline void LexFromStringHex(int32& OutValue, const TCHAR* Buffer)
{
	OutValue = FCString::Strtoi(Buffer, nullptr, 16);
}

static inline void LexFromStringHexU64(uint64& OutValue, const TCHAR* Buffer)
{
	OutValue = FCString::Strtoui64(Buffer, nullptr, 16);
}


static const TCHAR* GetTypeName(Electra::FCodecTypeFormat::EType InType)
{
	switch(InType)
	{
		case Electra::FCodecTypeFormat::EType::Video:
		{
			return TEXT("video");
		}
		case Electra::FCodecTypeFormat::EType::Audio:
		{
			return TEXT("audio");
		}
		case Electra::FCodecTypeFormat::EType::Subtitle:
		{
			return TEXT("subtitle");
		}
		case Electra::FCodecTypeFormat::EType::Timecode:
		{
			return TEXT("timecode");
		}
	}
	return TEXT("undefined");
}


static void RemapLegacyFourCC(Electra::FCodecTypeFormat& InOut)
{
	if (InOut.FourCC == Make4CC('.','m','p','3'))
	{
		InOut.FourCC = Make4CC('m','p','g','a');
	}
}

/**
 * For trivial 4CC's that do not require codec parameters, set the default mime type is none is set yet.
 */
static void SetDefaultMimeTypeAndRFC6381For4CC(Electra::FCodecTypeFormat& InOut)
{
	if (InOut.MimeType.IsEmpty())
	{
		switch(InOut.FourCC)
		{
			case Make4CC('m','p','g','a'):
			{
				InOut.MimeType = TEXT("audio/mpeg");
				// Unconditionally set the codec rfc to the type we can handle.
				InOut.RFC6381 = TEXT("mp4a.40.34");
				break;
			}
			case Make4CC('O','p','u','s'):
			{
				InOut.MimeType = TEXT("audio/opus");
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = TEXT("opus");
				}
				break;
			}
			case Make4CC('f','L','a','C'):
			{
				InOut.MimeType = TEXT("audio/flac");
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = TEXT("flac");
				}
				break;
			}
			case Make4CC('s','t','p','p'):
			{
				InOut.MimeType = TEXT("application/ttml+xml");
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = TEXT("stpp.ttml.im1t");
				}
				break;
			}
			case Make4CC('w','v','t','t'):
			{
				InOut.MimeType = TEXT("text/vtt");
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = TEXT("wvtt");
				}
				break;
			}
			case Make4CC('t','x','3','g'):
			{
				InOut.MimeType = TEXT("application/mp4");
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = TEXT("tx3g");
				}
				break;
			}
			default:
			{
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = MakePrintable4CC(InOut.FourCC);
				}
				break;
			}
		}
	}
}



/**
 * Attempts to parse the given MIME type string into its components.
 */
static bool ParseMimeType(FString& OutType, FString& OutSubType, TArray<FString>& OutCodecs, const FString& InMimeType)
{
	if (InMimeType.IsEmpty())
	{
		return false;
	}
	int32 DelimiterPos = INDEX_NONE;
	FString TypeSubType = InMimeType.FindChar(TCHAR(';'), DelimiterPos) ? InMimeType.Mid(0, DelimiterPos) : InMimeType;
	FString Parameter = DelimiterPos != INDEX_NONE ? InMimeType.Mid(DelimiterPos+1) : FString();
	TypeSubType.TrimStartAndEndInline();
	if (!TypeSubType.FindChar(TCHAR('/'), DelimiterPos))
	{
		return false;
	}
	OutType = TypeSubType.Mid(0, DelimiterPos);
	OutSubType = TypeSubType.Mid(DelimiterPos+1);
	Parameter.TrimStartAndEndInline();
	// Check if the parameter is 'codecs'.
	// Note: We do currently NOT handle 'codecs*' indicating the use of charset, language and escaped characters.
	if ((DelimiterPos = Parameter.Find(MimeType_CodecsPrefix)) != INDEX_NONE)
	{
		FString Codecs = Parameter.Mid(DelimiterPos + MimeType_CodecsPrefix.Len());
		// Codecs may be quoted. If it is a list it needs to because of the use of the comma ',' separator.
		Codecs.TrimQuotesInline();
		// It may be a comma separated list.
		OutCodecs.Empty();
		Codecs.ParseIntoArray(OutCodecs, TEXT(","), true);
		for(int32 i=0; i<OutCodecs.Num(); ++i)
		{
			OutCodecs[i].TrimStartAndEndInline();
			if (OutCodecs[i].IsEmpty())
			{
				OutCodecs.RemoveAt(i);
				--i;
			}
		}
	}
	return true;
}

/**
 * Attempts to determine the type of codec (video, audio, etc.) if the mimetype is provided.
 */
static void DetermineCodecTypeFromMimeType(Electra::FCodecTypeFormat& InOut)
{
	Electra::FCodecTypeFormat::EType Type = Electra::FCodecTypeFormat::EType::Invalid;
	// Is a MIME type given?
	FString MimeType, MimeSubtype;
	TArray<FString> MimeCodecs;
	if (ParseMimeType(MimeType, MimeSubtype, MimeCodecs, InOut.MimeType))
	{
		if (MimeType.Equals(MimeType_Video, ESearchCase::IgnoreCase))
		{
			Type = Electra::FCodecTypeFormat::EType::Video;
			// Check for audio subtypes that commonly appear in error with the video type
			if (MimeSubtype.Equals(MimeSubType_MPA, ESearchCase::IgnoreCase) ||
				MimeSubtype.Equals(MimeSubType_MP4A, ESearchCase::IgnoreCase) ||
				MimeSubtype.Equals(MimeSubType_MP3, ESearchCase::IgnoreCase))
			{
				Type = Electra::FCodecTypeFormat::EType::Audio;
			}
		}
		else if (MimeType.Equals(MimeType_Audio, ESearchCase::IgnoreCase))
		{
			Type = Electra::FCodecTypeFormat::EType::Audio;
		}
		else if (MimeType.Equals(MimeType_Text, ESearchCase::IgnoreCase))
		{
			if (MimeSubtype.Equals(MimeSubType_VTT, ESearchCase::IgnoreCase))
			{
				Type = Electra::FCodecTypeFormat::EType::Subtitle;
			}
		}
		else if (MimeType.Equals(MimeType_Application, ESearchCase::IgnoreCase))
		{
			if (MimeSubtype.Equals(MimeSubType_TTMLXML, ESearchCase::IgnoreCase) ||
				MimeSubtype.Equals(MimeSubType_MP4, ESearchCase::IgnoreCase))
			{
				Type = Electra::FCodecTypeFormat::EType::Subtitle;
			}
		}
		// Check if the mime type turns up a different format (if any) than the user had already set (if any).
		if (InOut.Type != Electra::FCodecTypeFormat::EType::Invalid && Type != Electra::FCodecTypeFormat::EType::Invalid && InOut.Type != Type)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "Format was set as %ls but identifed as %ls from MIME type \"%ls\"", GetTypeName(InOut.Type), GetTypeName(Type), *InOut.MimeType);
		}

		// Set the type from the mime type unless the user has explicitly set the type already.
		if (InOut.Type == Electra::FCodecTypeFormat::EType::Invalid)
		{
			InOut.Type = Type;
		}

		// Is there also a codec specified?
		if (!MimeCodecs.IsEmpty())
		{
			// There must only be a single codec given here, since the structure contains parameters for just one codec.
			// The user must not pass multiple codecs that may appear in the server received mimetype or conveyed in a playlist.
			// NOTE: Yes, there could be multiple codecs identifying eg. SVC or MVC on top of the base AVC.
			//       We do not handle this either.
			if (MimeCodecs.Num() == 1)
			{
				// The codecs parameter is rarely given on a mime type and if it is, if the user has already set an RFC6381 string
				// we do not look at the mime type provided codec at all. Otherwise we set it as that.
				if (InOut.RFC6381.IsEmpty())
				{
					InOut.RFC6381 = MimeCodecs[0];
				}
			}
			else
			{
				UE_LOGF(LogElectraFormatUtils, Log, "The MIME type \"%ls\" specifies more than one codec. Ignoring all of them.", *InOut.MimeType);
			}
		}
	}
}

static bool ParseCodecRFC_AVC(Electra::FCodecTypeFormat& InOut)
{
	// avc1 and avc3 (inband SPS/PPS) are recognized.
	const FString& rfc(InOut.RFC6381);
	if (rfc.Len() < 4)
	{
		return false;
	}
	// We support avc1 or avc3 only on the codec since we need the GOP to start with a SAP type 1 or 2.
	if (rfc[3] != TCHAR('1') && rfc[3] != TCHAR('3'))
	{
		return false;
	}
	InOut.Type = Electra::FCodecTypeFormat::EType::Video;
	if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
	{
		InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
	}
	uint32 Codec4CC = rfc[3] == TCHAR('1') ? Make4CC('a','v','c','1') : Make4CC('a','v','c','3');
	if (InOut.FourCC == 0)
	{
		InOut.FourCC = Codec4CC;
	}
	if (InOut.MimeType.IsEmpty())
	{
		InOut.MimeType = TEXT("video/H264");
	}

	// Profile and level follow?
	if (rfc.Len() > 5 && rfc[4] == TCHAR('.'))
	{
		Electra::FCodecTypeFormat::FVideo::FProfile pl;
		FString Temp;
		int32 TempValue;
		int32 DotPos;
		rfc.FindLastChar(TCHAR('.'), DotPos);
		// We recognize the expected format avcC.xxyyzz and for legacy reasons also avcC.xxx.zz
		if (rfc.Len() == 11 && DotPos == 4)
		{
			Temp = rfc.Mid(5, 2);
			LexFromStringHex(TempValue, *Temp);
			pl.Profile = TempValue;
			Temp = rfc.Mid(7, 2);
			LexFromStringHex(TempValue, *Temp);
			pl.Constraints = TempValue;
			Temp = rfc.Mid(9, 2);
			LexFromStringHex(TempValue, *Temp);
			pl.Level = TempValue;
		}
		else if (DotPos != INDEX_NONE)
		{
			Temp = rfc.Mid(5, DotPos-5);
			LexFromString(TempValue, *Temp);
			pl.Profile = TempValue;
			Temp = rfc.Mid(DotPos+1);
			LexFromString(TempValue, *Temp);
			pl.Level = TempValue;
		}
		else
		{
			return false;
		}

		if (InOut.FourCC && InOut.FourCC != Codec4CC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from RFC6381 \"%ls\"", InOut.FourCC, Codec4CC, *InOut.RFC6381);
		}
		if (InOut.FourCC == Codec4CC)
		{
			InOut.FourCC = Codec4CC;
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			vid.Profile = pl;
		}
		InOut.HumanReadableFormatInfo = FString::Printf(TEXT("AVC, %s"), *ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet::GetFormatInfo(pl.Profile, pl.Level, (uint32)pl.Constraints));
	}
	else
	{
		InOut.HumanReadableFormatInfo = TEXT("AVC");
	}
	return true;
}

static bool ParseCodecRFC_HEVC(Electra::FCodecTypeFormat& InOut)
{
	// hev1 and hvc1 (inband VPS/SPS/PPS) are recognized.
	FString rfc(InOut.RFC6381);
	if (rfc.Len() < 4)
	{
		return false;
	}
	InOut.Type = Electra::FCodecTypeFormat::EType::Video;
	if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
	{
		InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
	}
	uint32 Codec4CC = rfc.StartsWith(TEXT("hev")) ? Make4CC('h','e','v','1') : Make4CC('h','v','c','1');
	if (InOut.FourCC == 0)
	{
		InOut.FourCC = Codec4CC;
	}
	if (InOut.MimeType.IsEmpty())
	{
		InOut.MimeType = TEXT("video/H265");
	}

	int32 DotPos;
	if (rfc.FindChar(TCHAR('.'), DotPos))
	{
		Electra::FCodecTypeFormat::FVideo::FProfile pl;
		FString Temp;
		int32 general_profile_space = 0;
		int32 general_tier_flag = 0;
		int32 general_profile_idc = 0;
		int32 general_level_idc = 0;
		uint32 general_profile_compatibility_flag = 0;
		uint64 contraint_flags = 0;
		rfc.RightChopInline(DotPos + 1);
		// optional general_profile_space
		if (rfc[0] == TCHAR('A') || rfc[0] == TCHAR('B') || rfc[0] == TCHAR('C'))
		{
			general_profile_space = rfc[0] - TCHAR('A') + 1;
			rfc.RightChopInline(1);
		}
		else if (rfc[0] == TCHAR('a') || rfc[0] == TCHAR('b') || rfc[0] == TCHAR('c'))
		{
			general_profile_space = rfc[0] - TCHAR('a') + 1;
			rfc.RightChopInline(1);
		}
		// general_profile_idc
		if (rfc.FindChar(TCHAR('.'), DotPos))
		{
			Temp = rfc.Left(DotPos);
			rfc.RightChopInline(DotPos + 1);
			LexFromString(general_profile_idc, *Temp);
		}
		// general_profile_compatibility_flags
		if (rfc.FindChar(TCHAR('.'), DotPos))
		{
			Temp = rfc.Left(DotPos);
			rfc.RightChopInline(DotPos + 1);
			LexFromString(general_profile_compatibility_flag, *Temp);
		}
		// general_tier_flag
		if (rfc[0] != TCHAR('L') && rfc[0] != TCHAR('H') && rfc[0] != TCHAR('l') && rfc[0] != TCHAR('h'))
		{
			return false;
		}
		else if (rfc[0] == TCHAR('H') || rfc[0] == TCHAR('h'))
		{
			general_tier_flag = 1;
		}
		rfc.RightChopInline(1);
		// constraint_flags
		FString ConstraintFlags;
		if (rfc.FindChar(TCHAR('.'), DotPos))
		{
			ConstraintFlags = rfc.Mid(DotPos + 1);
			rfc.LeftInline(DotPos);
			ConstraintFlags.ReplaceInline(TEXT("."), TEXT(""));
			ConstraintFlags += TEXT("000000000000");
			ConstraintFlags.LeftInline(12);
			LexFromStringHexU64(contraint_flags, *ConstraintFlags);
		}
		// general_level_idc
		LexFromString(general_level_idc, *rfc);

		pl.Profile = general_profile_idc;
		pl.Level = general_level_idc;
		pl.ProfileSpace = general_profile_space;
		pl.CompatibilityFlags = BitReverse32(general_profile_compatibility_flag);
		pl.Tier = general_tier_flag;
		pl.Constraints = contraint_flags;

		if (InOut.FourCC && InOut.FourCC != Codec4CC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from RFC6381 \"%ls\"", InOut.FourCC, Codec4CC, *InOut.RFC6381);
		}
		if (InOut.FourCC == Codec4CC)
		{
			InOut.FourCC = Codec4CC;
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			vid.Profile = pl;
		}
		InOut.HumanReadableFormatInfo = FString::Printf(TEXT("HEVC, %s"), *ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet::GetFormatInfo(pl.Profile, pl.Level, pl.Constraints));
	}
	else
	{
		InOut.HumanReadableFormatInfo = TEXT("HEVC");
	}
	return true;
}

static bool ParseCodecRFC_VPx(Electra::FCodecTypeFormat& InOut)
{
	if (InOut.RFC6381.StartsWith(TEXT("vp9")))
	{
		InOut.RFC6381.RightChopInline(3);
		InOut.RFC6381.InsertAt(0, TEXT("vp09"));
	}
	else if (InOut.RFC6381.StartsWith(TEXT("vp8")))
	{
		InOut.RFC6381.RightChopInline(3);
		InOut.RFC6381.InsertAt(0, TEXT("vp08"));
	}
	FString rfc(InOut.RFC6381);
	if (rfc.Len() < 4)
	{
		return false;
	}
	InOut.Type = Electra::FCodecTypeFormat::EType::Video;
	if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
	{
		InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
	}
	uint32 Codec4CC = rfc.StartsWith(TEXT("vp08")) ? Make4CC('v','p','0','8') : Make4CC('v','p','0','9');
	if (InOut.FourCC == 0)
	{
		InOut.FourCC = Codec4CC;
	}
	if (InOut.FourCC && InOut.FourCC != Codec4CC)
	{
		UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from RFC6381 \"%ls\"", InOut.FourCC, Codec4CC, *InOut.RFC6381);
		InOut.FourCC = Codec4CC;
	}
	if (InOut.MimeType.IsEmpty())
	{
		InOut.MimeType = Codec4CC == Make4CC('v','p','0','8') ?	TEXT("video/VP8") : TEXT("video/VP9");
	}

	if (rfc.Len() > 4)
	{
		Electra::FCodecTypeFormat::FVideo::FProfile pl;
		FString Temp;
		int32 DotPos;
		if (rfc.FindChar(TCHAR('.'), DotPos))
		{
			int32 Components[8] {0}, NumComponents=0;
			rfc.RightChopInline(DotPos + 1);
			while(rfc.Len() && NumComponents<UE_ARRAY_COUNT(Components))
			{
				if (rfc.FindChar(TCHAR('.'), DotPos))
				{
					Temp = rfc.Left(DotPos);
					rfc.RightChopInline(DotPos + 1);
					LexFromString(Components[NumComponents++], *Temp);
				}
				else
				{
					LexFromString(Components[NumComponents++], *rfc);
					break;
				}
			}
			// If a parameter string is given then the first three components are mandatory.
			if (NumComponents < 3)
			{
				return false;
			}

			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			pl.Profile = Components[0];
			pl.Level = Components[1];
			vid.Profile = pl;
			vid.BitDepth = Components[2];
			Electra::FCodecTypeFormat::FVideo::FColorInfo ci;
			ci.chromaSubsampling = 1;
			ci.colourPrimaries = 1;
			ci.transferCharacteristics = 1;
			ci.matrixCoefficients = 1;
			ci.videoFullRangeFlag = 0;
			if (NumComponents > 3)
			{
				ci.chromaSubsampling = (uint8) Components[3];
				ci.colourPrimaries = (uint8) Components[4];
				ci.transferCharacteristics  = (uint8) Components[5];
				ci.matrixCoefficients = (uint8) Components[6];
				ci.videoFullRangeFlag = (uint8) Components[7];
			}
			vid.OptColorInfo = ci;
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("VP%d, Profile %u, level %u.%u"), Codec4CC == Make4CC('v','p','0','8') ? 8 : 9, pl.Profile, pl.Level / 10, pl.Level % 10);
		}
	}
	else
	{
		InOut.HumanReadableFormatInfo = FString::Printf(TEXT("VP%d"), Codec4CC == Make4CC('v','p','0','8') ? 8 : 9);
	}
	return true;
}

static bool ParseCodecRFC_AV1(Electra::FCodecTypeFormat& InOut)
{
	const FString& rfc(InOut.RFC6381);
	if (rfc.Len() < 4)
	{
		return false;
	}
	InOut.Type = Electra::FCodecTypeFormat::EType::Video;
	if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
	{
		InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
	}
	const uint32 Codec4CC = Make4CC('a','v','0','1');
	if (InOut.FourCC == 0)
	{
		InOut.FourCC = Codec4CC;
	}
	if (InOut.FourCC && InOut.FourCC != Codec4CC)
	{
		UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from RFC6381 \"%ls\"", InOut.FourCC, Codec4CC, *InOut.RFC6381);
		InOut.FourCC = Codec4CC;
	}
	if (InOut.MimeType.IsEmpty())
	{
		InOut.MimeType = TEXT("video/AV1");
	}
	if (rfc.Len() > 4)
	{
		TArray<FString> Parts;
		rfc.ParseIntoArray(Parts, TEXT("."), false);
		// Profile, level, tier and bitdepth are required.
		if (Parts.Num() < 4)
		{
			return false;
		}
		Electra::FCodecTypeFormat::FVideo::FProfile pl;
		ElectraDecodersUtil::AV1Video::FAV1CodecConfigurationRecord::FColorInfo av1Col;
		int32 NumBitsLuma = 8;
		auto ParseParts = [&]() -> bool
		{
			for(int32 i=1; i<Parts.Num(); ++i)
			{
				switch(i)
				{
					case 1:
					{
						// Profile. Has to be a single digit.
						if (Parts[i].Len() > 1)
						{
							return false;
						}
						LexFromString(pl.Profile, *Parts[i]);
						break;
					}
					case 2:
					{
						// Level and tier. Needs to be 2 digits and an 'M' or 'H'.
						if (Parts[i].Len() != 3 || (Parts[i][2]!=TCHAR('M') && Parts[i][2]!=TCHAR('m') && Parts[i][2]!=TCHAR('H') && Parts[i][2]!=TCHAR('h')))
						{
							return false;
						}
						pl.Tier = Parts[i][2]==TCHAR('H') || Parts[i][2]==TCHAR('h') ? 1 : 0;
						Parts[i].LeftChopInline(1);
						LexFromString(pl.Level, *Parts[i]);
						break;
					}
					case 3:
					{
						// Bitdepth, 2 digits
						if (Parts[i].Len() != 2)
						{
							return false;
						}
						LexFromString(NumBitsLuma, *Parts[i]);
						break;
					}
					/*
						From this point on missing or malformed components are not critical. As per the standard:
							"If any character that is not '.', digits, part of the AV1 4CC, or a tier value is encountered,
							the string SHALL be interpreted ignoring all the characters starting from that character."
					*/
					case 4:
					{
						// Monochrome, 1 digit
						if (Parts[i].Len() != 1)
						{
							return true;
						}
						LexFromString(av1Col.monochrome, *Parts[i]);
						break;
					}
					case 5:
					{
						// Chroma subsampling, 3 digits
						if (Parts[i].Len() != 3)
						{
							return true;
						}
						if (FChar::IsDigit(Parts[i][0])) av1Col.chromaSubsamplingX = Parts[i][0]-TCHAR('0'); else return true;
						if (FChar::IsDigit(Parts[i][1])) av1Col.chromaSubsamplingY = Parts[i][1]-TCHAR('0'); else return true;
						if (FChar::IsDigit(Parts[i][2])) av1Col.chromaSamplingPosition = Parts[i][2]-TCHAR('0'); else return true;
						break;
					}
					case 6:
					{
						// colorPrimaries.
						if (Parts[i].Len() != 2)
						{
							return false;
						}
						LexFromString(av1Col.colorPrimaries, *Parts[i]);
						break;
					}
					case 7:
					{
						// transferCharacteristics.
						if (Parts[i].Len() != 2)
						{
							return false;
						}
						LexFromString(av1Col.transferCharacteristics, *Parts[i]);
						break;
					}
					case 8:
					{
						// matrixCoefficients.
						if (Parts[i].Len() != 2)
						{
							return false;
						}
						LexFromString(av1Col.matrixCoefficients, *Parts[i]);
						break;
					}
					case 9:
					{
						// videoFullRangeFlag.
						if (Parts[i].Len() != 1)
						{
							return false;
						}
						LexFromString(av1Col.videoFullRangeFlag, *Parts[i]);
						break;
					}
					default:
					{
						break;
					}
				}
			}
			return true;
		};
		if (!ParseParts())
		{
			return false;
		}

		Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
		vid.Profile = pl;
		vid.BitDepth = NumBitsLuma;
		Electra::FCodecTypeFormat::FVideo::FColorInfo ci;
		ci.colourPrimaries = av1Col.colorPrimaries;
		ci.transferCharacteristics = av1Col.transferCharacteristics;
		ci.matrixCoefficients = av1Col.matrixCoefficients;
		ci.videoFullRangeFlag = av1Col.videoFullRangeFlag;
		Electra::FCodecTypeFormat::FVideo::FColorInfo::FChromaInfo cri;
		cri.monochrome = av1Col.monochrome;
		cri.chromaSubsamplingX = av1Col.chromaSubsamplingX;
		cri.chromaSubsamplingY = av1Col.chromaSubsamplingY;
		cri.chromaSamplingPosition = av1Col.chromaSamplingPosition;
		ci.ChromaInfo = cri;
		vid.OptColorInfo = ci;
		InOut.HumanReadableFormatInfo = FString::Printf(TEXT("AV1, %s"), *ElectraDecodersUtil::AV1Video::FAV1CodecConfigurationRecord::GetFormatInfo(pl.Profile, pl.Level));
	}
	else
	{
		InOut.HumanReadableFormatInfo = TEXT("AV1");
	}
	return true;
}



static void MakeHumanReadableRFC_MP4A(Electra::FCodecTypeFormat& InOut, int32 InAudioObjectType, int32 InMpegVersion, int32 InLayer)
{
	if (InOut.FourCC == Make4CC('m','p','4','a'))
	{
		if (InAudioObjectType)
		{
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("AAC (%s)"), InAudioObjectType==2 ? TEXT("LC") : InAudioObjectType==5 ? TEXT("SBR") : InAudioObjectType==29 ? TEXT("PS") : TEXT("unknown"));
		}
		else
		{
			InOut.HumanReadableFormatInfo = TEXT("AAC");
		}
	}
	else
	{
		FString mp(InMpegVersion ? FString::Printf(TEXT("MPEG%d"), InMpegVersion) : TEXT("MPEG"));
		if (InLayer)
		{
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("%s layer %d"), *mp, InLayer);
		}
		else
		{
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("%s audio"), *mp);
		}
	}
}

static bool ParseCodecRFC_MP4A(Electra::FCodecTypeFormat& InOut)
{
	const FString& rfc(InOut.RFC6381);
	if (rfc.Len() < 4)
	{
		return false;
	}
	InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
	if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
	{
		InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
	}
	Electra::FCodecTypeFormat::FAudio& aud(InOut.Properties.Get<Electra::FCodecTypeFormat::FAudio>());

	TArray<FString> Parts;
	rfc.ParseIntoArray(Parts, TEXT("."));
	uint32 Codec4CC = Make4CC('m','p','4','a');
	int32 AudioObjectType = 0;
	int32 Mpeg = 0;
	int32 Layer = 0;
	if (Parts.Num() > 1)
	{
		int32 OTI = 0;
		LexFromStringHex(OTI, *Parts[1]);
		switch(static_cast<ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID>(OTI))
		{
			// ISO/IEC 14496-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG4_Audio:
			{
				if (Parts.Num() < 3)
				{
					UE_LOGF(LogElectraFormatUtils, Log, "RFC6381 \"%ls\" type needs the audio type indicator", *InOut.RFC6381);
					return false;
				}
				LexFromString(AudioObjectType, *Parts[2]);
				switch(AudioObjectType)
				{
					case 2:		// AAC-LC
					case 5:		// SBR
					case 29:	// PS
					{
						Codec4CC = Make4CC('m','p','4','a');
						if (InOut.MimeType.IsEmpty())
						{
							InOut.MimeType = TEXT("audio/mp4");
						}
						break;
					}
					case 32:	// Layer-1
					case 33:	// Layer-2
					case 34:	// Layer-3
					{
						Codec4CC = Make4CC('m','p','g','a');
						if (InOut.MimeType.IsEmpty())
						{
							InOut.MimeType = TEXT("audio/mpeg");
						}
						Layer = AudioObjectType - 31;
						Mpeg = 1;
						break;
					}
					default:
					{
						// not supported
						return false;
					}
				}
				break;
			}
			// ISO/IEC 13818-7
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG2_AAC_LC:
			{
				Codec4CC = Make4CC('m','p','4','a');
				AudioObjectType = 2;
				if (InOut.MimeType.IsEmpty())
				{
					InOut.MimeType = TEXT("audio/mp4a");
				}
				break;
			}
			// ISO/IEC 13818-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG2_Audio:
			// ISO/IEC 11172-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG1_Audio:
			{
				Mpeg = static_cast<ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID>(OTI) == ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG1_Audio ? 1 : 2;
				// Both can be handled by an MPEG1 layer III decoder
				Codec4CC = Make4CC('m','p','g','a');
				if (InOut.MimeType.IsEmpty())
				{
					InOut.MimeType = TEXT("audio/mpeg");
				}
				break;
			}
			default:
			{
				return false;
			}
		}
		if (!aud.ObjectType.IsType<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>())
		{
			aud.ObjectType.Emplace<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>();
		}
		Electra::FCodecTypeFormat::FAudio::FMPEGObjectType& OT(aud.ObjectType.Get<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>());
		OT.ObjectType = (uint32) OTI;
		OT.AudioObjectType = (uint32)AudioObjectType;
		OT.MPEG = Mpeg;
		OT.Layer = Layer;
	}

	if (InOut.FourCC == 0)
	{
		InOut.FourCC = Codec4CC;
	}
	if (InOut.FourCC && Codec4CC && InOut.FourCC != Codec4CC)
	{
		UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from RFC6381 \"%ls\"", InOut.FourCC, Codec4CC, *InOut.RFC6381);
		InOut.FourCC = Codec4CC;
	}

	MakeHumanReadableRFC_MP4A(InOut, AudioObjectType, Mpeg, Layer);
	return true;
}


static bool DetermineCodecTypeFromRFC(Electra::FCodecTypeFormat& InOut)
{
	bool bOk = true;
	// Check the codec RFC
	if (!InOut.RFC6381.IsEmpty())
	{
		if (InOut.RFC6381.StartsWith(CodecPrefix_AVC))
		{
			bOk = ParseCodecRFC_AVC(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_HVC1) || InOut.RFC6381.StartsWith(CodecPrefix_HEV1))
		{
			bOk = ParseCodecRFC_HEVC(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_VP08) || InOut.RFC6381.StartsWith(CodecPrefix_VP8))
		{
			bOk = ParseCodecRFC_VPx(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_VP09) || InOut.RFC6381.StartsWith(CodecPrefix_VP9))
		{
			bOk = ParseCodecRFC_VPx(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_AV1))
		{
			bOk = ParseCodecRFC_AV1(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_MP4A))
		{
			bOk = ParseCodecRFC_MP4A(InOut);
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_Opus))
		{
			InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
			}
			InOut.FourCC = Make4CC('O','p','u','s');
			InOut.HumanReadableFormatInfo = TEXT("Opus");
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_Flac))
		{
			InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
			}
			InOut.FourCC = Make4CC('f','L','a','C');
			InOut.HumanReadableFormatInfo = TEXT("Free Lossless Audio Codec (FLAC)");
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_WebVTT))
		{
			InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
			}
			InOut.FourCC = Make4CC('w','v','t','t');
			InOut.HumanReadableFormatInfo = TEXT("WebVTT");
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_TTML))
		{
			InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
			}
			InOut.FourCC = Make4CC('s','t','p','p');
			InOut.HumanReadableFormatInfo = TEXT("Timed Text (TTML)");
		}
		else if (InOut.RFC6381.StartsWith(CodecPrefix_TX3G))
		{
			InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
			}
			InOut.FourCC = Make4CC('t','x','3','g');
			InOut.HumanReadableFormatInfo = TEXT("Timed Text (TX3G)");
		}
	}
	return bOk;
}


static int32 DetermineCodecTypeFromDCR_avcc(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* avcC_Box = InOut.ExtraBoxes.Find(Make4CC('a','v','c','C'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	bool bFromDCR = false;
	if (!avcC_Box && InOut.DCR.Num() && (InOut.FourCC == Make4CC('a','v','c','1') || InOut.FourCC == Make4CC('a','v','c','3')))
	{
		InOut.ExtraBoxes.Emplace(Make4CC('a','v','c','C'), InOut.DCR);
		avcC_Box = InOut.ExtraBoxes.Find(Make4CC('a','v','c','C'));
		bFromDCR = true;
	}
	if (avcC_Box)
	{
		if (InOut.MimeType.IsEmpty())
		{
			InOut.MimeType = TEXT("video/H264");
		}
		ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord dcr;
		if (dcr.Parse(*avcC_Box))
		{
			InOut.DCR = *avcC_Box;
			InOut.Type = Electra::FCodecTypeFormat::EType::Video;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
			}
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			if ((vid.Profile.Profile && vid.Profile.Profile != dcr.GetAVCProfileIndication()) ||
				(vid.Profile.Level && vid.Profile.Level != dcr.GetAVCLevelIndication()))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "AVC profile and level was set as %d,%d but specified as %d,%d in AVCDecoderConfigurationRecord", vid.Profile.Profile, vid.Profile.Level, dcr.GetAVCProfileIndication(), dcr.GetAVCLevelIndication());
			}
			vid.Profile.Profile = dcr.GetAVCProfileIndication();
			vid.Profile.Level = dcr.GetAVCLevelIndication();
			vid.Profile.Constraints = dcr.GetProfileCompatibility();
			InOut.CSD = dcr.GetCodecSpecificData();
			int32 FourCC = InOut.CSD.IsEmpty() ? Make4CC('a','v','c','3') : Make4CC('a','v','c','1');
			if (InOut.FourCC && InOut.FourCC != FourCC && FourCC == Make4CC('a','v','c','3'))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from AVCDecoderConfigurationRecord", InOut.FourCC, FourCC);
			}
			InOut.FourCC = FourCC;

			vid.BitDepth = dcr.GetBitDepthLuma();

			uint32 MaxWidth = 0;
			uint32 MaxHeight = 0;
			FFrameRate MaxResoFPS(0, 0);
			for(int32 i=0,iMax=dcr.GetSequenceParameterSets().Num(); i<iMax; ++i)
			{
				ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet sps;
				if (ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(sps, dcr.GetSequenceParameterSets()[i]))
				{
					uint32 w, h;
					sps.GetDisplaySize(w, h);
					if (w > MaxWidth || h > MaxHeight)
					{
						MaxWidth = w > MaxWidth ? w : MaxWidth;
						MaxHeight = h > MaxHeight ? h : MaxHeight;

						ElectraDecodersUtil::FFractionalValue vuiTiming = sps.GetTiming();
						if (vuiTiming.Num && vuiTiming.Denom)
						{
							double fps = (double)vuiTiming.Num / vuiTiming.Denom;
							double cfps = MaxResoFPS.IsValid() ? MaxResoFPS.AsDecimal() : 0.0;
							double r = cfps / fps;
							// There is no guarantee that the VUI timing in the SPS is correct if the timing is conveyed in the stream in SEI messages.
							if ((r < 1.2 && r > 0.8) || (cfps == 0.0 && fps <= 240.0))
							{
								MaxResoFPS.Numerator = (int32)vuiTiming.Num;
								MaxResoFPS.Denominator = (int32)vuiTiming.Denom;
							}
						}
					}
				}
			}
			// Check dimensions
			if ((vid.Width && vid.Width != MaxWidth) || (vid.Height && vid.Height != MaxHeight))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "Dimensions were set as %u*%u but given as %d*%d by the SPS", vid.Width, vid.Height, MaxWidth, MaxHeight);
				vid.Width = MaxWidth > 0 ? MaxWidth : vid.Width;
				vid.Height = MaxHeight > 0 ? MaxHeight : vid.Height;
			}
			else if ((!vid.Width && MaxWidth) || (!vid.Height && MaxHeight))
			{
				vid.Width = MaxWidth;
				vid.Height = MaxHeight;
			}
			// Only overwrite FrameRate from VUI timing if it was actually found.
			// Otherwise preserve the container-derived FrameRate (e.g. from STTS/MDHD).
			if (MaxResoFPS.Numerator > 0)
			{
				vid.FrameRate = MaxResoFPS;
			}

			InOut.RFC6381 = dcr.GetCodecSpecifierRFC6381(FourCC == Make4CC('a','v','c','1') ? TEXT("avc1") : TEXT("avc3"));
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("AVC, %s"), *ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet::GetFormatInfo(vid.Profile.Profile, vid.Profile.Level, (uint32)vid.Profile.Constraints));
			return 1;
		}
		return bFromDCR ? 0 : -1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_hvcc(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* hvcC_Box = InOut.ExtraBoxes.Find(Make4CC('h','v','c','C'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	bool bFromDCR = false;
	if (!hvcC_Box && InOut.DCR.Num() && (InOut.FourCC == Make4CC('h','v','c','1') || InOut.FourCC == Make4CC('h','e','v','1')))
	{
		InOut.ExtraBoxes.Emplace(Make4CC('h','v','c','C'), InOut.DCR);
		hvcC_Box = InOut.ExtraBoxes.Find(Make4CC('h','v','c','C'));
		bFromDCR = true;
	}
	if (hvcC_Box)
	{
		if (InOut.MimeType.IsEmpty())
		{
			InOut.MimeType = TEXT("video/H265");
		}
		ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord dcr;
		if (dcr.Parse(*hvcC_Box))
		{
			InOut.DCR = *hvcC_Box;
			InOut.Type = Electra::FCodecTypeFormat::EType::Video;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
			}
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			if ((vid.Profile.Profile && vid.Profile.Profile != dcr.GetGeneralProfileIDC()) ||
				(vid.Profile.Level && vid.Profile.Level != dcr.GetGeneralLevelIDC()))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "HEVC profile and level was set as %d,%d but specified as %d,%d in HEVCDecoderConfigurationRecord", vid.Profile.Profile, vid.Profile.Level, dcr.GetGeneralProfileIDC(), dcr.GetGeneralLevelIDC());
			}
			vid.Profile.Tier = dcr.GetGeneralTierFlag();
			vid.Profile.ProfileSpace = dcr.GetGeneralProfileSpace();
			vid.Profile.Profile = dcr.GetGeneralProfileIDC();
			vid.Profile.Level = dcr.GetGeneralLevelIDC();
			vid.Profile.Constraints = dcr.GetGeneralConstraintIndicatorFlags();
			vid.Profile.CompatibilityFlags = dcr.GetGeneralProfileCompatibilityFlags();
			InOut.CSD = dcr.GetCodecSpecificData();
			int32 FourCC = InOut.CSD.IsEmpty() ? Make4CC('h','e','v','1') : Make4CC('h','v','c','1');
			if (InOut.FourCC && InOut.FourCC != FourCC && FourCC == Make4CC('h','e','v','1'))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from HEVCDecoderConfigurationRecord", InOut.FourCC, FourCC);
			}
			InOut.FourCC = FourCC;

			vid.BitDepth = 8 + dcr.GetBitDepthLumaMinus8();

			uint32 MaxWidth = 0;
			uint32 MaxHeight = 0;
			FFrameRate MaxResoFPS(0, 0);
			for(int32 i=0,iMax=dcr.GetSequenceParameterSets().Num(); i<iMax; ++i)
			{
				ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet sps;
				if (ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(sps, dcr.GetSequenceParameterSets()[i]))
				{
					uint32 w, h;
					sps.GetDisplaySize(w, h);
					if (w > MaxWidth || h > MaxHeight)
					{
						MaxWidth = w > MaxWidth ? w : MaxWidth;
						MaxHeight = h > MaxHeight ? h : MaxHeight;

						ElectraDecodersUtil::FFractionalValue vuiTiming = sps.GetTiming();
						if (vuiTiming.Num && vuiTiming.Denom)
						{
							double fps = (double)vuiTiming.Num / vuiTiming.Denom;
							double cfps = MaxResoFPS.IsValid() ? MaxResoFPS.AsDecimal() : 0.0;
							double r = cfps / fps;
							// There is no guarantee that the VUI timing in the SPS is correct if the timing is conveyed in the stream in SEI messages.
							if ((r < 1.2 && r > 0.8) || (cfps == 0.0 && fps <= 240.0))
							{
								MaxResoFPS.Numerator = (int32)vuiTiming.Num;
								MaxResoFPS.Denominator = (int32)vuiTiming.Denom;
							}
						}
					}
				}
			}
			// Check dimensions
			if ((vid.Width && vid.Width != MaxWidth) || (vid.Height && vid.Height != MaxHeight))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "Dimensions were set as %u*%u but given as %d*%d by the SPS", vid.Width, vid.Height, MaxWidth, MaxHeight);
				vid.Width = MaxWidth > 0 ? MaxWidth : vid.Width;
				vid.Height = MaxHeight > 0 ? MaxHeight : vid.Height;
			}
			else if ((!vid.Width && MaxWidth) || (!vid.Height && MaxHeight))
			{
				vid.Width = MaxWidth;
				vid.Height = MaxHeight;
			}
			// Only overwrite FrameRate from VUI timing if it was actually found.
			// Otherwise preserve the container-derived FrameRate (e.g. from STTS/MDHD).
			// Android HEVC recordings often omit VUI timing parameters in the SPS.
			if (MaxResoFPS.Numerator > 0)
			{
				vid.FrameRate = MaxResoFPS;
			}

			InOut.RFC6381 = dcr.GetCodecSpecifierRFC6381(FourCC == Make4CC('h','e','v','1') ? TEXT("hev1") : TEXT("hvc1"));
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("HEVC, %s"), *ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet::GetFormatInfo(vid.Profile.Profile, vid.Profile.Level, (uint32)vid.Profile.Constraints));
			return 1;
		}
		return bFromDCR ? 0 : -1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_vpcc(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* vpcC_Box = InOut.ExtraBoxes.Find(Make4CC('v','p','c','C'));
	int32 DataOffset = 4;
	// Check if in absence of the codec format box a codec specific DCR is provided.
	bool bFromDCR = false;
	if (!vpcC_Box && InOut.DCR.Num() > 4 && (InOut.FourCC == Make4CC('v','p','0','8') || InOut.FourCC == Make4CC('v','p','0','9')))
	{
		// Check if the DCR which should be a VPCodecConfigurationRecord is accidentally wrapped in an mp4 full box.
		DataOffset = (InOut.DCR.Num() > 8 && (InOut.DCR[0] == 0 || InOut.DCR[0] == 1) && (InOut.DCR[1] == 0 && InOut.DCR[2] == 0 && InOut.DCR[3] == 0)) ? 4 : 0;
		InOut.ExtraBoxes.Emplace(Make4CC('v','p','c','C'), InOut.DCR);
		vpcC_Box = InOut.ExtraBoxes.Find(Make4CC('v','p','c','C'));
		bFromDCR = true;
	}
	if (vpcC_Box && vpcC_Box->Num() > 4)
	{
		if (InOut.MimeType.IsEmpty())
		{
			InOut.MimeType = InOut.FourCC == Make4CC('v','p','0','8') ?	TEXT("video/VP8") : TEXT("video/VP9");
		}
		ElectraDecodersUtil::VPxVideo::FVPCodecConfigurationRecord dcr;
		// If provided in the `vpcC` box, which is a full box, we need to skip the first 4 bytes.
		auto BoxData = MakeConstArrayView<const uint8>(vpcC_Box->GetData() + DataOffset, vpcC_Box->Num() - DataOffset);
		if (dcr.Parse(InOut.FourCC == Make4CC('v','p','0','8') ? 8 : 9, BoxData))
		{
			InOut.DCR = BoxData;
			InOut.Type = Electra::FCodecTypeFormat::EType::Video;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
			}
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			if ((vid.Profile.Profile && vid.Profile.Profile != dcr.GetProfile()) ||
				(vid.Profile.Level && vid.Profile.Level != dcr.GetLevel()))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "VPx profile and level was set as %d,%d but specified as %d,%d in VPCodecConfigurationRecord", vid.Profile.Profile, vid.Profile.Level, dcr.GetProfile(), dcr.GetLevel());
			}
			vid.Profile.Profile = dcr.GetProfile();
			vid.Profile.Level = dcr.GetLevel();
			InOut.CSD = dcr.GetCodecSpecificData();

			vid.BitDepth = dcr.GetBitDepth();
			Electra::FCodecTypeFormat::FVideo::FColorInfo ci;
			const ElectraDecodersUtil::VPxVideo::FVPCodecConfigurationRecord::FColorInfo& vpxCol = dcr.GetColorInfo();
			ci.chromaSubsampling = vpxCol.chromaSubsampling;
			ci.colourPrimaries = vpxCol.colourPrimaries;
			ci.transferCharacteristics = vpxCol.transferCharacteristics;
			ci.matrixCoefficients = vpxCol.matrixCoefficients;
			ci.videoFullRangeFlag = vpxCol.videoFullRangeFlag;
			vid.OptColorInfo = ci;

			FString rfc(FString::Printf(TEXT("vp0%d.%02d.%02d.%02d.%02d.%02d.%02d.%02d.%02d"), InOut.FourCC == Make4CC('v','p','0','8') ? 8 : 9,
							vid.Profile.Profile, vid.Profile.Level, vid.BitDepth,ci.chromaSubsampling, ci.colourPrimaries, ci.transferCharacteristics, ci.matrixCoefficients, ci.videoFullRangeFlag));
			if (rfc.EndsWith(TEXT(".01.01.01.01.00")))
			{
				rfc.LeftChopInline(15);
			}
			InOut.RFC6381 = MoveTemp(rfc);
			InOut.HumanReadableFormatInfo = FString::Printf(TEXT("VP%d, Profile %u, level %u.%u"), InOut.FourCC == Make4CC('v','p','0','8') ? 8 : 9, vid.Profile.Profile, vid.Profile.Level / 10, vid.Profile.Level % 10);
			return 1;
		}
		return bFromDCR ? 0 : -1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_av1c(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* av1C_Box = InOut.ExtraBoxes.Find(Make4CC('a','v','1','C'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	bool bFromDCR = false;
	if (!av1C_Box && InOut.DCR.Num() && InOut.FourCC == Make4CC('a','v','0','1'))
	{
		InOut.ExtraBoxes.Emplace(Make4CC('a','v','1','C'), InOut.DCR);
		av1C_Box = InOut.ExtraBoxes.Find(Make4CC('a','v','1','C'));
		bFromDCR = true;
	}
	if (av1C_Box && av1C_Box->Num() > 4)
	{
		if (InOut.MimeType.IsEmpty())
		{
			InOut.MimeType = TEXT("video/AV1");
		}
		ElectraDecodersUtil::AV1Video::FAV1CodecConfigurationRecord dcr;
		if (dcr.Parse(*av1C_Box))
		{
			InOut.DCR = *av1C_Box;
			InOut.Type = Electra::FCodecTypeFormat::EType::Video;
			if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
			{
				InOut.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
			}
			Electra::FCodecTypeFormat::FVideo& vid(InOut.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
			if ((vid.Profile.Profile && vid.Profile.Profile != dcr.GetProfile()) ||
				(vid.Profile.Level && vid.Profile.Level != dcr.GetLevel()))
			{
				UE_LOGF(LogElectraFormatUtils, Log, "AV1 profile and level was set as %d,%d but specified as %d,%d in AV1CodecConfigurationRecord", vid.Profile.Profile, vid.Profile.Level, dcr.GetProfile(), dcr.GetLevel());
			}
			vid.Profile.Profile = dcr.GetProfile();
			vid.Profile.Level = dcr.GetLevel();
			InOut.CSD = dcr.GetCodecSpecificData();
			if (!InOut.CSD.IsEmpty())
			{
				const uint8* CurrentOBU = InOut.CSD.GetData();
				const uint8* EndOfOBUs = CurrentOBU + InOut.CSD.Num();
				while(CurrentOBU < EndOfOBUs)
				{
					const uint8 OBUheaderByte = *CurrentOBU;
					if ((OBUheaderByte & 0x80) != 0)
					{
						UE_LOGF(LogElectraFormatUtils, Log, "obu_forbidden_bit not zero");
						return -1;
					}
					const ElectraDecodersUtil::AV1Video::EOBUType OBUtype = (const ElectraDecodersUtil::AV1Video::EOBUType)(OBUheaderByte >> 3);
					const uint8 obu_extension_flag = (OBUheaderByte >> 2) & 1;
					const uint8 obu_has_size_field = (OBUheaderByte >> 1) & 1;
					const uint8 obu_reserved_1bit = OBUheaderByte & 1;
					CurrentOBU += obu_extension_flag ? 2 : 1;
					uint32 obu_size;
					uint32 nr = 0;
					if (obu_has_size_field)
					{
						ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, EndOfOBUs - CurrentOBU);
						obu_size = bs.leb128(nr);
						CurrentOBU += nr;
					}
					else
					{
						obu_size = EndOfOBUs - CurrentOBU;
					}

					if (OBUtype == ElectraDecodersUtil::AV1Video::EOBUType::OBU_SEQUENCE_HEADER)
					{
						ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, obu_size);
						ElectraDecodersUtil::AV1Video::FSequenceHeader sh;
						if (!sh.ParseFrom(bs))
						{
							UE_LOGF(LogElectraFormatUtils, Log, "Failed to parse sequence_header_obu");
							return -1;
						}

						// Check dimensions
						uint32 w = 1 + sh.max_frame_width_minus_1;
						uint32 h = 1 + sh.max_frame_height_minus_1;
						if ((vid.Width && vid.Width != w) ||
							(vid.Height && vid.Height != h))
						{
							UE_LOGF(LogElectraFormatUtils, Log, "Dimensions were set as %u*%u but given as %d*%d by the sequence header", vid.Width, vid.Height, w, h);
							vid.Width = w ? w : vid.Width;
							vid.Height = h ? h : vid.Height;
						}
						else if ((!vid.Width && w) || (!vid.Height && h))
						{
							vid.Width = w;
							vid.Height = h;
						}
					}
#if 0
					// HDR metadata?
					else if (OBUtype == ElectraDecodersUtil::AV1Video::EOBUType::OBU_METADATA)
					{
						ElectraDecodersUtil::AV1Video::FBitstreamReaderAV1 bs(CurrentOBU, obu_size);
						const ElectraDecodersUtil::AV1Video::EMetadataType MetadataType = (const ElectraDecodersUtil::AV1Video::EMetadataType)(bs.leb128(nr));
						switch(MetadataType)
						{
							case ElectraDecodersUtil::AV1Video::EMetadataType::METADATA_TYPE_HDR_CLL:
							{
								ElectraDecodersUtil::AV1Video::FMetadata_hdr_cll cll;
								if (cll.ParseFrom(bs))
								{
									// create clli box if not already provided
								}
								break;
							}
							case ElectraDecodersUtil::AV1Video::EMetadataType::METADATA_TYPE_HDR_MDCV:
							{
								ElectraDecodersUtil::AV1Video::FMetadata_hdr_mdvc mdcv;
								if (mdcv.ParseFrom(bs))
								{
									// create mdcv box if not already provided
								}
								break;
							}
						}
					}
#endif
					CurrentOBU += obu_size;
				}
			}

			vid.BitDepth = dcr.GetBitDepth();
			Electra::FCodecTypeFormat::FVideo::FColorInfo ci;
			const ElectraDecodersUtil::AV1Video::FAV1CodecConfigurationRecord::FColorInfo& av1Col = dcr.GetColorInfo();
			ci.colourPrimaries = av1Col.colorPrimaries;
			ci.transferCharacteristics = av1Col.transferCharacteristics;
			ci.matrixCoefficients = av1Col.matrixCoefficients;
			ci.videoFullRangeFlag = av1Col.videoFullRangeFlag;
			Electra::FCodecTypeFormat::FVideo::FColorInfo::FChromaInfo cri;
			cri.monochrome = av1Col.monochrome;
			cri.chromaSubsamplingX = av1Col.chromaSubsamplingX;
			cri.chromaSubsamplingY = av1Col.chromaSubsamplingY;
			cri.chromaSamplingPosition = av1Col.chromaSamplingPosition;
			ci.ChromaInfo = cri;
			vid.OptColorInfo = ci;
			InOut.RFC6381 = dcr.GetCodecSpecifierRFC6381();
			InOut.HumanReadableFormatInfo = dcr.GetFormatInfo();
			return 1;
		}
		return bFromDCR ? 0 : -1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_mp4a(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* esds_Box = InOut.ExtraBoxes.Find(Make4CC('e','s','d','s'));
	// With QuickTime files the `esds` box might be stored inside a `wave` box.
	if (!esds_Box)
	{
		if (const TArray<uint8>* wave_Box = InOut.ExtraBoxes.Find(Make4CC('w','a','v','e')))
		{
			TArray<TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>> ChildBoxes;
			if (MP4Utilities::FMP4BoxLocator::LocateAndReadBoxesFromBuffer(ChildBoxes, *wave_Box))
			{
				if (auto ESDS = ChildBoxes.FindByPredicate([](const TSharedPtr<MP4Utilities::FMP4BoxData, ESPMode::ThreadSafe>& e) { return e->Type == MP4Utilities::MakeBoxAtom('e','s','d','s');} ))
				{
					InOut.ExtraBoxes.Emplace(Make4CC('e','s','d','s'), (*ESDS)->DataBuffer);
					esds_Box = InOut.ExtraBoxes.Find(Make4CC('e','s','d','s'));
				}
			}
		}
	}
	// Check if in absence of the codec format box a codec specific DCR is provided.
	bool bFromDCR = false;
	if (!esds_Box && InOut.DCR.Num() && InOut.FourCC == Make4CC('m','p','4','a'))
	{
		InOut.ExtraBoxes.Emplace(Make4CC('e','s','d','s'), InOut.DCR);
		esds_Box = InOut.ExtraBoxes.Find(Make4CC('e','s','d','s'));
		bFromDCR = true;
	}

	if (esds_Box && esds_Box->Num() > 4)
	{
		ElectraDecodersUtil::MPEG::FESDescriptor esds;
		// The ESDescriptorBox is a full box, so we need to skip the first 4 bytes.
		auto BoxData = MakeConstArrayView<const uint8>(esds_Box->GetData() + 4, esds_Box->Num() - 4);
		if (!esds.Parse(BoxData))
		{
			return bFromDCR ? 0 : -1;
		}
		if (esds.GetStreamType() != ElectraDecodersUtil::MPEG::FESDescriptor::FStreamType::AudioStream)
		{
			return 0;
		}
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
		}
		Electra::FCodecTypeFormat::FAudio& aud(InOut.Properties.Get<Electra::FCodecTypeFormat::FAudio>());

		if (!aud.ObjectType.IsType<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>())
		{
			aud.ObjectType.Emplace<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>();
		}
		Electra::FCodecTypeFormat::FAudio::FMPEGObjectType& OT(aud.ObjectType.Get<Electra::FCodecTypeFormat::FAudio::FMPEGObjectType>());

		uint32 FourCC = 0;
		OT.ObjectType = (uint32) esds.GetObjectTypeID();
		switch(esds.GetObjectTypeID())
		{
			// ISO/IEC 14496-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG4_Audio:
			{
				ElectraDecodersUtil::MPEG::FAACDecoderConfigurationRecord dcr;
				if (!(esds.GetCodecSpecificData().Num() && dcr.Parse(esds.GetCodecSpecificData())))
				{
					UE_LOGF(LogElectraFormatUtils, Log, "ESDescriptor lacks mandatory AudioSpecificConfig for ISO/IEC 14496-3 streams");
					return -1;
				}
				InOut.DCR = BoxData;
				InOut.CSD = esds.GetCodecSpecificData();
				InOut.RFC6381 = dcr.GetCodecSpecifierRFC6381();

				uint32 NumChan = (uint32) ElectraDecodersUtil::MPEG::AACUtils::GetNumberOfChannelsFromChannelConfiguration(dcr.ChannelConfiguration);
				uint32 SmpRate = dcr.ExtSamplingFrequency ? dcr.ExtSamplingFrequency : dcr.SamplingRate;
				aud.ChannelConfiguration = dcr.ChannelConfiguration;
				if ((aud.NumChannels == 0) || (aud.NumChannels && NumChan && aud.NumChannels != NumChan))
				{
					aud.NumChannels = NumChan;
				}
				if ((aud.SampleRate == 0) || (aud.SampleRate && SmpRate && aud.SampleRate != SmpRate))
				{
					aud.SampleRate = SmpRate;
				}
				OT.AudioObjectType = dcr.ExtAOT ? dcr.ExtAOT : dcr.AOT;
				switch(OT.AudioObjectType)
				{
					case 2:		// AAC-LC
					case 5:		// SBR
					case 29:	// PS
					{
						FourCC = Make4CC('m','p','4','a');
						if (InOut.MimeType.IsEmpty())
						{
							InOut.MimeType = TEXT("audio/mp4");
						}
						break;
					}
					case 32:	// Layer-1
					case 33:	// Layer-2
					case 34:	// Layer-3
					{
						FourCC = Make4CC('m','p','g','a');
						if (InOut.MimeType.IsEmpty())
						{
							InOut.MimeType = TEXT("audio/mpeg");
						}
						OT.Layer = OT.AudioObjectType - 31;
						OT.MPEG = 1;
						break;
					}
					default:
					{
						// not supported
						return -1;
					}
				}
				break;
			}
			// ISO/IEC 13818-7
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG2_AAC_LC:
			{
				FourCC = Make4CC('m','p','4','a');
				OT.AudioObjectType = 2;
				if (InOut.MimeType.IsEmpty())
				{
					InOut.MimeType = TEXT("audio/mp4a");
				}
				InOut.RFC6381 = FString::Printf(TEXT("mp4a.%02X"), (uint32)esds.GetObjectTypeID());
				break;
			}
			// ISO/IEC 13818-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG2_Audio:
			// ISO/IEC 11172-3
			case ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG1_Audio:
			{
				OT.MPEG = esds.GetObjectTypeID() == ElectraDecodersUtil::MPEG::FESDescriptor::FObjectTypeID::MPEG1_Audio ? 1 : 2;
				FourCC = Make4CC('m','p','g','a');
				if (InOut.MimeType.IsEmpty())
				{
					InOut.MimeType = TEXT("audio/mpeg");
				}
				InOut.RFC6381 = FString::Printf(TEXT("mp4a.%02X"), (uint32)esds.GetObjectTypeID());
				break;
			}
			default:
			{
				return -1;
			}
		}

		InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
		if (InOut.FourCC == 0)
		{
			InOut.FourCC = FourCC;
		}
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x from FESDescriptor", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		MakeHumanReadableRFC_MP4A(InOut, OT.AudioObjectType, OT.MPEG, OT.Layer);
		return 1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_Opus(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* dOps_Box = InOut.ExtraBoxes.Find(Make4CC('d','O','p','s'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	if (!dOps_Box && InOut.DCR.Num() && InOut.FourCC == Make4CC('O','p','u','s'))
	{
		// Check if the DCR _could_ be an OpusSpecificBox
		if ((InOut.DCR.Num() == 11 && InOut.DCR[0] == 0 && InOut.DCR[10] == 0) ||
			(InOut.DCR.Num() > 11 && InOut.DCR[0] == 0 && InOut.DCR.Num() == 11+InOut.DCR[10]))
		{
			InOut.ExtraBoxes.Emplace(Make4CC('d','O','p','s'), InOut.DCR);
			dOps_Box = InOut.ExtraBoxes.Find(Make4CC('d','O','p','s'));
		}
	}
	if (dOps_Box)
	{
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
		}
		InOut.DCR = *dOps_Box;
		InOut.RFC6381 = TEXT("opus");
		InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
		uint32 FourCC = Make4CC('O','p','u','s');
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		return 1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_Flac(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* dfLa_Box = InOut.ExtraBoxes.Find(Make4CC('d','f','L','a'));
	int32 DataOffset = 4;
	// Check if in absence of the codec format box a codec specific DCR is provided.
	if (!dfLa_Box && InOut.DCR.Num() > 4 && InOut.FourCC == Make4CC('f','L','a','C'))
	{
		// Check if the DCR which should be a FLACSpecificBox is accidentally wrapped in an mp4 full box.
		DataOffset = (InOut.DCR.Num() > 4 && InOut.DCR[0] == 0  && (InOut.DCR[1] == 0 && InOut.DCR[2] == 0 && InOut.DCR[3] == 0)) ? 4 : 0;
		// Try to validate the DCR as valid Flac metadata
		auto BoxData = MakeConstArrayView<const uint8>(InOut.DCR.GetData() + DataOffset, InOut.DCR.Num() - DataOffset);
		int32 LenToGo = BoxData.Num();
		int32 Off = 0;
		while(LenToGo > 0)
		{
			uint8 BlockType = BoxData[Off] & 0x7f;
			bool bIsLast = (BoxData[Off] & 0x80) != 0;
			int32 Len = (int32)((((uint32)BoxData[Off+1] << 16) | ((uint32)BoxData[Off+2] << 8) | (uint32)BoxData[Off+3]) + 4);
			if (LenToGo - Len < 0)
			{
				return 0;
			}
			LenToGo -= Len;
			Off += Len;
		}
		InOut.ExtraBoxes.Emplace(Make4CC('d','f','L','a'), InOut.DCR);
		dfLa_Box = InOut.ExtraBoxes.Find(Make4CC('d','f','L','a'));
	}
	if (dfLa_Box)
	{
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
		}
		// If provided in the `dfLa` box, which is a full box, we need to skip the first 4 bytes.
		auto BoxData = MakeConstArrayView<const uint8>(dfLa_Box->GetData() + DataOffset, dfLa_Box->Num() - DataOffset);
		InOut.DCR = BoxData;
		InOut.RFC6381 = TEXT("flac");
		InOut.Type = Electra::FCodecTypeFormat::EType::Audio;
		uint32 FourCC = Make4CC('f','L','a','C');
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		return 1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_tx3g(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* tx3g_Box = InOut.ExtraBoxes.Find(Make4CC('t','x','3','g'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	if (!tx3g_Box && InOut.DCR.Num() && InOut.FourCC == Make4CC('t','x','3','g'))
	{
		// We cannot validate if the DCR is valid for tx3g. We have to assume that it is
		// based on the fact that the codec 4cc says it is so.
		InOut.ExtraBoxes.Emplace(Make4CC('t','x','3','g'), InOut.DCR);
		tx3g_Box = InOut.ExtraBoxes.Find(Make4CC('t','x','3','g'));
	}
	if (tx3g_Box)
	{
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
		}
		InOut.DCR = *tx3g_Box;
		InOut.CSD = *tx3g_Box;
		InOut.RFC6381 = TEXT("tx3g");
		InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
		uint32 FourCC = Make4CC('t','x','3','g');
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		return 1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_wvtt(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* wvtt_Box = InOut.ExtraBoxes.Find(Make4CC('w','v','t','t'));
	// Check if in absence of the codec format box a codec specific DCR is provided.
	if (!wvtt_Box && InOut.DCR.Num() && InOut.FourCC == Make4CC('w','v','t','t'))
	{
		// We cannot validate if the DCR is valid for WebVTT. We have to assume that it is
		// based on the fact that the codec 4cc says it is so.
		InOut.ExtraBoxes.Emplace(Make4CC('w','v','t','t'), InOut.DCR);
		wvtt_Box = InOut.ExtraBoxes.Find(Make4CC('w','v','t','t'));
	}
	if (wvtt_Box)
	{
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
		}
		InOut.DCR = *wvtt_Box;
		InOut.CSD = *wvtt_Box;
		InOut.RFC6381 = TEXT("wvtt");
		InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
		uint32 FourCC = Make4CC('w','v','t','t');
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		return 1;
	}
	return 0;
}

static int32 DetermineCodecTypeFromDCR_stpp(Electra::FCodecTypeFormat& InOut)
{
	const TArray<uint8>* stpp_Box = InOut.ExtraBoxes.Find(Make4CC('s','t','p','p'));
	if (stpp_Box)
	{
		if (!InOut.Properties.IsType<Electra::FCodecTypeFormat::FSubtitle>())
		{
			InOut.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
		}
		// TTML does not have codec specific data
		//InOut.DCR = *stpp_Box;
		//InOut.CSD = *stpp_Box;
		InOut.RFC6381 = TEXT("stpp.ttml.im1t");
		InOut.Type = Electra::FCodecTypeFormat::EType::Subtitle;
		uint32 FourCC = Make4CC('s','t','p','p');
		if (InOut.FourCC && InOut.FourCC != FourCC)
		{
			UE_LOGF(LogElectraFormatUtils, Log, "FourCC was set as %08x but identifed as %08x", InOut.FourCC, FourCC);
		}
		InOut.FourCC = FourCC;
		return 1;
	}
	return 0;
}



static bool DetermineCodecTypeFromDCR(Electra::FCodecTypeFormat& InOut)
{
	int32 Result = 0;

	// AVC (H.264)
	Result = DetermineCodecTypeFromDCR_avcc(InOut);
	if (Result) return Result < 0 ? false : true;
	// HEVC (H.265)
	Result = DetermineCodecTypeFromDCR_hvcc(InOut);
	if (Result) return Result < 0 ? false : true;
	// VP8 / VP9
	Result = DetermineCodecTypeFromDCR_vpcc(InOut);
	if (Result) return Result < 0 ? false : true;
	// AV1
	Result = DetermineCodecTypeFromDCR_av1c(InOut);
	if (Result) return Result < 0 ? false : true;

	// MP4A
	Result = DetermineCodecTypeFromDCR_mp4a(InOut);
	if (Result) return Result < 0 ? false : true;
	// Opus
	Result = DetermineCodecTypeFromDCR_Opus(InOut);
	if (Result) return /*Result < 0 ? false :*/ true;
	// Flac
	Result = DetermineCodecTypeFromDCR_Flac(InOut);
	if (Result) return /*Result < 0 ? false :*/ true;

	// TX3G
	Result = DetermineCodecTypeFromDCR_tx3g(InOut);
	if (Result) return /*Result < 0 ? false :*/ true;
	// WebVTT
	Result = DetermineCodecTypeFromDCR_wvtt(InOut);
	if (Result) return /*Result < 0 ? false :*/ true;
	// TTML
	Result = DetermineCodecTypeFromDCR_stpp(InOut);
	if (Result) return /*Result < 0 ? false :*/ true;

	return true;
}

/**
 * Checks for some additionally provided boxes and extracts metadata of interest.
 */
static bool ProcessExtraBoxes(Electra::FCodecTypeFormat& InOut)
{
	if (const TArray<uint8>* btrt_Box = InOut.ExtraBoxes.Find(Make4CC('b','t','r','t')))
	{
		auto Btrt = StaticCastSharedPtr<MP4Boxes::FMP4BoxBTRT>(MP4Boxes::FMP4BoxBTRT::Create(nullptr, { *btrt_Box }));
		InOut.Bitrate = Btrt->GetMaxBitrate() ? Btrt->GetMaxBitrate() : InOut.Bitrate;
		InOut.AverageBitrate = Btrt->GetAverageBitrate() ? Btrt->GetAverageBitrate() : InOut.AverageBitrate;
	}

	return true;
}

static bool DetermineKeyframeMode(Electra::FCodecTypeFormat& InOut)
{
	// Check if the keyframe mode was already set in the process.
	// If it was then this is already the definitive answer.
	// Otherwise we apply the most common defaults.
	if (InOut.KeyframeMode == Electra::FCodecTypeFormat::EKeyframeMode::Unknown)
	{
		// All audio and subtitles we handle are keyframe only. Timecode is keyframe as well.
		if (InOut.Type == Electra::FCodecTypeFormat::EType::Audio || InOut.Type == Electra::FCodecTypeFormat::EType::Subtitle || InOut.Type == Electra::FCodecTypeFormat::EType::Timecode)
		{
			InOut.KeyframeMode = Electra::FCodecTypeFormat::EKeyframeMode::OnlyKeyframes;
		}
		else if (InOut.Type == Electra::FCodecTypeFormat::EType::Video)
		{
			// For video we start out assuming everything is a keyframe. Of the codecs we know there are
			// several where this is not true so these need to be checked for.
			switch(InOut.FourCC)
			{
				case Make4CC('a','v','c','1'):
				case Make4CC('a','v','c','3'):
				case Make4CC('h','e','v','1'):
				case Make4CC('h','v','c','1'):
				case Make4CC('v','p','0','8'):
				case Make4CC('v','p','0','9'):
				case Make4CC('a','v','0','1'):
				{
					InOut.KeyframeMode = Electra::FCodecTypeFormat::EKeyframeMode::DeltaFrames;
					break;
				}
				default:
				{
					InOut.KeyframeMode = Electra::FCodecTypeFormat::EKeyframeMode::OnlyKeyframes;
					break;
				}
			}
		}
	}
	return true;
}

}


bool FElectraCodecFormatUtilsModularFeature::SetupCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat, const FCodecTypeFormatParams& InCodecTypeFormatParams)
{
	RemapLegacyFourCC(InOutCodecTypeFormat);
	SetDefaultMimeTypeAndRFC6381For4CC(InOutCodecTypeFormat);
	DetermineCodecTypeFromMimeType(InOutCodecTypeFormat);
	bool bOk = DetermineCodecTypeFromRFC(InOutCodecTypeFormat) && DetermineCodecTypeFromDCR(InOutCodecTypeFormat) && ProcessExtraBoxes(InOutCodecTypeFormat) && DetermineKeyframeMode(InOutCodecTypeFormat);
	return bOk;
}

bool FElectraCodecFormatUtilsModularFeature::SetupCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat)
{
	return SetupCodecTypeFormat(InOutCodecTypeFormat, FCodecTypeFormatParams());
}



bool FElectraCodecFormatUtilsModularFeature::PrepareCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat, const FCodecTypeFormatParams& InCodecTypeFormatParams)
{
	return SetupCodecTypeFormat(InOutCodecTypeFormat, InCodecTypeFormatParams);
}
