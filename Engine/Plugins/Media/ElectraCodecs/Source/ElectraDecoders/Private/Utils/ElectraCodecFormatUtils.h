// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IElectraCodecFormatUtils.h"

class FElectraCodecFormatUtilsModularFeature : public IElectraCodecFormatUtilsModularFeature
{
public:
	static void Startup();
	static void Shutdown();

	virtual ~FElectraCodecFormatUtilsModularFeature() = default;
	bool PrepareCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat, const FCodecTypeFormatParams& InCodecTypeFormatParams) override;

	ELECTRADECODERS_API	static bool SetupCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat, const FCodecTypeFormatParams& InCodecTypeFormatParams);
	ELECTRADECODERS_API	static bool SetupCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat);
private:
	static FElectraCodecFormatUtilsModularFeature* Self;
};
