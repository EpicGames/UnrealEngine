// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraDecodersUtils.h"
#include "Utils/ElectraCodecFormatUtils.h"

namespace ElectraDecodersUtil
{

bool PrepareCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat)
{
	return FElectraCodecFormatUtilsModularFeature::SetupCodecTypeFormat(InOutCodecTypeFormat);
}


int64 GetVariantValueSafeI64(const TMap<FString, FVariant>& InFromMap, const FString& InName, int64 InDefaultValue)
{
	int64 V = InDefaultValue;
	const FVariant* Var = InFromMap.Find(InName);
	if (Var)
	{
		if (Var->GetType() == EVariantTypes::Int32)
		{
			V = Var->GetValue<int32>();
		}
		else if (Var->GetType() == EVariantTypes::Int64)
		{
			V = Var->GetValue<int64>();
		}
		else if (Var->GetType() == EVariantTypes::UInt64)
		{
			V = (int64) Var->GetValue<uint64>();
		}
		else if (Var->GetType() == EVariantTypes::UInt32)
		{
			V = (int64) Var->GetValue<uint32>();
		}
		else if (Var->GetType() == EVariantTypes::Int16)
		{
			V = Var->GetValue<int16>();
		}
		else if (Var->GetType() == EVariantTypes::Int8)
		{
			V = Var->GetValue<int8>();
		}
		else if (Var->GetType() == EVariantTypes::UInt16)
		{
			V = Var->GetValue<uint16>();
		}
		else if (Var->GetType() == EVariantTypes::UInt8)
		{
			V = Var->GetValue<uint8>();
		}
		else if (Var->GetType() == EVariantTypes::Float)
		{
			V = (int64) Var->GetValue<float>();
		}
		else if (Var->GetType() == EVariantTypes::Double)
		{
			V = (int64) Var->GetValue<double>();
		}
		else if (Var->GetType() == EVariantTypes::Bool)
		{
			V = Var->GetValue<bool>() ? 1 : 0;
		}
	}
	return V;
}


uint64 GetVariantValueSafeU64(const TMap<FString, FVariant>& InFromMap, const FString& InName, uint64 InDefaultValue)
{
	uint64 V = InDefaultValue;
	const FVariant* Var = InFromMap.Find(InName);
	if (Var)
	{
		if (Var->GetType() == EVariantTypes::Int32)
		{
			V = Var->GetValue<int32>();
		}
		else if (Var->GetType() == EVariantTypes::Int64)
		{
			V = (uint64) Var->GetValue<int64>();
		}
		else if (Var->GetType() == EVariantTypes::UInt64)
		{
			V = Var->GetValue<uint64>();
		}
		else if (Var->GetType() == EVariantTypes::UInt32)
		{
			V = Var->GetValue<uint32>();
		}
		else if (Var->GetType() == EVariantTypes::Int16)
		{
			V = Var->GetValue<int16>();
		}
		else if (Var->GetType() == EVariantTypes::Int8)
		{
			V = Var->GetValue<int8>();
		}
		else if (Var->GetType() == EVariantTypes::UInt16)
		{
			V = Var->GetValue<uint16>();
		}
		else if (Var->GetType() == EVariantTypes::UInt8)
		{
			V = Var->GetValue<uint8>();
		}
		else if (Var->GetType() == EVariantTypes::Float)
		{
			V = (uint64) Var->GetValue<float>();
		}
		else if (Var->GetType() == EVariantTypes::Double)
		{
			V = (uint64) Var->GetValue<double>();
		}
		else if (Var->GetType() == EVariantTypes::Bool)
		{
			V = Var->GetValue<bool>() ? 1 : 0;
		}
	}
	return V;
}


double GetVariantValueSafeDouble(const TMap<FString, FVariant>& InFromMap, const FString& InName, double InDefaultValue)
{
	double V = InDefaultValue;
	const FVariant* Var = InFromMap.Find(InName);
	if (Var)
	{
		if (Var->GetType() == EVariantTypes::Int32)
		{
			V = (double) Var->GetValue<int32>();
		}
		else if (Var->GetType() == EVariantTypes::Int64)
		{
			V = (double) Var->GetValue<int64>();
		}
		else if (Var->GetType() == EVariantTypes::UInt32)
		{
			V = (double) Var->GetValue<uint32>();
		}
		else if (Var->GetType() == EVariantTypes::Int16)
		{
			V = (double) Var->GetValue<int16>();
		}
		else if (Var->GetType() == EVariantTypes::Int8)
		{
			V = (double) Var->GetValue<int8>();
		}
		else if (Var->GetType() == EVariantTypes::UInt16)
		{
			V = (double) Var->GetValue<uint16>();
		}
		else if (Var->GetType() == EVariantTypes::UInt8)
		{
			V = (double) Var->GetValue<uint8>();
		}
		else if (Var->GetType() == EVariantTypes::Float)
		{
			V = (double) Var->GetValue<float>();
		}
		else if (Var->GetType() == EVariantTypes::Double)
		{
			V = (double) Var->GetValue<double>();
		}
		else if (Var->GetType() == EVariantTypes::Bool)
		{
			V = Var->GetValue<bool>() ? 1.0 : 0.0;
		}
	}
	return V;
}

TArray<uint8> GetVariantValueUInt8Array(const TMap<FString, FVariant>& InFromMap, const FString& InName)
{
	const FVariant* Var = InFromMap.Find(InName);
	if (Var)
	{
		if (Var->GetType() == EVariantTypes::ByteArray)
		{
			return Var->GetValue<TArray<uint8>>();
		}
	}
	return TArray<uint8>();
}

FString GetVariantValueFString(const TMap<FString, FVariant>& InFromMap, const FString& InName)
{
	const FVariant* Var = InFromMap.Find(InName);
	if (Var)
	{
		if (Var->GetType() == EVariantTypes::String)
		{
			return Var->GetValue<FString>();
		}
	}
	return FString();
}

}
