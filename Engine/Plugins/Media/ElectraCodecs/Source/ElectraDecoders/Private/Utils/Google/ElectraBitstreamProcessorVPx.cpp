// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/Google/ElectraBitstreamProcessorVPx.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"	// because information may be carried in mp4 specific boxes
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"
#include "Misc/Optional.h"



class FElectraDecoderBitstreamProcessorVPx::FImpl
{
public:
	enum class EVersion
	{
		VP8,
		VP9
	};

	FImpl(EVersion InVersion)
		: Version(InVersion)
	{ }

	void Clear()
	{
		bSentColorimetry = false;
		bSentMDCV = false;
		bSentCLLI = false;
		LastVP9Header.Reset();
		// Do not clear out any error message!
	}

	FString GetLastError()
	{
		return LastErrorMessage;
	}

	bool ProcessInputForDecoding(bool& OutCSDChanged, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
	{
		OutCSDChanged = false;
		InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsDiscardable;
		if (Version == EVersion::VP8)
		{
			ElectraDecodersUtil::VPxVideo::FVP8UncompressedHeader Header;
			if (ElectraDecodersUtil::VPxVideo::ParseVP8UncompressedHeader(Header, InOutAccessUnit.Data, InOutAccessUnit.DataSize))
			{
				if (Header.IsKeyframe())
				{
					InOutAccessUnit.Flags |= EElectraDecoderFlags::IsSyncSample;
				}
				else
				{
					InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsSyncSample;
				}
				return true;
			}
			else
			{
				LastErrorMessage = FString::Printf(TEXT("Failed to parse VP8 header"));
			}
		}
		else if (Version == EVersion::VP9)
		{
			ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader Header;
			if (!ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Header, InOutAccessUnit.Data, InOutAccessUnit.DataSize))
			{
				LastErrorMessage = FString::Printf(TEXT("Failed to parse VP9 header"));
				return false;
			}

			if (Header.IsKeyframe())
			{
				InOutAccessUnit.Flags |= EElectraDecoderFlags::IsSyncSample;

				if (LastVP9Header.IsSet())
				{
					const ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader& prv(LastVP9Header.GetValue());
					OutCSDChanged = Header.frame_width != prv.frame_width ||
									Header.frame_height != prv.frame_height ||
									Header.render_width != prv.render_width ||
									Header.render_height != prv.render_height ||
									Header.profile_low_bit != prv.profile_low_bit ||
									Header.profile_high_bit != prv.profile_high_bit ||
									Header.ten_or_twelve_bit != prv.ten_or_twelve_bit ||
									Header.color_space != prv.color_space ||
									Header.color_range != prv.color_range ||
									Header.subsampling_x != prv.subsampling_x ||
									Header.subsampling_y != prv.subsampling_y;
				}
				LastVP9Header = Header;
			}
			else
			{
				InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsSyncSample;
			}
#if 0
			// Per-frame HDR info?
			TArray<uint8> ITUT35SidebandData = ElectraDecodersUtil::GetVariantValueUInt8Array(InAccessUnitSidebandData, TEXT("itu-t-35"));
			if (ITUT35SidebandData.Num())
			{
			}
#endif
			return true;
		}
		return false;
	}

	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties)
	{
		if (CurrentColorimetry.IsValid() && !bSentColorimetry)
		{
			bSentColorimetry = true;
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}

		if (CurrentMDCV.IsValid() && !bSentMDCV)
		{
			bSentMDCV = true;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentMDCV.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume, TArray<uint8>(Params));
		}

		if (CurrentCLLI.IsValid() && !bSentCLLI)
		{
			bSentCLLI = true;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentCLLI.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo, TArray<uint8>(Params));
		}
	}

	void SetColorimetryFromCOLRBox(const TArray<uint8>& InCOLRBox)
	{
		COLRBox = InCOLRBox;
		UpdateCOLR();
	}
	void SetMasteringDisplayColorVolumeFromMDCVBox(const TArray<uint8>& InMDCVBox)
	{
		MDCVBox = InMDCVBox;
		UpdateMDCV();
	}
	void SetContentLightLevelInfoFromCLLIBox(const TArray<uint8>& InCLLIBox)
	{
		CLLIBox = InCLLIBox;
		UpdateCLLI();
	}
	void SetContentLightLevelFromCOLLBox(const TArray<uint8>& InCOLLBox)
	{
		if (InCOLLBox.Num() > 4)
		{
			uint8 BoxVersion = InCOLLBox[0];
			if (BoxVersion == 0)
			{
				// 'clli' box is the same as a version 0 'COLL 'coll' box.
				CLLIBox = TArray<uint8>(InCOLLBox.GetData() + 4, InCOLLBox.Num() - 4);
			}
		}
		UpdateCLLI();
	}
private:
	FImpl() = delete;

	void UpdateCOLR()
	{
		if (COLRBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> NewColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromCOLRBox(*NewColorimetry, COLRBox))
		{
			CurrentColorimetry = MoveTemp(NewColorimetry);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `colr` box data");
		}
	}

	void UpdateMDCV()
	{
		if (MDCVBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> NewMDCV = MakeShared<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromMDCVBox(*NewMDCV, MDCVBox))
		{
			CurrentMDCV = MoveTemp(NewMDCV);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `mdcv` box data");
		}
	}

	void UpdateCLLI()
	{
		if (CLLIBox.IsEmpty())
		{
			return;
		}
		TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> NewCLLI = MakeShared<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe>();
		if (ElectraDecodersUtil::MPEG::ParseFromCLLIBox(*NewCLLI, CLLIBox))
		{
			CurrentCLLI = MoveTemp(NewCLLI);
		}
		else
		{
			LastErrorMessage = TEXT("Failed to parse `coll`/`clli` box data");
		}
	}

	EVersion Version = EVersion::VP8;
	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> CurrentMDCV;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> CurrentCLLI;
	TArray<uint8> COLRBox;
	TArray<uint8> MDCVBox;
	TArray<uint8> CLLIBox;
	TOptional<ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader> LastVP9Header;
	FString LastErrorMessage;
	bool bSentColorimetry = false;
	bool bSentMDCV = false;
	bool bSentCLLI = false;
};


FElectraDecoderBitstreamProcessorVPx::~FElectraDecoderBitstreamProcessorVPx()
{
}

FElectraDecoderBitstreamProcessorVPx::FElectraDecoderBitstreamProcessorVPx(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	uint32 Codec4CC = InInitialCodecFormat.FourCC;
	check(Codec4CC == ElectraDecodersUtil::Make4CC('v','p','0','8') || Codec4CC == ElectraDecodersUtil::Make4CC('v','p','0','9'));

	Impl = MakePimpl<FImpl>(Codec4CC == ElectraDecodersUtil::Make4CC('v','p','0','8') ? FImpl::EVersion::VP8 : FImpl::EVersion::VP9);
	const TArray<uint8>* colr_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','o','l','r'));
	const TArray<uint8>* mdcv_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('m','d','c','v'));
	const TArray<uint8>* coll_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','o','l','l'));
	const TArray<uint8>* clli_Box = InInitialCodecFormat.ExtraBoxes.Find(ElectraDecodersUtil::Make4CC('c','l','l','i'));

	if (colr_Box)
	{
		Impl->SetColorimetryFromCOLRBox(*colr_Box);
	}
	if (mdcv_Box)
	{
		Impl->SetMasteringDisplayColorVolumeFromMDCVBox(*mdcv_Box);
	}
	if (coll_Box)
	{
		Impl->SetContentLightLevelFromCOLLBox(*coll_Box);
	}
	if (clli_Box)
	{
		Impl->SetContentLightLevelInfoFromCLLIBox(*clli_Box);
	}
}

bool FElectraDecoderBitstreamProcessorVPx::WillModifyBitstreamInPlace()
{
	return false;
}

void FElectraDecoderBitstreamProcessorVPx::Clear()
{
	// This only clears the flags that we did not send the color parameters yet.
	// The actual parameters are left unchanged as they are one-time init on construction only.
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorVPx::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	if ((InOutAccessUnit.Flags & EElectraDecoderFlags::InputIsProcessed) == EElectraDecoderFlags::InputIsProcessed)
	{
		return EProcessResult::Ok;
	}

	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;
	bool bCSDChanged = false;
	if (!Impl->ProcessInputForDecoding(bCSDChanged, InOutAccessUnit, InAccessUnitSidebandData))
	{
		return EProcessResult::Error;
	}

	return bCSDChanged ? EProcessResult::CSDChanged : EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorVPx::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	Impl->SetPropertiesOnOutput(InOutProperties);
}

FString FElectraDecoderBitstreamProcessorVPx::GetLastError()
{
	return Impl->GetLastError();
}
