// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utilities/ElectraBitstream.h"


namespace ElectraDecodersUtil
{
	namespace VPxVideo
	{
		FString FVPCodecConfigurationRecord::GetFormatInfo(uint32 InProfile, uint32 InLevel)
		{
			return FString::Printf(TEXT("Profile %u, level %d.%d"), InProfile, InLevel / 10, InLevel % 10);
		}

		FString FVPCodecConfigurationRecord::GetFormatInfo()
		{
			return FString::Printf(TEXT("VP%d %s"), VPx, *GetFormatInfo(Profile, Level));
		}


		bool FVPCodecConfigurationRecord::Parse(int32 InVP8Or9, const TConstArrayView<uint8>& InDCR)
		{
			check(InVP8Or9 == 8 || InVP8Or9 == 9);
			VPx = InVP8Or9;
			CSD.Empty();
			RawData.Empty();
			if (InDCR.Num() < 8)
			{
				return false;
			}
			RawData = InDCR;
			Electra::FElectraByteReader ByteReader(InDCR.GetData(), InDCR.Num());

			ByteReader.ReadByte(Profile);
			if (Profile > 3)
			{
				return false;
			}
			ByteReader.ReadByte(Level);
			if (Level != 0 && Level != 10 && Level != 11 && Level != 20 && Level != 21 && Level != 30 && Level != 31 &&
				Level != 40 && Level != 41 && Level != 50 && Level != 51 && Level != 52 && Level != 60 && Level != 61 && Level != 62)
			{
				return false;
			}
			uint8 b;
			ByteReader.ReadByte(b);
			BitDepth = b >> 4;
			if (BitDepth != 8 && BitDepth != 10 && BitDepth != 12)
			{
				return false;
			}
			ColorInfo.chromaSubsampling = (b >> 1) & 7U;
			if (ColorInfo.chromaSubsampling > 3)
			{
				return false;
			}
			ColorInfo.videoFullRangeFlag = b & 1U;
			ByteReader.ReadByte(ColorInfo.colourPrimaries);
			ByteReader.ReadByte(ColorInfo.transferCharacteristics);
			ByteReader.ReadByte(ColorInfo.matrixCoefficients);

			uint16 csdSize;
			ByteReader.ReadByte(csdSize);
			// CSD?
			if (csdSize && ByteReader.BytesRemaining() >= csdSize)
			{
				CSD.Reserve(csdSize);
				CSD.SetNumUninitialized(csdSize);
				ByteReader.ReadBytes(CSD.GetData(), csdSize);
			}
			return true;
		}





		bool GetVP9SuperframeSizes(TArray<uint32>& OutSizes, const void* Data, int64 Size)
		{
			if (!Data || !Size)
			{
				return false;
			}
			const uint8* Beg = reinterpret_cast<const uint8*>(Data);
			const uint8* End = Beg + Size - 1;

			// Last byte is the superframe marker, if this is a superframe. The marker byte is an invalid coded last byte
			// so it will never appear in a regular frame.
			if ((*End & 0xe0) != 0xc0)
			{
				// Not a superframe
				OutSizes.Emplace(Size);
			}
			else
			{
				// Superframe
				uint32 BytesPerFrameSize = ((*End >> 3) & 3) + 1;
				uint32 NumFramesInIndex = (*End & 7) + 1;
				uint32 SizeOfIndex = 2 + NumFramesInIndex * BytesPerFrameSize;
				const uint8* SuperframeIndex = End - SizeOfIndex + 1;
				if (*SuperframeIndex != *End)
				{
					// Bitstream error. These two bytes must be identical!
					return false;
				}
				for(uint32 i=0; i<NumFramesInIndex; ++i)
				{
					uint32& FrameSize = OutSizes.Emplace_GetRef(0);
					for(uint32 j=0; j<BytesPerFrameSize; ++j)
					{
						FrameSize |= (uint32)*(++SuperframeIndex) << (j*8);
					}
				}
			}
			return true;
		}

		bool ParseVP9UncompressedHeader(FVP9UncompressedHeader& OutHeader, const void* Data, int64 Size)
		{
			auto ReadColorConfig = [](FVP9UncompressedHeader& OutHeader, Electra::FBitstreamReader& BitReader) -> bool
			{
				if (OutHeader.GetProfile() >= 2)
				{
					OutHeader.ten_or_twelve_bit = BitReader.GetBits(1);
				}
				OutHeader.color_space = BitReader.GetBits(3);
				if (OutHeader.GetColorSpace() != FVP9UncompressedHeader::EColorSpace::CS_RGB)
				{
					OutHeader.color_range = BitReader.GetBits(1);
					if (OutHeader.GetProfile() == 1 || OutHeader.GetProfile() == 3)
					{
						OutHeader.subsampling_x = BitReader.GetBits(1);
						OutHeader.subsampling_y = BitReader.GetBits(1);
						if (BitReader.GetBits(1) != 0)
						{
							return false;
						}
					}
					else
					{
						OutHeader.subsampling_x = OutHeader.subsampling_y = 1;
					}
				}
				else
				{
					OutHeader.color_range = 1;
					if (OutHeader.GetProfile() == 1 || OutHeader.GetProfile() == 3)
					{
						OutHeader.subsampling_x = OutHeader.subsampling_y = 0;
						if (BitReader.GetBits(1) != 0)
						{
							return false;
						}
					}
				}
				return true;
			};

			auto ReadFrameSize = [](FVP9UncompressedHeader& OutHeader, Electra::FBitstreamReader& BitReader) -> bool
			{
				OutHeader.frame_width = BitReader.GetBits(16) + 1;
				OutHeader.frame_height = BitReader.GetBits(16) + 1;
				return true;
			};

			auto ReadRenderSize = [](FVP9UncompressedHeader& OutHeader, Electra::FBitstreamReader& BitReader) -> bool
			{
				if (BitReader.GetBits(1) /*render_and_frame_size_different*/)
				{
					OutHeader.render_width = BitReader.GetBits(16) + 1;
					OutHeader.render_height = BitReader.GetBits(16) + 1;
				}
				else
				{
					OutHeader.render_width = OutHeader.frame_width;
					OutHeader.render_height = OutHeader.frame_height;
				}
				return true;
			};

			if (!Data || !Size)
			{
				return false;
			}
			Electra::FBitstreamReader BitReader(Data, (uint64) Size);
			if ((OutHeader.frame_marker = BitReader.GetBits(2)) != 2)
			{
				return false;
			}
			OutHeader.profile_low_bit = BitReader.GetBits(1);
			OutHeader.profile_high_bit = BitReader.GetBits(1);
			if (OutHeader.GetProfile() == 3)
			{
				if (BitReader.GetBits(1) != 0)
				{
					return false;
				}
			}
			if ((OutHeader.show_existing_frame = BitReader.GetBits(1)) == 1)
			{
				OutHeader.frame_to_show_map_idx = BitReader.GetBits(3);
				return true;
			}

			OutHeader.frame_type = BitReader.GetBits(1);
			OutHeader.show_frame = BitReader.GetBits(1);
			OutHeader.error_resilient_mode = BitReader.GetBits(1);
			// Keyframe?
			if (OutHeader.frame_type == 0)
			{
				// Yes.
				if (BitReader.GetBits(8) != 0x49 || BitReader.GetBits(8) != 0x83 || BitReader.GetBits(8) != 0x42)
				{
					return false;
				}
				// Color config.
				if (!ReadColorConfig(OutHeader, BitReader))
				{
					return false;
				}
				// Frame size
				if (!ReadFrameSize(OutHeader, BitReader))
				{
					return false;
				}
				// Render size
				if (!ReadRenderSize(OutHeader, BitReader))
				{
					return false;
				}
			}
			else
			{
				// Not a keyframe.
				OutHeader.intra_only = OutHeader.show_frame == 0 ? BitReader.GetBits(1) : 0;
				OutHeader.reset_frame_context = OutHeader.error_resilient_mode == 0 ? BitReader.GetBits(2) : 0;
				if (OutHeader.intra_only == 1)
				{
					if (BitReader.GetBits(8) != 0x49 || BitReader.GetBits(8) != 0x83 || BitReader.GetBits(8) != 0x42)
					{
						return false;
					}
					if (OutHeader.GetProfile() > 0)
					{
						// Color config.
						if (!ReadColorConfig(OutHeader, BitReader))
						{
							return false;
						}
					}
					else
					{
						OutHeader.color_space = 1;
						OutHeader.subsampling_x = OutHeader.subsampling_y = 1;
					}
					uint8 refresh_frame_flags = BitReader.GetBits(8);
					// Frame size
					if (!ReadFrameSize(OutHeader, BitReader))
					{
						return false;
					}
					// Render size
					if (!ReadRenderSize(OutHeader, BitReader))
					{
						return false;
					}
				}
			}
			return true;
		}



		bool ParseVP8UncompressedHeader(FVP8UncompressedHeader& OutHeader, const void* Data, int64 Size)
		{
			if (!Data || !Size || Size < 3)
			{
				return false;
			}

			const uint8* rb = reinterpret_cast<const uint8*>(Data);
			OutHeader.key_frame = rb[0] & 1;
			OutHeader.version = (rb[0] >> 1) & 3;
			OutHeader.is_experimental = (rb[0] >> 3) & 1;
			OutHeader.show_frame = (rb[0] >> 4) & 1;

			if (OutHeader.is_experimental)
			{
				return false;
			}

			if (OutHeader.IsKeyframe())
			{
				if (Size < 10 || rb[3] != 0x9d || rb[4] != 0x01 || rb[5] != 0x2a)
				{
					return false;
				}
				OutHeader.horizontal_size_code = (rb[6] + (((uint16)rb[7]) << 8)) & 0x3fff;
				OutHeader.vertical_size_code = (rb[8] + (((uint16)rb[9]) << 8)) & 0x3fff;
			}
			return true;
		}


	} // namespace VPxVideo

} // namespace ElectraDecodersUtil
