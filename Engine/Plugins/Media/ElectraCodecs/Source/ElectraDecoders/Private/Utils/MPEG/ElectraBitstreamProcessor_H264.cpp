// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraBitstreamProcessor_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"
#include "MP4Utilities.h"


class FElectraDecoderBitstreamProcessorH264::FImpl
{
public:
	class FBitstreamInfo : public IElectraDecoderBitstreamInfo
	{
	public:
		virtual ~FBitstreamInfo() = default;
		TArray<ElectraDecodersUtil::MPEG::FSEIMessage> SEIMessages;
		TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet> SPSs;
	};

	enum class ECSDResult
	{
		NoCSD,
		Unchanged,
		Changed,
		Error
	};

	ECSDResult ExtractSPS(const TMap<FString, FVariant>& InFromMap, const TConstArrayView<uint8>& InInbandSPS)
	{
		TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet> NewSPSs;

		auto ParseFromBitstream = [](TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet>& OutNewSPSs, const TConstArrayView<uint8>& InBitstream) -> bool
		{
			TArray<ElectraDecodersUtil::MPEG::H264::FNaluInfo> NALUs;
			if (!ensure(ElectraDecodersUtil::MPEG::H264::ParseBitstreamForNALUs(NALUs, InBitstream.GetData(), InBitstream.Num())))
			{
				return false;
			}
			for(int32 i=0; i<NALUs.Num(); ++i)
			{
				if (NALUs[i].Type != 7)
				{
					continue;
				}
				if (!ensure(ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(OutNewSPSs, InBitstream.GetData() + NALUs[i].Offset + NALUs[i].UnitLength, NALUs[i].Size)))
				{
					return false;
				}
			}
			return true;
		};

		auto AreSPSsIdentical = [](const TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet>& InA, const TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet>& InB) -> bool
		{
			if (InA.Num() != InB.Num())
			{
				return false;
			}
			for(auto& aIt : InA)
			{
				auto pB = InB.Find(aIt.Key);
				if (!pB || aIt.Value.ProcessedBitstream != pB->ProcessedBitstream)
				{
					return false;
				}
			}
			return true;
		};

		// Check inband SPS first. As this is not removed from the bitstream it is always sent to the
		// video decoder where it will most likely win over any prepended sideband CSD as it follows that.
		// Inband SPS does not appear with every AU though.
		if (InInbandSPS.Num())
		{
			if (CurrentDecoderConfiguration == InInbandSPS)
			{
				return ECSDResult::Unchanged;
			}
			if (!ParseFromBitstream(NewSPSs, InInbandSPS))
			{
				SPSs.Empty();
				return ECSDResult::Error;
			}
			if (AreSPSsIdentical(NewSPSs, SPSs))
			{
				CurrentDecoderConfiguration = InInbandSPS;
				return ECSDResult::Unchanged;
			}
			SPSs = MoveTemp(NewSPSs);
			CurrentDecoderConfiguration = InInbandSPS;
			return SPSs.IsEmpty() ? ECSDResult::NoCSD : ECSDResult::Changed;
		}

		// Try the `$avcC_box`
		TArray<uint8> ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, AvccBoxName);
		// If empty try the decoder confiuration record, which is the same as the ConfigData box.
		if (ConfigData.IsEmpty())
		{
			ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, DCRName);
		}
		if (ConfigData.Num())
		{
			// If the data is the same then we don't need to reparse it.
			if (ConfigData == CurrentDecoderConfiguration)
			{
				return ECSDResult::Unchanged;
			}
			ElectraDecodersUtil::MPEG::H264::FAVCDecoderConfigurationRecord dcr;
			if (!dcr.Parse(ConfigData))
			{
				SPSs.Empty();
				return ECSDResult::Error;
			}
			// If the DCR has no SPS (because of avc3 or avc4) and we do not have any previous inband SPS we have to fail.
			if (dcr.GetSequenceParameterSets().IsEmpty())
			{
				if (SPSs.IsEmpty())
				{
					return ECSDResult::Error;
				}
				else
				{
					return ECSDResult::Unchanged;
				}
			}

			const TArray<TArray<uint8>>& dcrSPSs = dcr.GetSequenceParameterSets();
			for(int32 i=0; i<dcrSPSs.Num(); ++i)
			{
				if (!ensure(ElectraDecodersUtil::MPEG::H264::ParseSequenceParameterSet(NewSPSs, dcrSPSs[i].GetData(), dcrSPSs[i].Num())))
				{
					SPSs.Empty();
					return ECSDResult::Error;
				}
			}
			CurrentDecoderConfiguration = MoveTemp(ConfigData);
			if (AreSPSsIdentical(NewSPSs, SPSs))
			{
				return ECSDResult::Unchanged;
			}
			SPSs = MoveTemp(NewSPSs);
			return SPSs.IsEmpty() ? ECSDResult::NoCSD : ECSDResult::Changed;
		}
		// See if pre-extracted CSD is given.
		ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, CSDName);
		if (ConfigData.Num())
		{
			// If the data is the same then we don't need to reparse it.
			if (ConfigData == CurrentDecoderConfiguration)
			{
				return ECSDResult::Unchanged;
			}

			if (!ParseFromBitstream(NewSPSs, ConfigData))
			{
				SPSs.Empty();
				return ECSDResult::Error;
			}
			if (AreSPSsIdentical(NewSPSs, SPSs))
			{
				return ECSDResult::Unchanged;
			}
			SPSs = MoveTemp(NewSPSs);
			CurrentDecoderConfiguration = ConfigData;
			return SPSs.IsEmpty() ? ECSDResult::NoCSD : ECSDResult::Changed;
		}
		return ECSDResult::NoCSD;
	}

	bool UpdateColorimetry(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
	{
		if (!CurrentColorimetry.IsValid() ||
			CurrentColorimetry->colour_primaries != colour_primaries ||
			CurrentColorimetry->transfer_characteristics != transfer_characteristics ||
			CurrentColorimetry->matrix_coeffs != matrix_coeffs ||
			CurrentColorimetry->video_full_range_flag != video_full_range_flag ||
			CurrentColorimetry->video_format != video_format)
		{
			CurrentColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
			CurrentColorimetry->colour_primaries = colour_primaries;
			CurrentColorimetry->transfer_characteristics = transfer_characteristics;
			CurrentColorimetry->matrix_coeffs = matrix_coeffs;
			CurrentColorimetry->video_full_range_flag = video_full_range_flag;
			CurrentColorimetry->video_format = video_format;
			return true;
		}
		return false;
	}

	bool HandlePicTiming(const ElectraDecodersUtil::MPEG::FSEIMessage& InSEI, const TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet>& InSPSs);

	void Clear()
	{
		CurrentDecoderConfiguration.Empty();
		SPSs.Empty();
		CurrentBSI.Reset();
		CurrentColorimetry.Reset();
		LastErrorMessage.Empty();
		FMemory::Memzero(ClockTimestamp);
	}

	static FString AvccBoxName;
	static FString DCRName;
	static FString CSDName;

	TArray<uint8> CurrentDecoderConfiguration;
	TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet> SPSs;
	TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> CurrentBSI;
	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	ElectraDecodersUtil::MPEG::FCommonPictureTiming ClockTimestamp[3];
	FString LastErrorMessage;

	bool bReplaceLengthWithStartcode = false;
};
FString FElectraDecoderBitstreamProcessorH264::FImpl::AvccBoxName(TEXT("$avcC_box"));
FString FElectraDecoderBitstreamProcessorH264::FImpl::DCRName(TEXT("dcr"));
FString FElectraDecoderBitstreamProcessorH264::FImpl::CSDName(TEXT("csd"));


FElectraDecoderBitstreamProcessorH264::~FElectraDecoderBitstreamProcessorH264()
{
}

FElectraDecoderBitstreamProcessorH264::FElectraDecoderBitstreamProcessorH264(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	Impl = MakePimpl<FImpl>();

	int32 S2L = (int32)ElectraDecodersUtil::GetVariantValueSafeI64(InDecoderParams, IElectraDecoderFeature::StartcodeToLength, -1);
	check(S2L == -1 || S2L == 0);

	Impl->bReplaceLengthWithStartcode = S2L == -1;
}

bool FElectraDecoderBitstreamProcessorH264::WillModifyBitstreamInPlace()
{
	return Impl->bReplaceLengthWithStartcode;
}

void FElectraDecoderBitstreamProcessorH264::Clear()
{
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorH264::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	static const uint8 StartCode[4] = {0,0,0,1};

	// Already processed?
	if ((InOutAccessUnit.Flags & EElectraDecoderFlags::InputIsProcessed) != EElectraDecoderFlags::None)
	{
		return EProcessResult::Ok;
	}
	// Set to processed even if we fail somewhere now.
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;

	// Assume this is not a sync sample and that it is discardable.
	// We update the flags in the loop below to reflect the actual states.
	InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsSyncSample;
	InOutAccessUnit.Flags |= EElectraDecoderFlags::IsDiscardable;

	TArray<ElectraDecodersUtil::MPEG::FSEIMessage> SEIMessages;
	TArray<uint8> InbandSPS, InbandPPS;

	// We now go over the data and replace the NALU sizes with startcode (unless the decoder wants to keep the sizes).
	uint32* NALU = (uint32*)InOutAccessUnit.Data;
	uint32* LastNALU = (uint32*)ElectraDecodersUtil::AdvancePointer(NALU, InOutAccessUnit.DataSize);
	while(NALU < LastNALU)
	{
		uint32 naluLen = MP4Utilities::GetFromBigEndian(*NALU);

		// Check the nal_ref_idc in the NAL unit for dependencies.
		uint8 nal = *(const uint8*)(NALU + 1);
		if ((nal & 0x80) != 0)
		{
			Impl->LastErrorMessage = FString::Printf(TEXT("NAL zero bit not zero"));
			return EProcessResult::Error;
		}

		if ((nal >> 5) != 0)
		{
			InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsDiscardable;
		}
		const uint8 nut = nal & 0x1f;
		// IDR frame?
		if (nut == 5)
		{
			InOutAccessUnit.Flags |= EElectraDecoderFlags::IsSyncSample;
		}
		// SEI message(s)?
		else if (nut == 6)
		{
			ElectraDecodersUtil::MPEG::ExtractSEIMessages(SEIMessages, ElectraDecodersUtil::AdvancePointer(NALU, 5), naluLen-1, ElectraDecodersUtil::MPEG::ESEIStreamType::H264, false);
		}
		// SPS?
		else if (nut == 7)
		{
			InbandSPS.Append(StartCode, sizeof(StartCode));
			InbandSPS.Append(reinterpret_cast<const uint8*>(ElectraDecodersUtil::AdvancePointer(NALU, 4)), naluLen);
		}
		// PPS?
		else if (nut == 8)
		{
			InbandPPS.Append(StartCode, sizeof(StartCode));
			InbandPPS.Append(reinterpret_cast<const uint8*>(ElectraDecodersUtil::AdvancePointer(NALU, 4)), naluLen);
		}

		if (Impl->bReplaceLengthWithStartcode)
		{
			*NALU = MP4Utilities::GetFromBigEndian(0x00000001U);
		}
		NALU = ElectraDecodersUtil::AdvancePointer(NALU, naluLen + 4);
	}

	// As for inband SPS and PPS, we need to have both or neither. If we have only one or the other, ignore them both.
	if (InbandSPS.IsEmpty() != InbandPPS.IsEmpty())
	{
		InbandSPS.Empty();
		InbandPPS.Empty();
	}

	FImpl::ECSDResult CsdResult = Impl->ExtractSPS(InAccessUnitSidebandData, InbandSPS);
	if (CsdResult == FImpl::ECSDResult::Error)
	{
		Impl->LastErrorMessage = FString::Printf(TEXT("Failed to parse codec specific data"));
		return EProcessResult::Error;
	}
	else if (CsdResult == FImpl::ECSDResult::Changed)
	{
		TUniquePtr<FImpl::FBitstreamInfo> BSI = MakeUnique<FImpl::FBitstreamInfo>();
		BSI->SPSs = Impl->SPSs;
		Impl->CurrentBSI = MakeShareable<IElectraDecoderBitstreamInfo>(BSI.Release());
	}

	// Narrow down the SEI messages to those we will ultimately handle in SetPropertiesOnOutput() to
	// avoid creating unnecessary individual bitstream info structures
	SEIMessages.RemoveAll([](const ElectraDecodersUtil::MPEG::FSEIMessage& m)
	{
		return m.PayloadType != ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_pic_timing;
	});
	// Are there SEI messages we need to store with this access unit?
	if (SEIMessages.Num())
	{
		TUniquePtr<FImpl::FBitstreamInfo> BSI = MakeUnique<FImpl::FBitstreamInfo>();
		BSI->SPSs = Impl->SPSs;
		BSI->SEIMessages = MoveTemp(SEIMessages);
		OutBSI = MakeShareable<IElectraDecoderBitstreamInfo>(BSI.Release());
	}
	else
	{
		OutBSI = Impl->CurrentBSI;
	}
	return CsdResult == FImpl::ECSDResult::Changed ? EProcessResult::CSDChanged : EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorH264::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	if (!InBSI.IsValid())
	{
		return;
	}
	const FImpl::FBitstreamInfo& BSI(*static_cast<FImpl::FBitstreamInfo*>(InBSI.Get()));

	// We do not know which SPS the decoded slices were referencing, so we just look at the first SPS we have.
	if (BSI.SPSs.Num())
	{
		auto It = BSI.SPSs.CreateConstIterator();
		const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet& sps(It->Value);
		// Set the bit depth and the colorimetry.
		uint8 colour_primaries=2, transfer_characteristics=2, matrix_coeffs=2;
		uint8 video_full_range_flag=0, video_format=5;
		if (sps.colour_description_present_flag)
		{
			colour_primaries = sps.colour_primaries;
			transfer_characteristics = sps.transfer_characteristics;
			matrix_coeffs = sps.matrix_coefficients;
		}
		if (sps.video_signal_type_present_flag)
		{
			video_full_range_flag = sps.video_full_range_flag;
			video_format = sps.video_format;
		}
		// Update the colorimetry information. If it changed we update it in the output properties map.
		if (Impl->UpdateColorimetry(colour_primaries, transfer_characteristics, matrix_coeffs, video_full_range_flag, video_format))
		{
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(Impl->CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}
	}
	uint8 num_bits = 8;
	InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::NumBitsLuma, FVariant(num_bits));

	// Handle SEI messages
	for(int32 i=0, iMax=BSI.SEIMessages.Num(); i<iMax; ++i)
	{
		const ElectraDecodersUtil::MPEG::FSEIMessage& sei = BSI.SEIMessages[i];
		switch(sei.PayloadType)
		{
			case ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_pic_timing:
			{
				if (Impl->HandlePicTiming(sei, BSI.SPSs))
				{
					// Set from the first clock since we are only dealing with progressive frames, not interlaced fields.
					TConstArrayView<uint8> PicTiming = MakeConstArrayView(reinterpret_cast<const uint8*>(&Impl->ClockTimestamp[0]), sizeof(ElectraDecodersUtil::MPEG::FCommonPictureTiming));
					InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonPictureTiming, TArray<uint8>(PicTiming));
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

FString FElectraDecoderBitstreamProcessorH264::GetLastError()
{
	return Impl->LastErrorMessage;
}


bool FElectraDecoderBitstreamProcessorH264::FImpl::HandlePicTiming(const ElectraDecodersUtil::MPEG::FSEIMessage& InSEI, const TMap<uint32, ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet>& InSPSs)
{
	// Parsing the pic_timing() SEI message requires the active SPS.
	// NOTE: We do NOT know which SPS is active, if there are several as the activation is determined by the slice being decoded.
	if (InSPSs.Num() != 1)
	{
		return false;
	}
	auto It = InSPSs.CreateConstIterator();
	const ElectraDecodersUtil::MPEG::H264::FSequenceParameterSet& sps(It->Value);

	ElectraDecodersUtil::MPEG::H264::FBitstreamReaderH264 br(InSEI.Message.GetData(), InSEI.Message.Num());

	const bool CpbDpbDelaysPresentFlag = sps.nal_hrd_parameters_present_flag || sps.vcl_hrd_parameters_present_flag;
	if (CpbDpbDelaysPresentFlag)
	{
		int32 NumBits1 = 1 + (sps.nal_hrd_parameters_present_flag ? sps.nal_hrd_parameters.cpb_removal_delay_length_minus1 : sps.vcl_hrd_parameters.cpb_removal_delay_length_minus1);
		int32 NumBits2 = 1 + (sps.nal_hrd_parameters_present_flag ? sps.nal_hrd_parameters.dpb_output_delay_length_minus1 : sps.vcl_hrd_parameters.dpb_output_delay_length_minus1);
		uint32 cpb_removal_delay = br.GetBits(NumBits1);
		uint32 dpb_output_delay = br.GetBits(NumBits2);
		(void)cpb_removal_delay;
		(void)dpb_output_delay;
	}
	if (sps.pic_struct_present_flag)
	{
		static const uint8 kNumClockTS[9] = { 1, 1, 1, 2, 2, 3, 3, 2, 3 };
		uint32 pic_struct = br.GetBits(4);
		if (pic_struct > 8)
		{
			return false;
		}
		for(int32 nc=0,ncMax=kNumClockTS[pic_struct]; nc<ncMax; ++nc)
		{
			ElectraDecodersUtil::MPEG::FCommonPictureTiming& ct(ClockTimestamp[nc]);
			ct.FromH26x = 4;
			ct.clock_timestamp_flag = (uint8) br.GetBits(1);
			if (ct.clock_timestamp_flag)
			{
				// Set the timing values from the SPS.
				ct.timing_info_present_flag = sps.timing_info_present_flag;
				ct.num_units_in_tick = sps.num_units_in_tick;
				ct.time_scale = sps.time_scale;

				// Read the values from the message.
				ct.ct_type = (uint8) br.GetBits(2);
				ct.nuit_field_based_flag = (uint8) br.GetBits(1);
				ct.counting_type = (uint8) br.GetBits(5);
				ct.full_timestamp_flag = (uint8) br.GetBits(1);
				ct.discontinuity_flag = (uint8) br.GetBits(1);
				ct.cnt_dropped_flag = (uint8) br.GetBits(1);
				ct.n_frames = (uint16) br.GetBits(8);
				if (ct.full_timestamp_flag)
				{
					ct.seconds_value = (uint8) br.GetBits(6);
					ct.minutes_value = (uint8) br.GetBits(6);
					ct.hours_value = (uint8) br.GetBits(5);
				}
				else
				{
					// seconds_flag
					if (br.GetBits(1))
					{
						ct.seconds_value = (uint8) br.GetBits(6);
						// minutes_flag
						if (br.GetBits(1))
						{
							ct.minutes_value = (uint8) br.GetBits(6);
							// hours_flag
							if (br.GetBits(1))
							{
								ct.hours_value = (uint8) br.GetBits(5);
							}
						}
					}
				}
				uint32 time_offset_length = sps.nal_hrd_parameters_present_flag ? sps.nal_hrd_parameters.time_offset_length : sps.vcl_hrd_parameters_present_flag ? sps.vcl_hrd_parameters.time_offset_length : 24;
				uint32 time_offset = br.GetBits(time_offset_length);
				ct.time_offset = (int32)(time_offset << (32 - time_offset_length)) >> (32 - time_offset_length);

				if (ct.timing_info_present_flag)
				{
					ct.clockTimestamp = (((int64)ct.hours_value * 60 + ct.minutes_value) * 60 + ct.seconds_value) * ct.time_scale + ct.n_frames * (ct.num_units_in_tick * (ct.nuit_field_based_flag + 1)) + ct.time_offset;
				}
			}
		}
	}
	return true;
}
