// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraBitstreamProcessor_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "MP4Utilities.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"


class FElectraDecoderBitstreamProcessorH265::FImpl
{
public:
	class FBitstreamInfo : public IElectraDecoderBitstreamInfo
	{
	public:
		virtual ~FBitstreamInfo() = default;
		TArray<ElectraDecodersUtil::MPEG::FSEIMessage> PrefixSEIMessages;
		TArray<ElectraDecodersUtil::MPEG::FSEIMessage> SuffixSEIMessages;
		TMap<uint32, ElectraDecodersUtil::MPEG::H265::FVideoParameterSet> VPSs;
		TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet> SPSs;
	};

	enum class ECSDResult
	{
		NoCSD,
		Unchanged,
		Changed,
		Error
	};

	ECSDResult ExtractVPSSPSPrefixSuffixNUTs(const TMap<FString, FVariant>& InFromMap)
	{
		// First try the `$hvcC_box`
		TArray<uint8> ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, HvccBoxName);
		// If empty try the decoder confiuration record, which is the same as the ConfigData box.
		if (ConfigData.IsEmpty())
		{
			ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, DCRName);
		}
		if (ConfigData.Num())
		{
			// If the data is the same then we don't need to reparse it.
			if (ConfigData == CurrentDecoderConfiguration)
			{
				return ECSDResult::Unchanged;
			}
			VPSs.Empty();
			SPSs.Empty();
			PrefixSEIMessages.Empty();
			SuffixSEIMessages.Empty();
			ElectraDecodersUtil::MPEG::H265::FHEVCDecoderConfigurationRecord dcr;
			if (!dcr.Parse(ConfigData) || dcr.GetSequenceParameterSets().IsEmpty())
			{
				return ECSDResult::Error;
			}
			const TArray<TArray<uint8>>& dcrVPSs = dcr.GetVideoParameterSets();
			for(int32 i=0; i<dcrVPSs.Num(); ++i)
			{
				if (!ensure(ElectraDecodersUtil::MPEG::H265::ParseVideoParameterSet(VPSs, dcrVPSs[i].GetData(), dcrVPSs[i].Num())))
				{
					return ECSDResult::Error;
				}
			}
			const TArray<TArray<uint8>>& dcrSPSs = dcr.GetSequenceParameterSets();
			for(int32 i=0; i<dcrSPSs.Num(); ++i)
			{
				if (!ensure(ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(SPSs, dcrSPSs[i].GetData(), dcrSPSs[i].Num())))
				{
					return ECSDResult::Error;
				}
			}
			const TArray<TArray<uint8>>& dcrPrfx = dcr.GetPrefixNUTs();
			for(int32 i=0; i<dcrPrfx.Num(); ++i)
			{
				if (!ensure(ElectraDecodersUtil::MPEG::ExtractSEIMessages(PrefixSEIMessages, dcrPrfx[i].GetData()+2, dcrPrfx[i].Num()-2, ElectraDecodersUtil::MPEG::ESEIStreamType::H265, true)))
				{
					return ECSDResult::Error;
				}
			}
			const TArray<TArray<uint8>>& dcrSufx = dcr.GetSuffixNUTs();
			for(int32 i=0; i<dcrSufx.Num(); ++i)
			{
				if (!ensure(ElectraDecodersUtil::MPEG::ExtractSEIMessages(SuffixSEIMessages, dcrSufx[i].GetData()+2, dcrSufx[i].Num()-2, ElectraDecodersUtil::MPEG::ESEIStreamType::H265, false)))
				{
					return ECSDResult::Error;
				}
			}
			CurrentDecoderConfiguration = MoveTemp(ConfigData);
			return SPSs.IsEmpty() ? ECSDResult::NoCSD : ECSDResult::Changed;
		}
		// See if pre-extracted CSD is given.
		ConfigData = ElectraDecodersUtil::GetVariantValueUInt8Array(InFromMap, CSDName);
		if (ConfigData.Num())
		{
			// If the data is the same then we don't need to reparse it.
			if (ConfigData == CurrentDecoderConfiguration)
			{
				return ECSDResult::Unchanged;
			}

			VPSs.Empty();
			SPSs.Empty();
			PrefixSEIMessages.Empty();
			SuffixSEIMessages.Empty();
			TArray<ElectraDecodersUtil::MPEG::H265::FNaluInfo> NALUs;
			if (!ensure(ElectraDecodersUtil::MPEG::H265::ParseBitstreamForNALUs(NALUs, ConfigData.GetData(), ConfigData.Num())))
			{
				return ECSDResult::Error;
			}
			for(int32 i=0; i<NALUs.Num(); ++i)
			{
				// VPS NUT?
				if (NALUs[i].Type == 32 && !ensure(ElectraDecodersUtil::MPEG::H265::ParseVideoParameterSet(VPSs, ConfigData.GetData() + NALUs[i].Offset + NALUs[i].UnitLength, NALUs[i].Size)))
				{
					return ECSDResult::Error;
				}
				// SPS NUT?
				else if (NALUs[i].Type == 33 && !ensure(ElectraDecodersUtil::MPEG::H265::ParseSequenceParameterSet(SPSs, ConfigData.GetData() + NALUs[i].Offset + NALUs[i].UnitLength, NALUs[i].Size)))
				{
					return ECSDResult::Error;
				}
				// Prefix or suffix NUT?
				else if (NALUs[i].Type == 39 || NALUs[i].Type == 40)
				{
					ElectraDecodersUtil::MPEG::ExtractSEIMessages(NALUs[i].Type == 39 ? PrefixSEIMessages : SuffixSEIMessages, ConfigData.GetData() + NALUs[i].Offset + NALUs[i].UnitLength + 2, NALUs[i].Size - 2, ElectraDecodersUtil::MPEG::ESEIStreamType::H265, NALUs[i].Type == 39);
				}

			}
			CurrentDecoderConfiguration = MoveTemp(ConfigData);
			return SPSs.IsEmpty() ? ECSDResult::NoCSD : ECSDResult::Changed;
		}
		return ECSDResult::NoCSD;
	}

	bool UpdateColorimetry(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
	{
		if (!CurrentColorimetry.IsValid() ||
			CurrentColorimetry->colour_primaries != colour_primaries ||
			CurrentColorimetry->transfer_characteristics != transfer_characteristics ||
			CurrentColorimetry->matrix_coeffs != matrix_coeffs ||
			CurrentColorimetry->video_full_range_flag != video_full_range_flag ||
			CurrentColorimetry->video_format != video_format)
		{
			CurrentColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
			CurrentColorimetry->colour_primaries = colour_primaries;
			CurrentColorimetry->transfer_characteristics = transfer_characteristics;
			CurrentColorimetry->matrix_coeffs = matrix_coeffs;
			CurrentColorimetry->video_full_range_flag = video_full_range_flag;
			CurrentColorimetry->video_format = video_format;
			return true;
		}
		return false;
	}

	bool HandleTimeCode(const ElectraDecodersUtil::MPEG::FSEIMessage& InSEI, const TMap<uint32, ElectraDecodersUtil::MPEG::H265::FVideoParameterSet>& InVPSs, const TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet>& InSPSs);

	void Clear()
	{
		CurrentDecoderConfiguration.Empty();
		SPSs.Empty();
		PrefixSEIMessages.Empty();
		SuffixSEIMessages.Empty();
		CurrentBSI.Reset();
		CurrentColorimetry.Reset();
		CurrentMDCV.Reset();
		CurrentCLLI.Reset();
		CurrentALTC.Reset();
		LastErrorMessage.Empty();
		FMemory::Memzero(ClockTimestamp);
	}

	static FString HvccBoxName;
	static FString DCRName;
	static FString CSDName;

	TArray<uint8> CurrentDecoderConfiguration;
	TMap<uint32, ElectraDecodersUtil::MPEG::H265::FVideoParameterSet> VPSs;
	TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet> SPSs;
	TArray<ElectraDecodersUtil::MPEG::FSEIMessage> PrefixSEIMessages;
	TArray<ElectraDecodersUtil::MPEG::FSEIMessage> SuffixSEIMessages;
	TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> CurrentBSI;
	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> CurrentMDCV;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> CurrentCLLI;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics, ESPMode::ThreadSafe> CurrentALTC;

	ElectraDecodersUtil::MPEG::FCommonPictureTiming ClockTimestamp[3];
	FString LastErrorMessage;

	bool bReplaceLengthWithStartcode = false;
};
FString FElectraDecoderBitstreamProcessorH265::FImpl::HvccBoxName(TEXT("$hvcC_box"));
FString FElectraDecoderBitstreamProcessorH265::FImpl::DCRName(TEXT("dcr"));
FString FElectraDecoderBitstreamProcessorH265::FImpl::CSDName(TEXT("csd"));


FElectraDecoderBitstreamProcessorH265::~FElectraDecoderBitstreamProcessorH265()
{
}

FElectraDecoderBitstreamProcessorH265::FElectraDecoderBitstreamProcessorH265(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	Impl = MakePimpl<FImpl>();

	int32 S2L = (int32)ElectraDecodersUtil::GetVariantValueSafeI64(InDecoderParams, IElectraDecoderFeature::StartcodeToLength, -1);
	check(S2L == -1 || S2L == 0);

	Impl->bReplaceLengthWithStartcode = S2L == -1;
}

bool FElectraDecoderBitstreamProcessorH265::WillModifyBitstreamInPlace()
{
	return Impl->bReplaceLengthWithStartcode;
}

void FElectraDecoderBitstreamProcessorH265::Clear()
{
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorH265::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	// Already processed?
	if ((InOutAccessUnit.Flags & EElectraDecoderFlags::InputIsProcessed) != EElectraDecoderFlags::None)
	{
		return EProcessResult::Ok;
	}

	auto ReduceSEIMessages = [](TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& io) -> void
	{
		io.RemoveAll([](const ElectraDecodersUtil::MPEG::FSEIMessage& m)
		{
			return m.PayloadType != ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_time_code &&
				   m.PayloadType != ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_mastering_display_colour_volume &&
				   m.PayloadType != ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_content_light_level_info &&
				   m.PayloadType != ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_alternative_transfer_characteristics;
		});
	};


	// Set to processed even if we fail somewhere now.
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;

	FImpl::ECSDResult CsdResult = Impl->ExtractVPSSPSPrefixSuffixNUTs(InAccessUnitSidebandData);
	if (CsdResult == FImpl::ECSDResult::Error)
	{
		Impl->LastErrorMessage = FString::Printf(TEXT("Failed to parse codec specific data"));
		return EProcessResult::Error;
	}
	else if (CsdResult == FImpl::ECSDResult::Changed)
	{
		TUniquePtr<FImpl::FBitstreamInfo> BSI = MakeUnique<FImpl::FBitstreamInfo>();
		BSI->SPSs = Impl->SPSs;
		BSI->VPSs = Impl->VPSs;
		ReduceSEIMessages(Impl->PrefixSEIMessages);
		ReduceSEIMessages(Impl->SuffixSEIMessages);
		Impl->CurrentBSI = MakeShareable<IElectraDecoderBitstreamInfo>(BSI.Release());
	}

	// Assume this is not a sync sample and that it is discardable.
	// We update the flags in the loop below to reflect the actual states.
	InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsSyncSample;
	InOutAccessUnit.Flags &= ~EElectraDecoderFlags::IsDiscardable;

	TArray<ElectraDecodersUtil::MPEG::FSEIMessage> PrefixSEIMessages;
	TArray<ElectraDecodersUtil::MPEG::FSEIMessage> SuffixSEIMessages;

	// We now go over the data and replace the NALU sizes with startcode (unless the decoder wants to keep the sizes).
	uint32* NALU = (uint32*)InOutAccessUnit.Data;
	uint32* LastNALU = (uint32*)ElectraDecodersUtil::AdvancePointer(NALU, InOutAccessUnit.DataSize);
	while(NALU < LastNALU)
	{
		uint32 naluLen = MP4Utilities::GetFromBigEndian(*NALU);

		// Check the nal_ref_idc in the NAL unit for dependencies.
		uint8 nut = *(const uint8*)(NALU + 1);
		if ((nut & 0x80) != 0)
		{
			Impl->LastErrorMessage = FString::Printf(TEXT("NUT zero bit not zero"));
			return EProcessResult::Error;
		}
		nut >>= 1;

		// IDR, CRA or BLA frame?
		if (nut >= 16 && nut <= 21)
		{
			InOutAccessUnit.Flags |= EElectraDecoderFlags::IsSyncSample;
		}
		// One of TRAIL_N, TSA_N, STSA_N, RADL_N, RASL_N, RSV_VCL_N10, RSV_VCL_N12 or RSV_VCL_N14 ?
		else if (nut <= 14 && (nut & 1) == 0)
		{
			InOutAccessUnit.Flags |= EElectraDecoderFlags::IsDiscardable;
		}
		// Prefix or suffix NUT?
		else if (nut == 39 || nut == 40)
		{
			ElectraDecodersUtil::MPEG::ExtractSEIMessages(nut == 39 ? PrefixSEIMessages : SuffixSEIMessages, ElectraDecodersUtil::AdvancePointer(NALU, 6), naluLen-2, ElectraDecodersUtil::MPEG::ESEIStreamType::H265, nut == 39);
		}

		// NOTE: If we want to handle hev1 with inband VPS/SPS/PPS we can extract them here!

		if (Impl->bReplaceLengthWithStartcode)
		{
			*NALU = MP4Utilities::GetFromBigEndian(0x00000001U);
		}
		NALU = ElectraDecodersUtil::AdvancePointer(NALU, naluLen + 4);
	}

	// Narrow down the SEI messages to those we will ultimately handle in SetPropertiesOnOutput() to
	// avoid creating unnecessary individual bitstream info structures
	ReduceSEIMessages(PrefixSEIMessages);
	ReduceSEIMessages(SuffixSEIMessages);
	// Are there SEI messages we need to store with this access unit?
	if (PrefixSEIMessages.Num() || SuffixSEIMessages.Num())
	{
		TUniquePtr<FImpl::FBitstreamInfo> BSI = MakeUnique<FImpl::FBitstreamInfo>();
		BSI->SPSs = Impl->SPSs;
		BSI->VPSs = Impl->VPSs;
		BSI->PrefixSEIMessages = Impl->PrefixSEIMessages;
		BSI->SuffixSEIMessages = Impl->SuffixSEIMessages;
		BSI->PrefixSEIMessages.Append(MoveTemp(PrefixSEIMessages));
		BSI->SuffixSEIMessages.Append(MoveTemp(SuffixSEIMessages));
		OutBSI = MakeShareable<IElectraDecoderBitstreamInfo>(BSI.Release());
	}
	else
	{
		OutBSI = Impl->CurrentBSI;
	}
	return CsdResult == FImpl::ECSDResult::Changed ? EProcessResult::CSDChanged : EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorH265::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	if (!InBSI.IsValid())
	{
		return;
	}
	const FImpl::FBitstreamInfo& BSI(*static_cast<FImpl::FBitstreamInfo*>(InBSI.Get()));

	uint8 num_bits = 8;

	// We do not know which SPS the decoded slices were referencing, so we just look at the first SPS we have.
	if (BSI.SPSs.Num())
	{
		auto It = Impl->SPSs.CreateConstIterator();
		const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& sps(It->Value);
		// Set the bit depth and the colorimetry.
		uint8 colour_primaries=2, transfer_characteristics=2, matrix_coeffs=2;
		uint8 video_full_range_flag=0, video_format=5;

		num_bits = sps.bit_depth_luma_minus8 + 8;
		if (sps.vui_parameters_present_flag)
		{
			if (sps.vui_parameters.colour_description_present_flag)
			{
				colour_primaries = sps.vui_parameters.colour_primaries;
				transfer_characteristics = sps.vui_parameters.transfer_characteristics;
				matrix_coeffs = sps.vui_parameters.matrix_coeffs;
			}
			if (sps.vui_parameters.video_signal_type_present_flag)
			{
				video_full_range_flag = sps.vui_parameters.video_full_range_flag;
				video_format = sps.vui_parameters.video_format;
			}
		}

		// Update the colorimetry information. If it changed we update it in the output properties map.
		if (Impl->UpdateColorimetry(colour_primaries, transfer_characteristics, matrix_coeffs, video_full_range_flag, video_format))
		{
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(Impl->CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}
	}
	InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::NumBitsLuma, FVariant(num_bits));

	// Handle prefix SEI messages
	bool bMDCVchanged = false;
	bool bCLLIchanged = false;
	bool bALTCchanged = false;
	for(int32 i=0, iMax=BSI.PrefixSEIMessages.Num(); i<iMax; ++i)
	{
		const ElectraDecodersUtil::MPEG::FSEIMessage& sei = BSI.PrefixSEIMessages[i];
		switch(sei.PayloadType)
		{
			case ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_time_code:
			{
				if (Impl->HandleTimeCode(sei, BSI.VPSs, BSI.SPSs))
				{
					// Set from the first clock since we are only dealing with progressive frames, not interlaced fields.
					TConstArrayView<uint8> TimeCode = MakeConstArrayView(reinterpret_cast<const uint8*>(&Impl->ClockTimestamp[0]), sizeof(ElectraDecodersUtil::MPEG::FCommonPictureTiming));
					InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonPictureTiming, TArray<uint8>(TimeCode));
				}
				break;
			}
			case ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_mastering_display_colour_volume:
			{
				ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume mdcv;
				if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(mdcv, sei))
				{
					if (!Impl->CurrentMDCV.IsValid() || FMemory::Memcmp(Impl->CurrentMDCV.Get(), &mdcv, sizeof(mdcv)))
					{
						Impl->CurrentMDCV = MakeShareable<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume>(new ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume(mdcv));
						bMDCVchanged = true;
					}
				}
				break;
			}
			case ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_content_light_level_info:
			{
				ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info clli;
				if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(clli, sei))
				{
					if (!Impl->CurrentCLLI.IsValid() || FMemory::Memcmp(Impl->CurrentCLLI.Get(), &clli, sizeof(clli)))
					{
						Impl->CurrentCLLI = MakeShareable<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info>(new ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info(clli));
						bCLLIchanged = true;
					}
				}
				break;
			}
			case ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_alternative_transfer_characteristics:
			{
				ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics altc;
				if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(altc, sei))
				{
					if (!Impl->CurrentALTC.IsValid() || FMemory::Memcmp(Impl->CurrentALTC.Get(), &altc, sizeof(altc)))
					{
						Impl->CurrentALTC = MakeShareable<ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics>(new ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics(altc));
						bALTCchanged = true;
					}
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}
	// Set the changed parameters in the output
	if (bMDCVchanged)
	{
		TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(Impl->CurrentMDCV.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume));
		InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume, TArray<uint8>(Params));
	}
	if (bCLLIchanged)
	{
		TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(Impl->CurrentCLLI.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info));
		InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo, TArray<uint8>(Params));
	}
	if (bALTCchanged)
	{
		TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(Impl->CurrentALTC.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics));
		InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiAlternateTransferCharacteristics, TArray<uint8>(Params));
	}
}

FString FElectraDecoderBitstreamProcessorH265::GetLastError()
{
	return Impl->LastErrorMessage;
}

bool FElectraDecoderBitstreamProcessorH265::FImpl::HandleTimeCode(const ElectraDecodersUtil::MPEG::FSEIMessage& InSEI, const TMap<uint32, ElectraDecodersUtil::MPEG::H265::FVideoParameterSet>& InVPSs, const TMap<uint32, ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet>& InSPSs)
{
	// Parsing the time_code() SEI message requires the active SPS (and VPS)
	// NOTE: We do NOT know which SPS is active, if there are several as the activation is determined by the slice being decoded.
	if (InSPSs.Num() != 1)
	{
		return false;
	}
	auto It = InSPSs.CreateConstIterator();
	const ElectraDecodersUtil::MPEG::H265::FSequenceParameterSet& sps(It->Value);

	// We also need to have the VPS that this SPS is referencing.
	if (!VPSs.Contains(sps.sps_video_parameter_set_id))
	{
		return false;
	}
	const ElectraDecodersUtil::MPEG::H265::FVideoParameterSet& vps(VPSs[sps.sps_video_parameter_set_id]);

	ElectraDecodersUtil::MPEG::H265::FBitstreamReaderH265 br(InSEI.Message.GetData(), InSEI.Message.Num());

	uint32 num_clock_ts = br.GetBits(2);
	for(uint32 nc=0; nc<num_clock_ts; ++nc)
	{
		ElectraDecodersUtil::MPEG::FCommonPictureTiming& ct(ClockTimestamp[nc]);
		ct.FromH26x = 5;
		ct.clock_timestamp_flag = (uint8) br.GetBits(1);
		if (ct.clock_timestamp_flag)
		{
			// Set the timing values from the SPS or VPS.
			ct.timing_info_present_flag = vps.vps_timing_info_present_flag;
			ct.num_units_in_tick = sps.vui_parameters_present_flag && sps.vui_parameters.vui_timing_info_present_flag ? sps.vui_parameters.vui_num_units_in_tick : vps.vps_timing_info_present_flag ? vps.vps_num_units_in_tick : 0;
			ct.time_scale = sps.vui_parameters_present_flag && sps.vui_parameters.vui_timing_info_present_flag ? sps.vui_parameters.vui_time_scale : vps.vps_timing_info_present_flag ? vps.vps_time_scale : 1;

			// Read the values from the message.
			ct.nuit_field_based_flag = (uint8) br.GetBits(1);		// In H.265 this is actually now named correctly `units_field_based_flag` ,...
			ct.counting_type = (uint8) br.GetBits(5);
			ct.full_timestamp_flag = (uint8) br.GetBits(1);
			ct.discontinuity_flag = (uint8) br.GetBits(1);
			ct.cnt_dropped_flag = (uint8) br.GetBits(1);
			ct.n_frames = (uint16) br.GetBits(9);
			if (ct.full_timestamp_flag)
			{
				ct.seconds_value = (uint8) br.GetBits(6);
				ct.minutes_value = (uint8) br.GetBits(6);
				ct.hours_value = (uint8) br.GetBits(5);
			}
			else
			{
				// seconds_flag
				if (br.GetBits(1))
				{
					ct.seconds_value = (uint8) br.GetBits(6);
					// minutes_flag
					if (br.GetBits(1))
					{
						ct.minutes_value = (uint8) br.GetBits(6);
						// hours_flag
						if (br.GetBits(1))
						{
							ct.hours_value = (uint8) br.GetBits(5);
						}
					}
				}
			}
			uint32 time_offset_length = br.GetBits(5);
			if (time_offset_length)
			{
				uint32 time_offset = br.GetBits(time_offset_length);
				ct.time_offset = (int32)(time_offset << (32 - time_offset_length)) >> (32 - time_offset_length);
			}
			else
			{
				ct.time_offset = 0;
			}

			if (ct.timing_info_present_flag)
			{
				ct.clockTimestamp = (((int64)ct.hours_value * 60 + ct.minutes_value) * 60 + ct.seconds_value) * ct.time_scale + ct.n_frames * (ct.num_units_in_tick * (ct.nuit_field_based_flag + 1)) + ct.time_offset;
			}
		}
	}
	return true;
}
