// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraUtilsMP4.h"
#include "MP4Utilities.h"
#include "Features/IModularFeatures.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraCodecFactory.h"
#include "IElectraCodecFormatUtils.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"

namespace ElectraDecodersUtil
{
	namespace MP4
	{
		bool GetTrackFormatInfo(FString& OutMessage, Electra::FCodecTypeFormat& OutCodecInfo, Electra::FDRMTypeFormat& OutDRMInfo, Electra::FDecoderInformation& OutDecoderInfo, const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrack, uint32 InTrackID, bool bInCheckIfDecodable)
		{
			IElectraCodecFactoryModule* CodecFactoryModule = nullptr;
			if (bInCheckIfDecodable)
			{
				CodecFactoryModule = static_cast<IElectraCodecFactoryModule*>(FModuleManager::Get().GetModule(TEXT("ElectraCodecFactory")));
				if (!CodecFactoryModule)
				{
					OutMessage = FString::Printf(TEXT("Electra decoder factory not found. Unable to use any track."));
					return false;
				}
			}
			auto CheckIfDecodable = [&]() -> void
			{
				// Shall we determine if there is a decoder able to decode this track?
				if (CodecFactoryModule)
				{
					TMap<FString, FVariant> FormatInfo;
					auto Factory = CodecFactoryModule->GetBestDecoderFactoryForFormat(FormatInfo, OutCodecInfo, TMap<FString, FVariant>());
					if (Factory.IsValid())
					{
						OutDecoderInfo.bIsDecodable = true;
						// Is there missing information the decoder factory could provide?
						TMap<FString, FVariant> ConfigOptions;
						Factory->GetConfigurationOptions(ConfigOptions);
						if (OutCodecInfo.KeyframeMode == Electra::FCodecTypeFormat::EKeyframeMode::Unknown && ConfigOptions.Find(IElectraDecoderFormatInfo::IsEveryFrameKeyframe))
						{
							OutCodecInfo.KeyframeMode = !!ElectraDecodersUtil::GetVariantValueSafeI64(FormatInfo, IElectraDecoderFormatInfo::IsEveryFrameKeyframe, 0)
														? Electra::FCodecTypeFormat::EKeyframeMode::OnlyKeyframes : Electra::FCodecTypeFormat::EKeyframeMode::DeltaFrames;
						}
						if (OutCodecInfo.HumanReadableFormatInfo.IsEmpty() && ConfigOptions.Find(IElectraDecoderFormatInfo::HumanReadableFormatName))
						{
							OutCodecInfo.HumanReadableFormatInfo = ElectraDecodersUtil::GetVariantValueFString(FormatInfo, IElectraDecoderFormatInfo::HumanReadableFormatName);
						}
						OutDecoderInfo.ProviderInfo.Name = Factory->GetProviderInformation().GetName();
						OutDecoderInfo.ProviderInfo.Version = Factory->GetProviderInformation().GetVersion();
						OutDecoderInfo.ProviderInfo.Implementation = Factory->GetProviderInformation().GetImplementation();
						OutDecoderInfo.ProviderInfo.Vendor = Factory->GetProviderInformation().GetVendor();
					}
					else
					{
						OutDecoderInfo.bIsDecodable = false;
					}
				}
			};

			// There needs to be an `stsd` box in this track. We do not try the expected path of `trak`->`mdia`->`minf`->`stbl`->`stsd` as
			// this is not that much faster and if the file is somewhat ill-formed we may not find it if it's grouped under elsewhere.
			auto Stsd = InTrack->FindBoxRecursive<MP4Boxes::FMP4BoxSTSD>(MP4Utilities::MakeBoxAtom('s','t','s','d'), 6);
			if (!Stsd)
			{
				// If not found we just ignore the track. That's a warning but not an error.
				OutMessage = FString::Printf(TEXT("No `stsd` box found, ignoring track #%u."), InTrackID);
				return false;
			}

			// Get language code
			auto Mdhd = InTrack->FindBoxRecursive<MP4Boxes::FMP4BoxMDHD>(MP4Utilities::MakeBoxAtom('m','d','h','d'), 2);
			if (Mdhd)
			{
				Electra::BCP47::ParseRFC5646Tag(OutCodecInfo.LanguageTag, Mdhd->GetLanguageCode639_2T());
			}
			auto Elng = InTrack->FindBoxRecursive<MP4Boxes::FMP4BoxELNG>(MP4Utilities::MakeBoxAtom('e','l','n','g'), 6);
			if (Elng)
			{
				Electra::BCP47::ParseRFC5646Tag(OutCodecInfo.LanguageTag, Elng->GetLanguageTag());
			}

			auto AddChildren = [&OutCodecInfo](const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InSample) -> void
			{
				for(auto& Ch : InSample->GetChildren())
				{
					OutCodecInfo.ExtraBoxes.Emplace(Ch->GetType(), Ch->GetBoxData());
				}
			};

			auto SetupEncryption = [&](uint32& InOutSampleType, const TSharedPtr<MP4Boxes::FMP4BoxBase, ESPMode::ThreadSafe>& InSample) -> bool
			{
				// Locate the encryption related boxes
				auto Frma = InSample->FindBoxRecursive<MP4Boxes::FMP4BoxFRMA>(MP4Utilities::MakeBoxAtom('f','r','m','a'), 2);
				if (!Frma)
				{
					OutMessage = FString::Printf(TEXT("Encrypted track #%u is missing mandatory `frma` box, ignoring this track."), InTrackID);
					return false;
				}
				InOutSampleType = Frma->GetOriginalFormat();

				OutDRMInfo.EncryptionInfo.Emplace<Electra::FDRMTypeFormat::FISOEncryptionInfo>();
				Electra::FDRMTypeFormat::FISOEncryptionInfo& iso = OutDRMInfo.EncryptionInfo.Get<Electra::FDRMTypeFormat::FISOEncryptionInfo>();
				// Optional `schm` box.
				if (auto Schm = InSample->FindBoxRecursive<MP4Boxes::FMP4BoxSCHM>(MP4Utilities::MakeBoxAtom('s','c','h','m'), 2))
				{
					iso.Scheme = Schm->GetSchemeType();
					iso.SchemeVersion = Schm->GetSchemeVersion();
				}
				// Optional `tenc` box.
				if (auto Tenc = InSample->FindBoxRecursive<MP4Boxes::FMP4BoxTENC>(MP4Utilities::MakeBoxAtom('t','e','n','c'), 3))
				{
					if (Tenc->HasDefaultCryptBlockValues())
					{
						Electra::FDRMTypeFormat::FISOEncryptionInfo::FBlockPattern ptn;
						ptn.CryptByteBlock = Tenc->GetDefaultCryptByteBlock();
						ptn.SkipByteBlock = Tenc->GetDefaultSkipByteBlock();
						iso.BlockPattern = MoveTemp(ptn);
					}
					iso.DefaultKID = Tenc->GetDefaultKID();
					iso.DefaultIV = Tenc->GetDefaultConstantIV();
					iso.DefaultIVSize = Tenc->GetDefaultPerSampleIVSize();
				}
				// Optional `pssh` boxes.
				if (auto Moov = InSample->FindParentBox<MP4Boxes::FMP4BoxMOOV>(MP4Utilities::MakeBoxAtom('m','o','o','v')))
				{
					TArray<TSharedPtr<MP4Boxes::FMP4BoxPSSH, ESPMode::ThreadSafe>> PsshBoxes;
					Moov->GetAllBoxInstances<MP4Boxes::FMP4BoxPSSH>(PsshBoxes, MP4Utilities::MakeBoxAtom('p','s','s','h'));
					for(auto &pssh : PsshBoxes)
					{
						Electra::FDRMTypeFormat::FISOEncryptionInfo::FCDMInfo& cdm = iso.CDMInfos.Emplace_GetRef();
						cdm.SystemID = pssh->GetSystemID();
						cdm.Data = pssh->GetData();
						for(auto &kid : pssh->GetKIDs())
						{
							cdm.KIDs.Emplace(kid);
						}
					}
				}
				return true;
			};

			// Get the sample type of this track. This call is necessary to trigger parsing of child nodes.
			MP4Boxes::FMP4BoxSampleEntry::ESampleType SampleType = Stsd->GetSampleType();
			// If already knows to not be supported, skip it.
			if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Unsupported)
			{
				OutMessage = FString::Printf(TEXT("Unsupported sample type, ignoring track #%u."), InTrackID);
				return false;
			}
			// Several entries are permitted, but we need this to be unambiguous.
			if (Stsd->GetChildren().Num() > 1)
			{
				OutMessage = FString::Printf(TEXT("Multiple sample descriptions found in `stsd` box, ignoring track #%u."), InTrackID);
				return false;
			}
			else if (Stsd->GetChildren().Num() == 0)
			{
				OutMessage = FString::Printf(TEXT("No sample description found in `stsd` box, ignoring track #%u."), InTrackID);
				return false;
			}

			OutDecoderInfo.bIsDecodable = true;
			// Based on the sample type, get it and see if it is using codec we support.
			if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Video)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxVisualSampleEntry, ESPMode::ThreadSafe>> Visuals;
				Stsd->GetSampleDescriptions(Visuals);
				if (Visuals.Num() != 1)
				{
					if (Visuals.IsEmpty())
					{
						OutMessage = FString::Printf(TEXT("No visual sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					else
					{
						OutMessage = FString::Printf(TEXT("More than one visual sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					return false;
				}

				uint32 VisualType = Visuals[0]->GetType();
				// Encrypted?
				if (VisualType == MP4Utilities::MakeBoxAtom('e','n','c','v'))
				{
					if (!SetupEncryption(VisualType, Visuals[0]))
					{
						return false;
					}
				}
				if (Visuals[0]->GetFrameCount() == 1)
				{
					OutCodecInfo.Type = Electra::FCodecTypeFormat::EType::Video;
					OutCodecInfo.FourCC = VisualType;
					OutCodecInfo.Properties.Emplace<Electra::FCodecTypeFormat::FVideo>();
					Electra::FCodecTypeFormat::FVideo& cvid = OutCodecInfo.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
					cvid.Width = Visuals[0]->GetWidth();
					cvid.Height = Visuals[0]->GetHeight();
					auto Stts = InTrack->FindBoxRecursive<MP4Boxes::FMP4BoxSTTS>(MP4Utilities::MakeBoxAtom('s','t','t','s'), 5);
					if (Mdhd.IsValid() && Stts.IsValid() && Stts->GetEntries().Num())
					{
						if (Stts->HasConstantSampleDuration())
						{
							// Simple case: all frames have the same decode delta.
							cvid.FrameRate = FFrameRate(Mdhd->GetTimescale(), Stts->GetConstantSampleDuration());
						}
						else
						{
							// Variable decode deltas (common with B-frame reordering in Android HEVC).
							// Compute the average rate from totals and snap to the nearest integer rate
							// if close, since the variance is typically just a muxer artifact.
							const uint32 Timescale = Mdhd->GetTimescale();
							const uint32 TotalSamples = Stts->GetNumTotalSamples();
							const int64 TotalDuration = Stts->GetTotalDuration();
							if (TotalSamples > 0 && TotalDuration > 0 && Timescale > 0)
							{
								const double AvgFps = (double)TotalSamples * Timescale / TotalDuration;
								const double NearestInt = FMath::RoundToDouble(AvgFps);
								// Snap to integer rate if within 0.05% (tight enough to not confuse 29.97/23.976 with 30/24)
								if (NearestInt > 0 && FMath::Abs(AvgFps - NearestInt) / NearestInt < 0.0005)
								{
									cvid.FrameRate = FFrameRate((uint32)NearestInt, 1);
								}
								else
								{
									// Non-integer rate (e.g. 29.97): use 128-bit integer math to safely
									// compute milliFPS without uint32 overflow on long videos.
									// Bias by TotalDuration/2000 to round instead of truncate
									// (the /2000 accounts for the ×1000 inside ConvertToTimescale).
									int64 Numerator = (int64)TotalSamples * Timescale + TotalDuration / 2000;
									int64 MilliFps = MP4Utilities::ConvertToTimescale(1000, Numerator, (uint32)TotalDuration);
									cvid.FrameRate = FFrameRate((int32)MilliFps, 1000);
								}
							}
						}
					}
					for(auto& Ch : Visuals[0]->GetChildren())
					{
						OutCodecInfo.ExtraBoxes.Emplace(Ch->GetType(), Ch->GetBoxData());
					}
					if (!ElectraDecodersUtil::PrepareCodecTypeFormat(OutCodecInfo))
					{
						OutMessage = FString::Printf(TEXT("Failed to prepare the codec format type, ignoring track #%u."), InTrackID);
						return false;
					}
					// Check if this video is decodable.
					CheckIfDecodable();
				}
				else
				{
					OutMessage = FString::Printf(TEXT("Track #%u has a frame count other than 1 in the VisualSampleEntry, ignoring this track."), InTrackID);
					return false;
				}
			}
			else if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Audio)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxAudioSampleEntry, ESPMode::ThreadSafe>> Audios;
				Stsd->GetSampleDescriptions(Audios);
				if (Audios.Num() != 1)
				{
					if (Audios.IsEmpty())
					{
						OutMessage = FString::Printf(TEXT("No audio sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					else
					{
						OutMessage = FString::Printf(TEXT("More than one audio sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					return false;
				}

				uint32 AudioType = Audios[0]->GetType();
				// Encrypted?
				if (AudioType == MP4Utilities::MakeBoxAtom('e','n','c','a'))
				{
					if (!SetupEncryption(AudioType, Audios[0]))
					{
						return false;
					}
				}
				OutCodecInfo.Type = Electra::FCodecTypeFormat::EType::Audio;
				OutCodecInfo.FourCC = AudioType;
				OutCodecInfo.Properties.Emplace<Electra::FCodecTypeFormat::FAudio>();
				Electra::FCodecTypeFormat::FAudio& caud = OutCodecInfo.Properties.Get<Electra::FCodecTypeFormat::FAudio>();
				caud.NumChannels = Audios[0]->GetChannelCount();
				caud.SampleRate = Audios[0]->GetSampleRate();
				for(auto& Ch : Audios[0]->GetChildren())
				{
					OutCodecInfo.ExtraBoxes.Emplace(Ch->GetType(), Ch->GetBoxData());
				}
				if (!ElectraDecodersUtil::PrepareCodecTypeFormat(OutCodecInfo))
				{
					OutMessage = FString::Printf(TEXT("Failed to prepare the codec format type, ignoring track #%u."), InTrackID);
					return false;
				}
				// Check if this audio is decodable.
				CheckIfDecodable();
			}
			else if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::Subtitles || SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::TimedText)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxSampleEntry, ESPMode::ThreadSafe>> Subtitles;
				Stsd->GetSampleDescriptions(Subtitles);
				if (Subtitles.Num() != 1)
				{
					if (Subtitles.IsEmpty())
					{
						OutMessage = FString::Printf(TEXT("No subtitle sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					else
					{
						OutMessage = FString::Printf(TEXT("More than one subtitle sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					return false;
				}

				uint32 SubtitleType = Subtitles[0]->GetType();
				// Encrypted?
				if (SubtitleType == MP4Utilities::MakeBoxAtom('e','n','c','t') || SubtitleType == MP4Utilities::MakeBoxAtom('e','n','c','u'))
				{
					if (!SetupEncryption(SubtitleType, Subtitles[0]))
					{
						return false;
					}
				}
				OutCodecInfo.Type = Electra::FCodecTypeFormat::EType::Subtitle;
				OutCodecInfo.FourCC = SubtitleType;
				OutCodecInfo.Properties.Emplace<Electra::FCodecTypeFormat::FSubtitle>();
				//Electra::FCodecTypeFormat::FSubtitle& csub = OutCodecInfo.Properties.Get<Electra::FCodecTypeFormat::FSubtitle>();
				// Add the sample box itself as an extra box.
				OutCodecInfo.ExtraBoxes.Emplace(SubtitleType, Subtitles[0]->GetBoxData());
				for(auto& Ch : Subtitles[0]->GetChildren())
				{
					OutCodecInfo.ExtraBoxes.Emplace(Ch->GetType(), Ch->GetBoxData());
				}
				if (!ElectraDecodersUtil::PrepareCodecTypeFormat(OutCodecInfo))
				{
					OutMessage = FString::Printf( TEXT("Failed to prepare the codec format type, ignoring track #%u."), InTrackID);
					return false;
				}
			}
			else if (SampleType == MP4Boxes::FMP4BoxSampleEntry::ESampleType::QTFFTimecode)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxQTFFTimecodeSampleEntry, ESPMode::ThreadSafe>> Timecodes;
				Stsd->GetSampleDescriptions(Timecodes);
				if (Timecodes.Num() != 1)
				{
					if (Timecodes.IsEmpty())
					{
						OutMessage = FString::Printf(TEXT("No timecode sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					else
					{
						OutMessage = FString::Printf(TEXT("More than one timecode sample entry found in `stsd` box, ignoring track #%u."), InTrackID);
					}
					return false;
				}

				switch(Timecodes[0]->GetType())
				{
					case MP4Utilities::MakeBoxAtom('t','m','c','d'):
					{
						Electra::FCodecTypeFormat::FTMCDTimecode Timecode;

						Timecode.Flags = Timecodes[0]->GetFlags();
						Timecode.Timescale = Timecodes[0]->GetTimescale();
						Timecode.FrameDuration = Timecodes[0]->GetFrameDuration();
						Timecode.NumberOfFrames = Timecodes[0]->GetNumberOfFrames();
						OutCodecInfo.Properties.Emplace<Electra::FCodecTypeFormat::FTMCDTimecode>(MoveTemp(Timecode));
						OutCodecInfo.RFC6381 = MP4Utilities::GetPrintableBoxAtom(Timecodes[0]->GetType());
						OutCodecInfo.FourCC = Timecodes[0]->GetType();
						OutCodecInfo.DCR = Timecodes[0]->GetBoxData();
						OutCodecInfo.Type = Electra::FCodecTypeFormat::EType::Timecode;
						AddChildren(Timecodes[0]);
						break;
					}
				}
			}

			// If the track has not been identified as usable so far, remove it.
			if (OutCodecInfo.Type == Electra::FCodecTypeFormat::EType::Invalid)
			{
				TArray<TSharedPtr<MP4Boxes::FMP4BoxSampleEntry, ESPMode::ThreadSafe>> Entries;
				Stsd->GetSampleDescriptions(Entries);
				check(Entries.Num() == 1);
				OutMessage = FString::Printf(TEXT("Track of sample type \"%s\" is not supported, ignoring track #%u."), *MP4Utilities::GetPrintableBoxAtom(Entries[0]->GetType()), InTrackID);
				return false;
			}
			return true;
		}

	} // namespace MP4

} // namespace ElectraDecodersUtil
