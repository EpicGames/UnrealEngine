// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraUtilsMPEGAudio.h"
#include "Utilities/ElectraBitstream.h"

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		bool FESDescriptor::Parse(const TConstArrayView<uint8>& InESDS)
		{
			class FMP4BitReader : public Electra::FBitstreamReader
			{
			public:
				FMP4BitReader(const void* pData, int64 nData) : FBitstreamReader(pData, nData)
				{
				}
				int32 ReadMP4Length()
				{
					int32 Length = 0;
					for (int32 i = 0; i < 4; ++i)
					{
						uint32 Bits = GetBits(8);
						Length = (Length << 7) + (Bits & 0x7f);
						if ((Bits & 0x80) == 0)
							break;
					}
					return Length;
				}
			};
			FMP4BitReader BitReader(InESDS.GetData(), InESDS.Num());

			CSD.Empty();

			if (BitReader.GetBits(8) != 3)
			{
				return false;
			}

			int32 ESSize = BitReader.ReadMP4Length();

			ESID = (uint16)BitReader.GetBits(16);
			bDependsOnStream = BitReader.GetBits(1) != 0;
			bool bURLFlag = BitReader.GetBits(1) != 0;
			bool bOCRflag = BitReader.GetBits(1) != 0;
			StreamPriority = (uint8)BitReader.GetBits(5);
			if (bDependsOnStream)
			{
				DependsOnStreamESID = BitReader.GetBits(16);
			}
			if (bURLFlag)
			{
				// Skip over the URL
				uint32 urlLen = BitReader.GetBits(8);
				BitReader.SkipBytes(urlLen);
			}
			if (bOCRflag)
			{
				// Skip the OCR ES ID
				BitReader.SkipBits(16);
			}

			// Parse the config descriptor
			if (BitReader.GetBits(8) != 4)
			{
				return false;
			}
			int32 ConfigDescrSize = BitReader.ReadMP4Length();
			ObjectTypeID = static_cast<FObjectTypeID>(BitReader.GetBits(8));
			StreamTypeID = static_cast<FStreamType>(BitReader.GetBits(6));
			// Skip upstream flag
			BitReader.SkipBits(1);
			// Reserved '1'
			BitReader.SkipBits(1);	// this bit must be 1, but it is sometimes incorrectly set to 0, so we do not check it.
			BufferSize = BitReader.GetBits(24);
			MaxBitrate = BitReader.GetBits(32);
			AvgBitrate = BitReader.GetBits(32);
			if (ConfigDescrSize > 13)
			{
				// Optional codec specific descriptor
				if (BitReader.GetBits(8) != 5)
				{
					return false;
				}
				int32 CodecSize = BitReader.ReadMP4Length();
				CSD.Reserve(CodecSize);
				for (int32 i = 0; i < CodecSize; ++i)
				{
					CSD.Push(BitReader.GetBits(8));
				}
			}

			// SL config (we do not need it, we require it to be there though as per the standard)
			if (BitReader.GetBits(8) != 6)
			{
				return false;
			}
			int32 nSLSize = BitReader.ReadMP4Length();
			if (nSLSize != 1)
			{
				return false;
			}
			if (BitReader.GetBits(8) != 2)
			{
				return false;
			}

			return true;
		}





		namespace AACUtils
		{
			static uint32 GIndexToSampleRate[16] =
			{
				96000, 88200, 64000, 48000, 44100, 32000, 24000, 22050, 16000, 12000, 11025, 8000, 7350, 0, 0, 0
			};

			int32 GetNumberOfChannelsFromChannelConfiguration(uint32 InChannelConfiguration)
			{
				static const uint8 NumChannelsForConfig[16] = { 0, 1, 2, 3, 4, 5, 6, 7, 0, 0, 0, 7, 8, 0, 8, 0 };
				return InChannelConfiguration < 16 ? NumChannelsForConfig[InChannelConfiguration] : 0;
			}

			int32 GetSampleRateFromFrequenceIndex(uint32 InSamplingFrequencyIndex)
			{
				return InSamplingFrequencyIndex < 13 ? GIndexToSampleRate[InSamplingFrequencyIndex] : 0;
			}

			bool ParseADTSHeader(ADTSheader& OutHeader, const TConstArrayView<uint8>& InHeaderData)
			{
				if (InHeaderData.Num() < 7)
				{
					return false;
				}

				Electra::FBitstreamReader bs(InHeaderData.GetData(), InHeaderData.Num());
				OutHeader.SyncWord = bs.GetBits(12);
				OutHeader.Version = bs.GetBits(1);
				OutHeader.Layer = bs.GetBits(2);
				OutHeader.ProtectionAbsent = bs.GetBits(1);
				OutHeader.Profile = bs.GetBits(2);
				OutHeader.SamplingFrequencyIndex = bs.GetBits(4);
				OutHeader.PrivateStream = bs.GetBits(1);
				OutHeader.ChannelConfiguration = bs.GetBits(3);
				OutHeader.Originality = bs.GetBits(1);
				OutHeader.Home = bs.GetBits(1);
				OutHeader.Copyrighted = bs.GetBits(1);
				OutHeader.CopyrightStart = bs.GetBits(1);
				OutHeader.FrameLength = bs.GetBits(13);
				OutHeader.BufferFullness = bs.GetBits(11);
				OutHeader.NumFrames = bs.GetBits(2);
				OutHeader.HeaderSize = 7;
				if (InHeaderData.Num() >= 9 && !OutHeader.ProtectionAbsent)
				{
					OutHeader.CRCifPresent = bs.GetBits(16);
					OutHeader.HeaderSize += 2;
				}
				return OutHeader.SyncWord == 0xfff && OutHeader.Layer == 0 && OutHeader.SamplingFrequencyIndex < 13;
			}

		}


		namespace AACParseHelper
		{
			static uint32 GetAudioObjectType(Electra::FBitstreamReader& Bitstream)
			{
				uint32 ObjectType = Bitstream.GetBits(5);
				if (ObjectType == 31)
				{
					ObjectType = 32 + Bitstream.GetBits(6);
				}
				return ObjectType;
			}
			static uint32 GetSamplingRateIndex(Electra::FBitstreamReader& Bitstream, uint32& Rate)
			{
				uint32 SamplingFrequencyIndex = Bitstream.GetBits(4);
				if (SamplingFrequencyIndex == 15)
				{
					Rate = Bitstream.GetBits(24);
				}
				else
				{
					Rate = AACUtils::GIndexToSampleRate[SamplingFrequencyIndex];
				}
				return SamplingFrequencyIndex;
			}
			static void GetGASpecificConfig(Electra::FBitstreamReader& Bitstream, uint32 SamplingFrequencyIndex, uint32 ChannelConfiguration, uint32 AudioObjectType)
			{
				int32 FrameLengthFlag = Bitstream.GetBits(1);
				int32 DependsOnCoreDecoder = Bitstream.GetBits(1);
				if (DependsOnCoreDecoder)
				{
					Bitstream.SkipBits(14);		// coreCoderDelay
				}
				int32 ExtensionFlag = Bitstream.GetBits(1);
				if (ChannelConfiguration == 0)
				{
					// TODO:
				}
				if (AudioObjectType == 6 || AudioObjectType == 20)
				{
					Bitstream.SkipBits(3);	// layerNr
				}
				if (ExtensionFlag)
				{
					if (AudioObjectType == 22)
					{
						Bitstream.SkipBits(5);	// numOfSubFrame
						Bitstream.SkipBits(11);	// layer_length
					}
					if (AudioObjectType == 17 || AudioObjectType == 19 || AudioObjectType == 20 || AudioObjectType == 23)
					{
						Bitstream.SkipBits(1);	// aacSectionDataResilienceFlag
						Bitstream.SkipBits(1);	// aacScalefactorDataResilienceFlag;
						Bitstream.SkipBits(1);	// aacSpectralDataResilienceFlag;
					}
					int32 ExtensionFlag3 = Bitstream.GetBits(1);
					if (ExtensionFlag)
					{
						// TODO:
					}
				}
			}

		} // namespace AACParseHelper


		FAACDecoderConfigurationRecord::FAACDecoderConfigurationRecord()
		{
			Reset();
		}

		void FAACDecoderConfigurationRecord::Reset()
		{
			SBRSignal = -1;
			PSSignal = -1;
			ChannelConfiguration = 0;
			SamplingFrequencyIndex = 0;
			SamplingRate = 0;
			ExtSamplingFrequencyIndex = 0;
			ExtSamplingFrequency = 0;
			AOT = 0;
			ExtAOT = 0;
		}

		FString FAACDecoderConfigurationRecord::GetFormatInfo() const
		{
			FString fi;
			if (PSSignal > 0)
			{
				fi = TEXT("HE-AAC v2");
			}
			else if (SBRSignal > 0)
			{
				fi = TEXT("HE-AAC");
			}
			else
			{
				fi = TEXT("AAC");
			}
			return fi;
		}

		const TArray<uint8>& FAACDecoderConfigurationRecord::GetCodecSpecificData() const
		{
			return CodecSpecificData;
		}

		bool FAACDecoderConfigurationRecord::ParseFrom(const void* Data, int64 Size)
		{
			CodecSpecificData.Empty();
			CodecSpecificData.SetNumUninitialized(Size);
			FMemory::Memcpy(CodecSpecificData.GetData(), Data, Size);

			Electra::FBitstreamReader bsp(Data, Size);
			SBRSignal = -1;
			PSSignal = -1;
			SamplingRate = 0;
			ExtSamplingFrequency = 0;
			ExtSamplingFrequencyIndex = 0;
			ExtAOT = 0;
			AOT = AACParseHelper::GetAudioObjectType(bsp);
			SamplingFrequencyIndex = AACParseHelper::GetSamplingRateIndex(bsp, SamplingRate);
			ChannelConfiguration = bsp.GetBits(4);

			if (AOT == 5 /*SBR*/ || AOT == 29 /*PS*/)
			{
				ExtAOT = AOT;
				SBRSignal = 1;
				if (AOT == 29)
				{
					PSSignal = 1;
				}

				ExtSamplingFrequencyIndex = AACParseHelper::GetSamplingRateIndex(bsp, ExtSamplingFrequency);
				AOT = AACParseHelper::GetAudioObjectType(bsp);
			}
			// Handle supported AOT configs
			if (AOT == 2 /*LC*/)		// Only LC for now
			{
				AACParseHelper::GetGASpecificConfig(bsp, SamplingFrequencyIndex, ChannelConfiguration, AOT);
			}
			// Would need to handle epConfig here now for a couple of AOT's we are NOT supporting
			// ...

			// Check for backward compatible SBR signaling.
			if (ExtAOT != 5)
			{
				while (bsp.GetRemainingBits() > 15)
				{
					int32 syncExtensionType = bsp.PeekBits(11);
					if (syncExtensionType == 0x2b7)
					{
						bsp.SkipBits(11);
						ExtAOT = AACParseHelper::GetAudioObjectType(bsp);
						if (ExtAOT == 5)
						{
							SBRSignal = bsp.GetBits(1);
							if (SBRSignal)
							{
								ExtSamplingFrequencyIndex = AACParseHelper::GetSamplingRateIndex(bsp, ExtSamplingFrequency);
							}
						}
						if (bsp.GetRemainingBits() > 11 && bsp.GetBits(11) == 0x548)
						{
							PSSignal = bsp.GetBits(1);
						}
						break;
					}
					else
					{
						bsp.SkipBits(1);
					}
				}
			}

			//int64 remainingBits = bsp.GetRemainingBits();
			return true;
		}




		namespace UtilsMPEG123
		{
			bool HasValidSync(uint32 InFrameHeader)
			{
				return (InFrameHeader & 0xffe00000U) == 0xffe00000U;
			}

			int32 GetVersionId(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 19) & 3U);
			}

			int32 GetLayerIndex(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 17) & 3U);
			}

			int32 GetBitrateIndex(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 12) & 15U);
			}

			int32 GetSamplingRateIndex(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 10) & 3U);
			}

			int32 GetChannelMode(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 6) & 3U);
			}

			int32 GetNumPaddingBytes(uint32 InFrameHeader)
			{
				return (int32) ((InFrameHeader >> 9) & 1U);
			}

			int32 GetVersion(uint32 InFrameHeader)
			{
				switch(GetVersionId(InFrameHeader))
				{
					default:
					{
						return 0;
					}
					case 0:
					{
						return 3;
					}
					case 2:
					{
						return 2;
					}
					case 3:
					{
						return 1;
					}
				}
			}


			int32 GetLayer(uint32 InFrameHeader)
			{
				switch(GetLayerIndex(InFrameHeader))
				{
					default:
					{
						return 0;
					}
					case 1:
					{
						return 3;
					}
					case 2:
					{
						return 2;
					}
					case 3:
					{
						return 1;
					}
				}
			}

			int32 GetBitrate(uint32 InFrameHeader)
			{
				static const int16 kBitrateTableMPEG1[3][16] =
				{
					{ 0, 32, 64, 96, 128, 160, 192, 224, 256, 288, 320, 352, 384, 416, 448, -1 },
					{ 0, 32, 48, 56, 64, 80, 96, 112, 128, 160, 192, 224, 256, 320, 384, -1 },
					{ 0, 32, 40, 48, 56, 64, 80, 96, 112, 128, 160, 192, 224, 256, 320, -1 }
				};
				static const int16 kBitrateTableMPEG2[3][16] =
				{
					{ 0, 32, 48, 56, 64, 80, 96, 112, 128, 144, 160, 176, 192, 224, 256, -1 },
					{ 0, 8, 16, 24, 32, 40, 48, 56, 64, 80, 96, 112, 128, 144, 160 , -1 },
					{ 0, 8, 16, 24, 32, 40, 48, 56, 64, 80, 96, 112, 128, 144, 160 , -1 }
				};
				int32 Version = GetVersion(InFrameHeader);
				int32 Layer = GetLayer(InFrameHeader);
				int32 BitrateIndex = GetBitrateIndex(InFrameHeader);
				if (Version == 0 || Layer == 0 || BitrateIndex == 15)
				{
					return -1;
				}
				int32 bps = Version == 1 ? kBitrateTableMPEG1[Layer - 1][BitrateIndex] : kBitrateTableMPEG2[Layer - 1][BitrateIndex];
				return bps * 1000;
			}

			int32 GetSamplingRate(uint32 InFrameHeader)
			{
				static const int32 kSamplingRates[4][4] =
				{
					{ -1, -1, -1, -1 },				// invalid
					{ 44100, 48000, 32000, -1 },	// MPEG 1
					{ 22050, 24000, 16000, -1 },	// MPEG 2
					{ 11025, 12000,  8000, -1 }		// MPEG 2.5
				};
				int32 Version = GetVersion(InFrameHeader);
				int32 SampleRateIndex = GetSamplingRateIndex(InFrameHeader);
				return kSamplingRates[Version][SampleRateIndex];
			}
			int32 GetChannelCount(uint32 InFrameHeader)
			{
				return GetChannelMode(InFrameHeader) == 3 ? 1 : 2;
			}

			int32 GetFrameSize(uint32 InFrameHeader, int32 InForcedPadding)
			{
				static const int32 kNumCoeffs[2][3] =
				{
					{ 12, 144, 144 },		// MPEG 1 (layer 1, 2, 3)
					{ 12, 144, 72 }			// MPEG 2 / 2.5 (layer 1, 2, 3)
				};
				static const int32 kSlotSize[3] =
				{
					4, 1, 1					// Layer 1, 2, 3
				};

				int32 NumPadding = InForcedPadding < 0 ? GetNumPaddingBytes(InFrameHeader) : InForcedPadding;
				int32 Version = GetVersion(InFrameHeader);
				int32 Layer = GetLayer(InFrameHeader);
				int32 Bitrate = GetBitrate(InFrameHeader);
				int32 SampleRate = GetSamplingRate(InFrameHeader);
				if (Version == 0 || Layer == 0 || Bitrate <= 0 || SampleRate <= 0)
				{
					return 0;
				}
				int32 FrameSize = (kNumCoeffs[Version==1?0:1][Layer-1] * Bitrate / SampleRate + NumPadding) * kSlotSize[Layer-1];
				return FrameSize;
			}

			int32 GetSamplesPerFrame(uint32 InFrameHeader)
			{
				static const int32 kSamplesPerFrame[2][4] =
				{
					{ 0, 384, 1152, 1152 },	// MPEG 1 (layer 1, 2, 3)
					{ 0, 384, 1152, 576 }	// MPEG 2 / 2.5 (layer 1, 2, 3)
				};
				int32 Version = GetVersion(InFrameHeader);
				int32 Layer = GetLayer(InFrameHeader);
				return kSamplesPerFrame[Version==1?0:1][Layer];
			}

		} // namespace UtilsMPEG123


	} // namespace MPEG
} // namespace ElectraDecodersUtil
