// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "Utilities/ElectraBitstream.h"
#include "MP4Utilities.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "ElectraDecodersUtils.h"

namespace ElectraDecodersUtil
{
	namespace MPEG
	{




		template <typename T>
		struct FScopedDataPtr
		{
			FScopedDataPtr(void* Addr)
				: Data(static_cast<T*>(Addr))
			{
			}
			~FScopedDataPtr()
			{
				FMemory::Free(static_cast<void*>(Data));
			}
			operator T* ()
			{
				return Data;
			}
			T* Data;
		};





		bool ExtractSEIMessages(TArray<FSEIMessage>& OutMessages, const void* InBitstream, uint64 InBitstreamLength, ESEIStreamType StreamType, bool bIsPrefixSEI)
		{
			auto EBSPtoRBSP = [](uint8* OutBuf, const uint8* InBuf, int32 NumBytesIn) -> int32
			{
				uint8* OutBase = OutBuf;
				while(NumBytesIn-- > 0)
				{
					uint8 b = *InBuf++;
					*OutBuf++ = b;
					if (b == 0)
					{
						if (NumBytesIn > 1)
						{
							if (InBuf[0] == 0x00 && InBuf[1] == 0x03)
							{
								*OutBuf++ = 0x00;
								InBuf += 2;
								NumBytesIn -= 2;
							}
						}
					}
				}
				return OutBuf - OutBase;
			};

			FScopedDataPtr<uint8> RBSP(FMemory::Malloc((uint32) InBitstreamLength));
			int32 RBSPsize = EBSPtoRBSP(RBSP, static_cast<const uint8*>(InBitstream), (int32)InBitstreamLength);
			Electra::FBitstreamReader BitReader(RBSP, RBSPsize);

			while(BitReader.GetRemainingByteLength() > 0)
			{
				uint32 next8;
				uint32 payloadType = 0;
				do
				{
					if (BitReader.GetRemainingByteLength() == 0)
					{
						return false;
					}
					next8 = BitReader.GetBits(8);
					payloadType += next8;
				}
				while(next8 == 255);

				uint32 payloadSize = 0;
				do
				{
					if (BitReader.GetRemainingByteLength() == 0)
					{
						return false;
					}
					next8 = BitReader.GetBits(8);
					payloadSize += next8;
				}
				while(next8 == 255);

				check(BitReader.GetRemainingByteLength() >= payloadSize);
				if (BitReader.GetRemainingByteLength() < payloadSize)
				{
					return false;
				}
				FSEIMessage &m = OutMessages.Emplace_GetRef();
				m.PayloadType = payloadType;

				m.Message.Reserve(payloadSize);
				m.Message.SetNumUninitialized(payloadSize);
				if (!BitReader.GetAlignedBytes(m.Message.GetData(), payloadSize))
				{
					return false;
				}

				/*
					We do not parse the SEI messages here, we merely copy their entire payload.
					Therefore, the current position in the bit reader will always be byte-aligned
					with no yet-unhandled bytes remaining in the payload.
					The following checks that are normally in place can therefore be ignored:

						H.265:
							if (more_data_in_payload())		// more_data_in_payload() return false if byte aligned AND all bits consumed, otherwise return true
							{
								if (payload_extension_present())
								{
									reserved_payload_extension_data	// u_v()
								}
								payload_bit_equal_to_one()
								while(!byte_aligned())
								{
									payload_bit_equal_to_zero()
								}
							}

						H.264:
							if (!byte_aligned())
							{
								bit_equal_to_one()
								while(!byte_aligned())
								{
									bit_equal_to_zero()
								}
							}
				*/

				// Check for more_rbsp_data()
				if (BitReader.GetRemainingByteLength() == 1 && BitReader.PeekBits(8) == 0x80)
				{
					break;
				}
			}
			return true;
		}


		bool ParseFromSEIMessage(FSEImastering_display_colour_volume& OutMDCV, const FSEIMessage& InMessage)
		{
			if (InMessage.PayloadType == FSEIMessage::EPayloadType::PT_mastering_display_colour_volume && InMessage.Message.Num() >= 24)
			{
				Electra::FBitstreamReader BitReader(InMessage.Message.GetData(), InMessage.Message.Num());
				for(int32 i=0; i<3; ++i)
				{
					OutMDCV.display_primaries_x[i] = BitReader.GetBits(16);
					OutMDCV.display_primaries_y[i] = BitReader.GetBits(16);
				}
				OutMDCV.white_point_x = BitReader.GetBits(16);
				OutMDCV.white_point_y = BitReader.GetBits(16);
				OutMDCV.max_display_mastering_luminance = BitReader.GetBits(32);
				OutMDCV.min_display_mastering_luminance = BitReader.GetBits(32);
				return true;
			}
			return false;
		}

		bool ParseFromMDCVBox(FSEImastering_display_colour_volume& OutMDCV, const TConstArrayView<uint8>& InMDCVBox)
		{
			if (InMDCVBox.IsEmpty())
			{
				return false;
			}
			MP4Utilities::FMP4AtomReader boxReader(InMDCVBox);
			for(int32 i=0; i<3; ++i)
			{
				if (!boxReader.Read(OutMDCV.display_primaries_x[i]) || !boxReader.Read(OutMDCV.display_primaries_y[i]))
				{
					return false;
				}
			}
			if (!boxReader.Read(OutMDCV.white_point_x) || !boxReader.Read(OutMDCV.white_point_y))
			{
				return false;
			}
			if (!boxReader.Read(OutMDCV.max_display_mastering_luminance) || !boxReader.Read(OutMDCV.min_display_mastering_luminance))
			{
				return false;
			}
			return true;
		}
		TArray<uint8> ELECTRADECODERS_API BuildMDCVBox(const FSEImastering_display_colour_volume& InFromDisplayColorVolume)
		{
			Electra::FBitstreamWriter bw;
			// The order in the MDCV box is G,B,R. Since we assume that in this structure the order is RGB we swap it.
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_x[1], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_y[1], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_x[2], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_y[2], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_x[0], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.display_primaries_y[0], 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.white_point_x, 16);
			bw.PutBits((uint16)InFromDisplayColorVolume.white_point_y, 16);
			bw.PutBits((uint32)InFromDisplayColorVolume.max_display_mastering_luminance, 32);
			bw.PutBits((uint32)InFromDisplayColorVolume.min_display_mastering_luminance, 32);
			TArray<uint8> Box;
			bw.GetArray(Box);
			return Box;
		}


		bool ParseFromSEIMessage(FSEIcontent_light_level_info& OutCLLI, const FSEIMessage& InMessage)
		{
			if (InMessage.PayloadType == FSEIMessage::EPayloadType::PT_content_light_level_info && InMessage.Message.Num() >= 4)
			{
				Electra::FBitstreamReader BitReader(InMessage.Message.GetData(), InMessage.Message.Num());
				OutCLLI.max_content_light_level = BitReader.GetBits(16);
				OutCLLI.max_pic_average_light_level = BitReader.GetBits(16);
				return true;
			}
			return false;
		}
		bool ParseFromCOLLBox(FSEIcontent_light_level_info& OutCLLI, const TConstArrayView<uint8>& InCOLLBox)
		{
			if (InCOLLBox.Num() > 4)
			{
				uint8 BoxVersion = InCOLLBox[0];
				if (BoxVersion == 0)
				{
					// 'clli' box is the same as a version 0 'COLL 'coll' box.
					return ParseFromCLLIBox(OutCLLI, MakeConstArrayView(InCOLLBox.GetData() + 4, InCOLLBox.Num() - 4));
				}
			}
			return false;
		}
		bool ParseFromCLLIBox(FSEIcontent_light_level_info& OutCLLI, const TConstArrayView<uint8>& InCLLIBox)
		{
			static_assert(sizeof(FSEIcontent_light_level_info::max_content_light_level) == 2);
			static_assert(sizeof(FSEIcontent_light_level_info::max_pic_average_light_level) == 2);
			if (InCLLIBox.IsEmpty())
			{
				return false;
			}
			MP4Utilities::FMP4AtomReader boxReader(InCLLIBox);
			OutCLLI.max_content_light_level = 0;			// MaxCLL
			OutCLLI.max_pic_average_light_level = 0;		// MaxFALL
			if (!boxReader.Read(OutCLLI.max_content_light_level) || !boxReader.Read(OutCLLI.max_pic_average_light_level))
			{
				return false;
			}
			return true;
		}
		TArray<uint8> BuildCLLIBox(const FSEIcontent_light_level_info& InFromContentLightLevel)
		{
			Electra::FBitstreamWriter bw;
			bw.PutBits((uint16)InFromContentLightLevel.max_content_light_level, 16);
			bw.PutBits((uint16)InFromContentLightLevel.max_pic_average_light_level, 16);
			TArray<uint8> Box;
			bw.GetArray(Box);
			return Box;
		}




		bool ParseFromSEIMessage(FSEIalternative_transfer_characteristics& OutATC, const FSEIMessage& InMessage)
		{
			if (InMessage.PayloadType == FSEIMessage::EPayloadType::PT_alternative_transfer_characteristics && InMessage.Message.Num() >= 1)
			{
				Electra::FBitstreamReader BitReader(InMessage.Message.GetData(), InMessage.Message.Num());
				OutATC.preferred_transfer_characteristics = BitReader.GetBits(8);
				return true;
			}
			return false;
		}

		bool ParseFromCOLRBox(FCommonColorimetry& OutColorimetry, const TConstArrayView<uint8>& InCOLRBox)
		{
			if (InCOLRBox.Num())
			{
				MP4Utilities::FMP4AtomReader boxReader(InCOLRBox);
				uint32 Type;
				if (boxReader.Read(Type))
				{
					if (Type == MP4Utilities::MakeBoxAtom('n','c','l','x') || Type == MP4Utilities::MakeBoxAtom('n', 'c', 'l', 'c'))
					{
						uint16 colour_primaries, transfer_characteristics, matrix_coeffs;
						uint8 video_full_range_flag = 0;
						if (boxReader.Read(colour_primaries) && boxReader.Read(transfer_characteristics) && boxReader.Read(matrix_coeffs))
						{
							if (Type == MP4Utilities::MakeBoxAtom('n','c','l','x') && !boxReader.Read(video_full_range_flag))
							{
								return false;
							}
							OutColorimetry.colour_primaries = (uint8) colour_primaries;
							OutColorimetry.transfer_characteristics = (uint8)transfer_characteristics;
							OutColorimetry.matrix_coeffs = (uint8)matrix_coeffs;
							OutColorimetry.video_full_range_flag = (uint8)video_full_range_flag;
							OutColorimetry.video_format = 5;	// Unspecified video format
							return true;
						}
					}
				}
			}
			return false;
		}

		TArray<uint8> ELECTRADECODERS_API BuildCOLRBox(const FCommonColorimetry& InFromColorimetry)
		{
			Electra::FBitstreamWriter bw;
			bw.PutBits(MP4Utilities::MakeBoxAtom('n','c','l','x'), 32);
			bw.PutBits((uint16)InFromColorimetry.colour_primaries, 16);
			bw.PutBits((uint16)InFromColorimetry.transfer_characteristics, 16);
			bw.PutBits((uint16)InFromColorimetry.matrix_coeffs, 16);
			bw.PutBits(InFromColorimetry.video_full_range_flag, 1);
			bw.PutBits(0, 7);
			TArray<uint8> Box;
			bw.GetArray(Box);
			return Box;
		}


	} // namespace MPEG
} // namespace ElectraDecodersUtil
