// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraUtilsMPEGVideo_H264.h"
#include "Utilities/ElectraBitstream.h"
#include "ElectraDecodersUtils.h"
#include "ElectraDecodersModule.h"

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		namespace H264
		{
			// Rec. ITU-T H.264 (08/2021) Table 7-3
			static const uint8 Default_4x4_Intra[16] = { 6, 13, 13, 20, 20, 20, 28, 28, 28, 28, 32, 32, 32, 37, 37, 42 };
			static const uint8 Default_4x4_Inter[16] = { 10, 14, 14, 20, 20, 20, 24, 24, 24, 24, 27, 27, 27, 30, 30, 34 };
			static const uint8 Default_8x8_Intra[64] = { 6, 10, 10, 13, 11, 13, 16, 16, 16, 16, 18, 18, 18, 18, 18, 23,
														23, 23, 23, 23, 23, 25, 25, 25, 25, 25, 25, 25, 27, 27, 27, 27,
														27, 27, 27, 27, 29, 29, 29, 29, 29, 29, 29, 31, 31, 31, 31, 31,
														31, 33, 33, 33, 33, 33, 36, 36, 36, 36, 38, 38, 38, 40, 40, 42 };
			static const uint8 Default_8x8_Inter[64] = { 9, 13, 13, 15, 13, 15, 17, 17, 17, 17, 19, 19, 19, 19, 19, 21,
														21, 21, 21, 21, 21, 22, 22, 22, 22, 22, 22, 22, 24, 24, 24, 24,
														24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 27, 27, 27, 27, 27,
														27, 28, 28, 28, 28, 28, 30, 30, 30, 30, 32, 32, 32, 33, 33, 35 };
			// Rec. ITU-T H.264 (08/2021) Table 8-13
			// Cij calculates as j*4+i
			static const uint8 ScanOrderZZ4[16] = { 0, 1, 4, 8, 5, 2, 3, 6, 9, 12, 13, 10, 7, 11, 14, 15 };
			// Rec. ITU-T H.264 (08/2021) Table 8-14
			// Cij calculates as j*8+i
			static const uint8 ScanOrderZZ8[64] = { 0, 1, 8, 16, 9, 2, 3, 10, 17, 24, 32, 25, 18, 11, 4, 5,
													12, 19, 26, 33, 40, 48, 41, 34, 27, 20, 13, 6, 7, 14, 21, 28,
													35, 42, 49, 56, 57, 50, 43, 36, 29, 22, 15, 23, 30, 37, 44, 51,
													58, 59, 52, 45, 38, 31, 39, 46, 53, 60, 61, 54, 47, 55, 62, 63 };


			uint64 EBSPtoRBSP(TArray<uint64>& OutRemovedInBufPos, uint8* OutBuf, const uint8* InBuf, uint64 InNumBytes)
			{
				// Scan the input in steps of 3.
				const uint8* OutBase = OutBuf;
				uint64 Prev=0;
				for(int64 i=0,iMax=(int64)InNumBytes-2; i<iMax; )
				{
					// If the 3rd byte is larger than a 3 then we are not in a '00 00 03' sequence.
					if (InBuf[i+2] > 3)
					{
						i += 3;
						continue;
					}
					if (InBuf[i+2] != 3)
					{
						++i;
						continue;
					}
					// Sequence found?
					if (InBuf[i] == 0 && InBuf[i+1] == 0)
					{
						const uint32 np = i+2;
						OutRemovedInBufPos.Emplace(np);
						uint32 nb = np-Prev;
						FMemory::Memcpy(OutBuf, InBuf + Prev, nb);
						OutBuf += nb;
						Prev = np+1;
					}
					i += 3;
				}
				if (Prev < InNumBytes)
				{
					uint32 nb = InNumBytes-Prev;
					FMemory::Memcpy(OutBuf, InBuf + Prev, nb);
					OutBuf += nb;
				}
				return OutBuf - OutBase;
			}
			TUniquePtr<FRBSP> MakeRBSP(const uint8* InData, uint64 InDataSize)
			{
				auto Ptr = MakeUnique<FRBSP>(InDataSize);
				uint64 AdjustedSize = EBSPtoRBSP(Ptr->RemovedAtSrc, Ptr->Data.GetData(), InData, InDataSize);
				Ptr->Data.SetNum((int32)AdjustedSize);
				return Ptr;
			}

			bool ParseBitstreamForNALUs(TArray<FNaluInfo>& OutNALUs, const uint8* InBitstream, uint64 InBitstreamLength)
			{
				if (!InBitstream)
				{
					return false;
				}
				else if (InBitstreamLength < 3)
				{
					UE_LOGF(LogElectraDecoders, Error, "Bitstream too short to parse for NALUs");
					return false;
				}
				// Skip over stream in intervals of 3 until InBitstream[i + 2] is either 0 or 1
				uint64 i=0;
				for(uint64 iMax=InBitstreamLength-2; i<iMax; )
				{
					if (InBitstream[i+2] > 1)
					{
						i += 3;
					}
					else if (InBitstream[i+2] == 1)
					{
						if (InBitstream[i+1] == 0 && InBitstream[i] == 0)
						{
							// Found start sequence but we don't know if it has a 3 or 4 byte start code so we check.
							const bool bOff1 = i > 0 && InBitstream[i-1] == 0;
							uint64 Offset = bOff1 ? i-1 : i;
							// Update length of previous entry.
							if (OutNALUs.Num())
							{
								OutNALUs.Last().Size = Offset - (OutNALUs.Last().Offset + OutNALUs.Last().UnitLength);
							}

							FNaluInfo& NalInfo = OutNALUs.Emplace_GetRef();
							NalInfo.Offset = Offset;
							NalInfo.UnitLength = bOff1 ? 4 : 3;
							uint8 nut = InBitstream[NalInfo.Offset + NalInfo.UnitLength];
							if ((nut & 0x80) != 0)
							{
								UE_LOGF(LogElectraDecoders, Error, "Forbidden zero bit in NAL header is not zero!");
								return false;
							}
							NalInfo.Type = nut & 31;
							NalInfo.RefIdc = (nut >> 5) & 3;
						}
						i += 3;
					}
					else
					{
						++i;
					}
				}
				if (OutNALUs.IsEmpty())
				{
					UE_LOGF(LogElectraDecoders, Error, "No NALUs found in bitstream!");
					return false;
				}
				OutNALUs.Last().Size = InBitstreamLength - (OutNALUs.Last().Offset + OutNALUs.Last().UnitLength);
				return true;
			}

			bool ParseScalingLists(const FSequenceParameterSet* InSPS, const FPictureParameterSet* InPPS, uint8 scaling_list_present_flag[12], uint8 ScalingList4x4[6][16], uint8 ScalingList8x8[6][64], FBitstreamReaderH264& br)
			{
				struct sl
				{
					static bool scaling_list(uint8* ScalingList, int32 sizeOfScalingList, const uint8* DefaultScalingMatrix, FBitstreamReaderH264& br)
					{
						const uint8* Scanninglist = sizeOfScalingList == 16 ? ScanOrderZZ4 : ScanOrderZZ8;
						int32 lastScale = 8;
						int32 nextScale = 8;
						for(int32 j=0; j<sizeOfScalingList; ++j)
						{
							int32 scanJ = Scanninglist[j];
							if (nextScale != 0)
							{
								int32 delta_scale = br.se_v();
								if (!(delta_scale >= -128 && delta_scale < 127))
								{
									UE_LOGF(LogElectraDecoders, Error, "delta_scale in scaling list is out of range");
									return false;
								}
								nextScale = (lastScale + delta_scale + 256) % 256;
								bool useDefaultScalingMatrixFlag = (scanJ==0 && nextScale==0);
								if (useDefaultScalingMatrixFlag)
								{
									// This can only happen on the first entry (scanJ == j == 0) and if
									// it does the loop won't do anything useful any more as nextScale is
									// and will remain 0.
									// So it is ok to copy the provided default list now and leave.
									FMemory::Memcpy(ScalingList, DefaultScalingMatrix, sizeOfScalingList);
									return true;
								}
							}
							ScalingList[scanJ] = nextScale == 0 ? lastScale : nextScale;
							lastScale = ScalingList[scanJ];
						}
						return true;
					}
					static bool parse_scaling_list(uint8* ScalingList, int32 sizeOfScalingList, const uint8* DefaultScalingMatrix, const uint8* FallbackScalingMatrix, FBitstreamReaderH264& br)
					{
						if (br.GetBits(1))
						{
							return scaling_list(ScalingList, sizeOfScalingList, DefaultScalingMatrix, br);
						}
						else
						{
							FMemory::Memcpy(ScalingList, FallbackScalingMatrix, sizeOfScalingList);
							return true;
						}
					}
				};

				bool bUseSPS = InPPS && InSPS->seq_scaling_matrix_present_flag != 0;
				// The first 6 entries are parsed unconditionally for both SPS and PPS.
				if (!InPPS)
				{
					if (!sl::parse_scaling_list(ScalingList4x4[0], sizeof(ScalingList4x4[0]), Default_4x4_Intra, Default_4x4_Intra, br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[1], sizeof(ScalingList4x4[1]), Default_4x4_Intra, ScalingList4x4[0], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[2], sizeof(ScalingList4x4[2]), Default_4x4_Intra, ScalingList4x4[1], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[3], sizeof(ScalingList4x4[3]), Default_4x4_Inter, Default_4x4_Inter, br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[4], sizeof(ScalingList4x4[4]), Default_4x4_Inter, ScalingList4x4[3], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[5], sizeof(ScalingList4x4[5]), Default_4x4_Inter, ScalingList4x4[4], br)) return false;
				}
				else
				{
					if (!sl::parse_scaling_list(ScalingList4x4[0], sizeof(ScalingList4x4[0]), Default_4x4_Intra, bUseSPS ? InSPS->ScalingList4x4[0] : Default_4x4_Intra, br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[1], sizeof(ScalingList4x4[1]), Default_4x4_Intra, ScalingList4x4[0], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[2], sizeof(ScalingList4x4[2]), Default_4x4_Intra, ScalingList4x4[1], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[3], sizeof(ScalingList4x4[3]), Default_4x4_Inter, bUseSPS ? InSPS->ScalingList4x4[3] : Default_4x4_Inter, br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[4], sizeof(ScalingList4x4[4]), Default_4x4_Inter, ScalingList4x4[3], br)) return false;
					if (!sl::parse_scaling_list(ScalingList4x4[5], sizeof(ScalingList4x4[5]), Default_4x4_Inter, ScalingList4x4[4], br)) return false;
				}
				// The remaining lists are conditional.
				if (!InPPS || InPPS->transform_8x8_mode_flag)
				{
					if (!InPPS)
					{
						if (!sl::parse_scaling_list(ScalingList8x8[0], sizeof(ScalingList8x8[0]), Default_8x8_Intra, Default_8x8_Intra, br)) return false; // Y Intra
						if (!sl::parse_scaling_list(ScalingList8x8[1], sizeof(ScalingList8x8[1]), Default_8x8_Inter, Default_8x8_Inter, br)) return false; // Y Inter
					}
					else
					{
						if (!sl::parse_scaling_list(ScalingList8x8[0], sizeof(ScalingList8x8[0]), Default_8x8_Intra, bUseSPS ? InSPS->ScalingList8x8[0] : Default_8x8_Intra, br)) return false; // Luma Intra
						if (!sl::parse_scaling_list(ScalingList8x8[1], sizeof(ScalingList8x8[1]), Default_8x8_Inter, bUseSPS ? InSPS->ScalingList8x8[1] : Default_8x8_Inter, br)) return false; // Luma Inter
					}
					// Additional chroma
					if (InSPS->chroma_format_idc == 3)
					{
						if (!sl::parse_scaling_list(ScalingList8x8[2], sizeof(ScalingList8x8[2]), Default_8x8_Intra, ScalingList8x8[0], br)) return false; // Cr Intra
						if (!sl::parse_scaling_list(ScalingList8x8[3], sizeof(ScalingList8x8[3]), Default_8x8_Inter, ScalingList8x8[1], br)) return false; // Cr Inter
						if (!sl::parse_scaling_list(ScalingList8x8[4], sizeof(ScalingList8x8[4]), Default_8x8_Intra, ScalingList8x8[2], br)) return false; // Cb Intra
						if (!sl::parse_scaling_list(ScalingList8x8[5], sizeof(ScalingList8x8[5]), Default_8x8_Inter, ScalingList8x8[3], br)) return false; // CB Inter
					}
				}
				return true;
			}


			int32 FSequenceParameterSet::GetMaxDPBSize() const
			{
				// From Table A.3.5 Table A-7
				int32 PicSizeMBs = (pic_width_in_mbs_minus1 + 1) * (pic_height_in_map_units_minus1 + 1) * (frame_mbs_only_flag ? 1:2);
				int32 Size = 0;
				switch(level_idc)
				{
					case 0: return 16;
					case 9: Size = 396; break;
					case 10: Size = 396; break;
					case 11: Size = (constraint_set3_flag && !(profile_idc==0 || profile_idc==44 || profile_idc==100 || profile_idc==110 || profile_idc==122 || profile_idc==244)) ? 396 : 900; break;
					case 12: Size = 2376; break;
					case 13: Size = 2376; break;
					case 20: Size = 2376; break;
					case 21: Size = 4752; break;
					case 22: Size = 8100; break;
					case 30: Size = 8100; break;
					case 31: Size = 18000; break;
					case 32: Size = 20480; break;
					case 40: Size = 32768; break;
					case 41: Size = 32768; break;
					case 42: Size = 34816; break;
					case 50: Size = 110400; break;
					case 51: Size = 184320; break;
					case 52: Size = 184320; break;
					case 60:
					case 61:
					case 62: Size = 696320; break;
					default:
						UE_LOGF(LogElectraDecoders, Error, "Invalid level value %u", level_idc);
						return -1;
				}
				Size /= PicSizeMBs;
				/*
				if (profile_idc == 118 || profile_idc == 128)
				{
					int num_views = active_subset_sps->num_views_minus1 + 1;
					Size = Min(2 * Size, Max(1, FMath::CeilLogTwo(num_views)) * 16) / num_views;
				}
				else
				*/
				{
					Size = Min(Size, 16);
				}
				return Size;
			}
			int32 FSequenceParameterSet::GetDPBSize() const
			{
				int32 MaxSize = GetMaxDPBSize();
				if (MaxSize <= 0)
				{
					return MaxSize;
				}
				if (vui_parameters_present_flag != 0 && bitstream_restriction_flag != 0)
				{
				/*
					Removed for now as this can happen despite it being invalid.

					if ((int32)max_dec_frame_buffering > MaxSize)
					{
						UE_LOGF(LogElectraDecoders, Error, "max_dec_frame_buffering in vui is larger than max dpb size for this level");
						return -1;
					}
				*/
					uint32 SizeFromVUI = Max(1U, max_dec_frame_buffering);
					MaxSize = (int32)SizeFromVUI;
				}
				return MaxSize;
			}
			int32 FSequenceParameterSet::GetWidth() const
			{
				return (pic_width_in_mbs_minus1 + 1) * 16;
			}
			int32 FSequenceParameterSet::GetHeight() const
			{
				return (pic_height_in_map_units_minus1 + 1) * 16;
			}
			void FSequenceParameterSet::GetDisplaySize(int32& OutWidth, int32& OutHeight) const
			{
				int32 cl,cr,ct,cb;
				GetCrop(cl,cr,ct,cb);
				OutWidth = GetWidth() - cl - cr;
				OutHeight = GetHeight() - ct - cb;
			}
			void FSequenceParameterSet::GetDisplaySize(uint32& OutWidth, uint32& OutHeight) const
			{
				int32 w,h;
				GetDisplaySize(w, h);
				OutWidth = (uint32) w;
				OutHeight = (uint32) h;
			}
			void FSequenceParameterSet::GetCrop(int32& OutLeft, int32& OutRight, int32& OutTop, int32& OutBottom) const
			{
				if (frame_cropping_flag)
				{
					// The scaling factors are determined by the chroma_format_idc (see ISO/IEC 14496-10, table 6.1)
					// For our purposes this will be 1, so the sub width/height are 2.
					const int32 CropUnitX = 2;
					const int32 CropUnitY = 2;
					OutLeft = (uint16)frame_crop_left_offset * CropUnitX;
					OutRight = (uint16)frame_crop_right_offset * CropUnitX;
					OutTop = (uint16)frame_crop_top_offset * CropUnitY;
					OutBottom = (uint16)frame_crop_bottom_offset * CropUnitY;
				}
				else
				{
					OutLeft = OutRight = OutTop = OutBottom = 0;
				}
			}
			void FSequenceParameterSet::GetAspect(int32& OutSarW, int32& OutSarH) const
			{
				if (vui_parameters_present_flag && aspect_ratio_info_present_flag)
				{
					switch (aspect_ratio_idc)
					{
						default:	OutSarW = OutSarH = 0;		break;
						case 0:		OutSarW = OutSarH = 0;		break;
						case 1:		OutSarW = OutSarH = 1;		break;
						case 2:		OutSarW = 12; OutSarH = 11;	break;
						case 3:		OutSarW = 10; OutSarH = 11;	break;
						case 4:		OutSarW = 16; OutSarH = 11;	break;
						case 5:		OutSarW = 40; OutSarH = 33;	break;
						case 6:		OutSarW = 24; OutSarH = 11;	break;
						case 7:		OutSarW = 20; OutSarH = 11;	break;
						case 8:		OutSarW = 32; OutSarH = 11;	break;
						case 9:		OutSarW = 80; OutSarH = 33;	break;
						case 10:	OutSarW = 18; OutSarH = 11;	break;
						case 11:	OutSarW = 15; OutSarH = 11;	break;
						case 12:	OutSarW = 64; OutSarH = 33;	break;
						case 13:	OutSarW = 160; OutSarH = 99;break;
						case 14:	OutSarW = 4; OutSarH = 3;	break;
						case 15:	OutSarW = 3; OutSarH = 2;	break;
						case 16:	OutSarW = 2; OutSarH = 1;	break;
						case 255:	OutSarW = sar_width; OutSarH = sar_height;	break;
					}
				}
				else
				{
					OutSarW = OutSarH = 1;
				}
			}
			FFractionalValue FSequenceParameterSet::GetTiming() const
			{
				if (vui_parameters_present_flag && timing_info_present_flag)
				{
					// Having just the SPS is not necessarily all that is needed to get the framerate as this
					// may require the pic_struct (from buffering SEI message) and field_pic_flag (from the slice).
					// Since we do not handle field encoded video we assume field_pic_flag to be 0.
					// If pic_struct_present_flag is zero then we use the timing info as it is given with a DeltaTfiDivisor of 2.
					// Otherwise, if pic_struct_present_flag is one then DeltaTfiDivisor is 2 only for pic_struct values of 0, 3 and 4.
					// Anyway, that is what we assume to handle, so there...
					const int32 DeltaTfiDivisor = 2;
					if (/*pic_struct_present_flag == 0 &&*/ time_scale && num_units_in_tick)
					{
						return FFractionalValue(time_scale, num_units_in_tick * DeltaTfiDivisor);
					}
				}
				return FFractionalValue();
			}

			FString FSequenceParameterSet::GetFormatInfo() const
			{
				return GetFormatInfo(profile_idc, level_idc, constraint_set0_flag | (constraint_set1_flag <<1) | (constraint_set2_flag << 2) | (constraint_set3_flag << 3) | (constraint_set4_flag << 4) | (constraint_set5_flag << 5));
			}
			FString FSequenceParameterSet::GetFormatInfo(uint32 InProfileIDC, uint32 InLevelIDC, uint32 InConstraintFlags)
			{
				FString fi;
				if (InProfileIDC == 44)
				{
					fi = TEXT("CAVLC 4:4:4 Intra");
				}
				else if (InProfileIDC == 66 && (InConstraintFlags & 2) == 2)
				{
					fi = TEXT("Constrained Baseline");
				}
				else if (InProfileIDC == 66)
				{
					fi = TEXT("Baseline");
				}
				else if (InProfileIDC == 77)
				{
					fi = TEXT("Main");
				}
				else if (InProfileIDC == 83 && (InConstraintFlags & 32) == 32)
				{
					fi = TEXT("Scalable Constrained Baseline");
				}
				else if (InProfileIDC == 83)
				{
					fi = TEXT("Scalable Baseline");
				}
				else if (InProfileIDC == 86 && (InConstraintFlags & 32) == 32)
				{
					fi = TEXT("Scalable Constrained High");
				}
				else if (InProfileIDC == 86 && (InConstraintFlags & 8) == 8)
				{
					fi = TEXT("Scalable High Intra");
				}
				else if (InProfileIDC == 86)
				{
					fi = TEXT("Scalable High");
				}
				else if (InProfileIDC == 88)
				{
					fi = TEXT("Extended");
				}
				else if (InProfileIDC == 100 && (InConstraintFlags & 48) == 48)
				{
					fi = TEXT("Constrained High");
				}
				else if (InProfileIDC == 100 && (InConstraintFlags & 16) == 16)
				{
					fi = TEXT("Progressive High");
				}
				else if (InProfileIDC == 100)
				{
					fi = TEXT("High");
				}
				else if (InProfileIDC == 110 && (InConstraintFlags & 8) == 8)
				{
					fi = TEXT("High 10 Intra");
				}
				else if (InProfileIDC == 110)
				{
					fi = TEXT("High 10");
				}
				else if (InProfileIDC == 118)
				{
					fi = TEXT("Multiview High");
				}
				else if (InProfileIDC == 122 && (InConstraintFlags & 8) == 8)
				{
					fi = TEXT("High 4:2:2 Intra");
				}
				else if (InProfileIDC == 122)
				{
					fi = TEXT("High 4:2:2");
				}
				else if (InProfileIDC == 128)
				{
					fi = TEXT("Stereo High");
				}
				else if (InProfileIDC == 134)
				{
					fi = TEXT("MFC High");
				}
				else if (InProfileIDC == 135)
				{
					fi = TEXT("MFC Depth High");
				}
				else if (InProfileIDC == 138)
				{
					fi = TEXT("Multiview Depth High");
				}
				else if (InProfileIDC == 139)
				{
					fi = TEXT("Enhanced Multiview Depth High");
				}
				else if (InProfileIDC == 244 && (InConstraintFlags & 8) == 8)
				{
					fi = TEXT("High 4:4:4 Intra");
				}
				else if (InProfileIDC == 244)
				{
					fi = TEXT("High 4:4:4 Predictive");
				}
				else
				{
					fi = TEXT("Unkown");
				}
				fi.Append(TEXT(" Profile"));
				if (InLevelIDC)
				{
					fi.Append(FString::Printf(TEXT(", level %d.%d"), InLevelIDC/10, InLevelIDC%10));
				}
				else
				{
					fi.Append(TEXT(" at unknown level"));
				}
				return fi;
			}



			bool ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				TMap<uint32, FSequenceParameterSet> spss;
				if (!ParseSequenceParameterSet(spss, InBitstream, InBitstreamLenInBytes))
				{
					return false;
				}
				OutSequenceParameterSet = spss.CreateConstIterator().Value();
				return true;
			}

			bool ParseSequenceParameterSet(TMap<uint32, FSequenceParameterSet>& InOutSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)													\
					if (!(expr))																		\
					{																					\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in SPS", elem);	\
						return false;																	\
					}

				TUniquePtr<FRBSP> RBSP(MakeRBSP(InBitstream, InBitstreamLenInBytes));
				FBitstreamReaderH264 br(RBSP->Data.GetData(), RBSP->Data.Num());

				// Verify that this is in fact an SPS NALU.
				if ((br.GetBits(8) & 0x1f) != 0x7)
				{
					UE_LOGF(LogElectraDecoders, Error, "Not an SPS NALU!");
					return false;
				}
				if (br.GetRemainingByteLength() < 4)
				{
					UE_LOGF(LogElectraDecoders, Error, "Insufficient bytes in bitstream to be an SPS NALU!");
					return false;
				}

				uint32 temp_profile_idc = br.GetBits(8);
				uint32 temp_constraint_flags = br.GetBits(8);
				uint32 temp_level_idc = br.GetBits(8);
				uint32 temp_seq_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(temp_seq_parameter_set_id <= 31, TEXT("seq_parameter_set_id"));

				FSequenceParameterSet* spsPtr = InOutSequenceParameterSets.Find(temp_seq_parameter_set_id);
				if (!spsPtr)
				{
					spsPtr = &InOutSequenceParameterSets.Emplace(temp_seq_parameter_set_id);
					check(spsPtr);
				}
				else
				{
					spsPtr->Reset();
				}
				FSequenceParameterSet& sps = *spsPtr;

				sps.profile_idc = temp_profile_idc;
				sps.constraint_set0_flag = (uint8)((temp_constraint_flags >> 7) & 1);
				sps.constraint_set1_flag = (uint8)((temp_constraint_flags >> 6) & 1);
				sps.constraint_set2_flag = (uint8)((temp_constraint_flags >> 5) & 1);
				sps.constraint_set3_flag = (uint8)((temp_constraint_flags >> 4) & 1);
				sps.constraint_set4_flag = (uint8)((temp_constraint_flags >> 3) & 1);
				sps.constraint_set5_flag = (uint8)((temp_constraint_flags >> 2) & 1);
				sps.level_idc = temp_level_idc;
				sps.seq_parameter_set_id = temp_seq_parameter_set_id;
				sps.chroma_format_idc = 1;
				if (sps.profile_idc == 100 || sps.profile_idc == 110 ||
					sps.profile_idc == 122 || sps.profile_idc == 244 || sps.profile_idc == 44 ||
					sps.profile_idc == 83 ||  sps.profile_idc == 86 || sps.profile_idc == 118 ||
					sps.profile_idc == 128 || sps.profile_idc == 138 || sps.profile_idc == 139 ||
					sps.profile_idc == 134 || sps.profile_idc == 135)
				{
					sps.chroma_format_idc = br.ue_v();
					RANGE_CHECK_FAILURE(sps.chroma_format_idc <= 3, TEXT("chroma_format_idc"));
					if (sps.chroma_format_idc == 3)
					{
						sps.separate_colour_plane_flag = br.GetBits(1);
					}

					sps.bit_depth_luma_minus8 = br.ue_v();
					RANGE_CHECK_FAILURE(sps.bit_depth_luma_minus8 <= 6, TEXT("bit_depth_luma_minus8"));
					sps.bit_depth_chroma_minus8 = br.ue_v();
					RANGE_CHECK_FAILURE(sps.bit_depth_chroma_minus8 <= 6, TEXT("bit_depth_chroma_minus8"));
					sps.qpprime_y_zero_transform_bypass_flag = br.GetBits(1);
					sps.seq_scaling_matrix_present_flag = br.GetBits(1);
					if (sps.seq_scaling_matrix_present_flag)
					{
						if (!ParseScalingLists(&sps, nullptr, sps.seq_scaling_list_present_flag, sps.ScalingList4x4, sps.ScalingList8x8, br))
						{
							UE_LOGF(LogElectraDecoders, Error, "Failed to parse SPS scaling list!");
							return false;
						}
					}
				}

				sps.log2_max_frame_num_minus4 = br.ue_v();
				RANGE_CHECK_FAILURE(sps.log2_max_frame_num_minus4 <= 12, TEXT("log2_max_frame_num_minus4"));
				sps.pic_order_cnt_type = br.ue_v();
				RANGE_CHECK_FAILURE(sps.pic_order_cnt_type <= 2, TEXT("pic_order_cnt_type"));
				if (sps.pic_order_cnt_type == 0)
				{
					sps.log2_max_pic_order_cnt_lsb_minus4 = br.ue_v();
					RANGE_CHECK_FAILURE(sps.log2_max_pic_order_cnt_lsb_minus4 <= 12, TEXT("log2_max_pic_order_cnt_lsb_minus4"));
				}
				else if (sps.pic_order_cnt_type == 1)
				{
					sps.delta_pic_order_always_zero_flag = br.GetBits(1);
					sps.offset_for_non_ref_pic = br.se_v();
					sps.offset_for_top_to_bottom_field = br.se_v();
					sps.num_ref_frames_in_pic_order_cnt_cycle = br.ue_v();
					RANGE_CHECK_FAILURE(sps.num_ref_frames_in_pic_order_cnt_cycle <= 255, TEXT("num_ref_frames_in_pic_order_cnt_cycle"));
					sps.ExpectedDeltaPerPicOrderCntCycle = 0;
					for (uint32 i=0; i<sps.num_ref_frames_in_pic_order_cnt_cycle; ++i)
					{
						sps.offset_for_ref_frame[i] = br.se_v();
						sps.ExpectedDeltaPerPicOrderCntCycle += sps.offset_for_ref_frame[i];
					}
				}

				sps.max_num_ref_frames = br.ue_v();
				sps.gaps_in_frame_num_value_allowed_flag = br.GetBits(1);
				sps.pic_width_in_mbs_minus1 = br.ue_v();
				sps.pic_height_in_map_units_minus1 = br.ue_v();
				sps.frame_mbs_only_flag = br.GetBits(1);
				if (!sps.frame_mbs_only_flag)
				{
					sps.mb_adaptive_frame_field_flag = br.GetBits(1);
				}
				sps.direct_8x8_inference_flag = br.GetBits(1);

				sps.frame_cropping_flag = br.GetBits(1);
				if (sps.frame_cropping_flag)
				{
					sps.frame_crop_left_offset = br.ue_v();
					sps.frame_crop_right_offset = br.ue_v();
					sps.frame_crop_top_offset = br.ue_v();
					sps.frame_crop_bottom_offset = br.ue_v();
				}
				uint32 MaxDpbFrames = sps.GetMaxDPBSize();
				// Removed for now as this can happen despite it being invalid.
					// RANGE_CHECK_FAILURE(sps.max_num_ref_frames <= MaxDpbFrames, TEXT("max_num_ref_frames"));
				bool bIsExt = (sps.profile_idc == 44 || sps.profile_idc == 86 || sps.profile_idc == 100 || sps.profile_idc == 110 || sps.profile_idc == 122 || sps.profile_idc == 244) && sps.constraint_set3_flag!=0;
				sps.max_num_reorder_frames = sps.max_dec_frame_buffering = bIsExt ? 0 : MaxDpbFrames;
				sps.vui_parameters_present_flag = br.GetBits(1);
				if (sps.vui_parameters_present_flag)
				{
					sps.aspect_ratio_info_present_flag = br.GetBits(1);
					if (sps.aspect_ratio_info_present_flag)
					{
						sps.aspect_ratio_idc = br.GetBits(8);
						RANGE_CHECK_FAILURE(sps.aspect_ratio_idc <= 16 || sps.aspect_ratio_idc == 255, TEXT("aspect_ratio_idc"));
						if (sps.aspect_ratio_idc == 255)
						{
							sps.sar_width = br.GetBits(16);
							sps.sar_height = br.GetBits(16);
						}
					}

					sps.overscan_info_present_flag = br.GetBits(1);
					if (sps.overscan_info_present_flag)
					{
						sps.overscan_appropriate_flag = br.GetBits(1);
					}

					sps.video_signal_type_present_flag = br.GetBits(1);
					if (sps.video_signal_type_present_flag)
					{
						sps.video_format = br.GetBits(3);
						RANGE_CHECK_FAILURE(sps.video_format <= 5, TEXT("video_format"));
						sps.video_full_range_flag = br.GetBits(1);

						sps.colour_description_present_flag = br.GetBits(1);
						if (sps.colour_description_present_flag)
						{
							sps.colour_primaries = br.GetBits(8);
							RANGE_CHECK_FAILURE(sps.colour_primaries <= 12 || sps.colour_primaries == 22, TEXT("colour_primaries"));
							sps.transfer_characteristics = br.GetBits(8);
							RANGE_CHECK_FAILURE(sps.transfer_characteristics <= 18 && sps.transfer_characteristics != 3, TEXT("transfer_characteristics"));
							sps.matrix_coefficients = br.GetBits(8);
							RANGE_CHECK_FAILURE(sps.matrix_coefficients <= 14 && sps.matrix_coefficients != 3, TEXT("matrix_coefficients"));
						}
					}

					sps.chroma_loc_info_present_flag = br.GetBits(1);
					if (sps.chroma_loc_info_present_flag)
					{
						sps.chroma_sample_loc_type_top_field = br.ue_v();
						RANGE_CHECK_FAILURE(sps.chroma_sample_loc_type_top_field <= 5, TEXT("chroma_sample_loc_type_top_field"));
						sps.chroma_sample_loc_type_bottom_field = br.ue_v();
						RANGE_CHECK_FAILURE(sps.chroma_sample_loc_type_bottom_field <= 5, TEXT("chroma_sample_loc_type_bottom_field"));
					}

					sps.timing_info_present_flag = br.GetBits(1);
					if (sps.timing_info_present_flag)
					{
						sps.num_units_in_tick = br.GetBits(32);
						RANGE_CHECK_FAILURE(sps.num_units_in_tick != 0, TEXT("num_units_in_tick"));
						sps.time_scale = br.GetBits(32);
						RANGE_CHECK_FAILURE(sps.time_scale != 0, TEXT("time_scale"));
						sps.fixed_frame_rate_flag = br.GetBits(1);
					}

					auto hrd_parameters = [&br](FSequenceParameterSet::FHRDParameters& hrd) -> bool
					{
						hrd.cpb_cnt_minus1 = br.ue_v();
						RANGE_CHECK_FAILURE(hrd.cpb_cnt_minus1 <= 31, TEXT("cpb_cnt_minus1"));
						hrd.bit_rate_scale = br.GetBits(4);
						hrd.cpb_size_scale = br.GetBits(4);
						for(uint32 SchedSelIdx=0; SchedSelIdx<=hrd.cpb_cnt_minus1; ++SchedSelIdx)
						{
							hrd.bit_rate_value_minus1[SchedSelIdx] = br.ue_v();
							hrd.cpb_size_value_minus1[SchedSelIdx] = br.ue_v();
							hrd.cbr_flag[SchedSelIdx] = br.GetBits(1);
						}

						hrd.initial_cpb_removal_delay_length_minus1 = br.GetBits(5);
						hrd.cpb_removal_delay_length_minus1 = br.GetBits(5);
						hrd.dpb_output_delay_length_minus1 = br.GetBits(5);
						hrd.time_offset_length = br.GetBits(5);
						return true;
					};

					sps.nal_hrd_parameters_present_flag = br.GetBits(1);
					if (sps.nal_hrd_parameters_present_flag)
					{
						if (!hrd_parameters(sps.nal_hrd_parameters))
						{
							return false;
						}
					}

					sps.vcl_hrd_parameters_present_flag = br.GetBits(1);
					if (sps.vcl_hrd_parameters_present_flag)
					{
						if (!hrd_parameters(sps.vcl_hrd_parameters))
						{
							return false;
						}
					}

					if (sps.nal_hrd_parameters_present_flag || sps.vcl_hrd_parameters_present_flag)
					{
						sps.low_delay_hrd_flag = br.GetBits(1);
					}

					sps.pic_struct_present_flag = br.GetBits(1);
					sps.bitstream_restriction_flag = br.GetBits(1);
					if (sps.bitstream_restriction_flag)
					{
						sps.motion_vectors_over_pic_boundaries_flag = br.GetBits(1);
						sps.max_bytes_per_pic_denom = br.ue_v();
						RANGE_CHECK_FAILURE(sps.max_bytes_per_pic_denom <= 16, TEXT("max_bytes_per_pic_denom"));
						sps.max_bits_per_mb_denom = br.ue_v();
						RANGE_CHECK_FAILURE(sps.max_bits_per_mb_denom <= 16, TEXT("max_bits_per_mb_denom"));
						sps.log2_max_mv_length_horizontal = br.ue_v();
						// The mv lengths should not be larger than 15 according to the standard, but we have
						// encountered streams where this is not the case, so we remove the checks for better compatibility.
						//RANGE_CHECK_FAILURE(sps.log2_max_mv_length_horizontal <= 15, TEXT("log2_max_mv_length_horizontal"));
						sps.log2_max_mv_length_vertical = br.ue_v();
						//RANGE_CHECK_FAILURE(sps.log2_max_mv_length_vertical <= 15, TEXT("log2_max_mv_length_vertical"));
						sps.max_num_reorder_frames = br.ue_v();
						sps.max_dec_frame_buffering = br.ue_v();
						RANGE_CHECK_FAILURE(sps.max_dec_frame_buffering <= sps.max_num_ref_frames, TEXT("max_dec_frame_buffering"));
						RANGE_CHECK_FAILURE(sps.max_num_reorder_frames <= sps.max_dec_frame_buffering, TEXT("max_num_reorder_frames"));
					}
				}
				RBSP->TrimToBitstream(br);
				sps.ProcessedBitstream = MoveTemp(RBSP->Data);
				return true;
				#undef RANGE_CHECK_FAILURE
			}



			bool ParsePictureParameterSet(TMap<uint32, FPictureParameterSet>& InOutPictureParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)													\
					if (!(expr))																		\
					{																					\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in PPS", elem);	\
						return false;																	\
					}

				TUniquePtr<FRBSP> RBSP(MakeRBSP(InBitstream, InBitstreamLenInBytes));
				FBitstreamReaderH264 br(RBSP->Data.GetData(), RBSP->Data.Num());

				// Verify that this is in fact a PPS NALU.
				if ((br.GetBits(8) & 0x1f) != 0x8)
				{
					UE_LOGF(LogElectraDecoders, Error, "Not a PPS NALU!");
					return false;
				}
				if (br.GetRemainingByteLength() < 3)
				{
					UE_LOGF(LogElectraDecoders, Error, "Insufficient bytes in bitstream to be a PPS NALU!");
					return false;
				}

				uint32 temp_pic_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(temp_pic_parameter_set_id <= 255, TEXT("pic_parameter_set_id"));

				FPictureParameterSet* ppsPtr = InOutPictureParameterSets.Find(temp_pic_parameter_set_id);
				if (!ppsPtr)
				{
					ppsPtr = &InOutPictureParameterSets.Emplace(temp_pic_parameter_set_id);
					check(ppsPtr);
				}
				else
				{
					ppsPtr->Reset();
				}
				FPictureParameterSet& pps = *ppsPtr;

				pps.pic_parameter_set_id = temp_pic_parameter_set_id;
				pps.seq_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(pps.seq_parameter_set_id <= 31, TEXT("seq_parameter_set_id"));

				const FSequenceParameterSet* spsPtr = InSequenceParameterSets.Find(pps.seq_parameter_set_id);
				if (!spsPtr)
				{
					UE_LOGF(LogElectraDecoders, Error, "SPS %u referenced by PPS has not been encountered yet.", pps.seq_parameter_set_id);
					return false;
				}
				const FSequenceParameterSet& sps = *spsPtr;

				pps.entropy_coding_mode_flag = br.GetBits(1);
				pps.bottom_field_pic_order_in_frame_present_flag = br.GetBits(1);
				pps.num_slice_groups_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(pps.num_slice_groups_minus1 <= 7, TEXT("num_slice_groups_minus1"));
				if (pps.num_slice_groups_minus1 > 0)
				{
					pps.slice_group_map_type = br.ue_v();
					RANGE_CHECK_FAILURE(pps.slice_group_map_type <= 6, TEXT("slice_group_map_type"));
					if (pps.slice_group_map_type == 0)
					{
						for(uint32 iGroup=0; iGroup<=pps.num_slice_groups_minus1; ++iGroup)
						{
							pps.run_length_minus1[iGroup] = br.ue_v();
						}
					}
					else if (pps.slice_group_map_type == 2)
					{
						for(uint32 iGroup=0; iGroup<=pps.num_slice_groups_minus1; ++iGroup)
						{
							pps.top_left[iGroup] = br.ue_v();
							pps.bottom_right[iGroup] = br.ue_v();
						}
					}
					else if (pps.slice_group_map_type == 3 || pps.slice_group_map_type == 4 || pps.slice_group_map_type == 5)
					{
						pps.slice_group_change_direction_flag = br.GetBits(1);
						pps.slice_group_change_rate_minus1 = br.ue_v();
					}
					else if (pps.slice_group_map_type == 6)
					{
						pps.pic_size_in_map_units_minus1 = br.ue_v();
						const uint32 NumBits = FMath::CeilLogTwo(pps.num_slice_groups_minus1 + 1);
						check(NumBits <	4); // the value needs to be in the range 0-7
						pps.slice_group_id.AddUninitialized(pps.pic_size_in_map_units_minus1 + 1);
						for(uint32 i=0; i<=pps.pic_size_in_map_units_minus1; ++i)
						{
							pps.slice_group_id[i] = (uint8)br.GetBits(NumBits);
							check(pps.slice_group_id[i] <= pps.num_slice_groups_minus1);
						}
					}
				}

				pps.num_ref_idx_l0_default_active_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(pps.num_ref_idx_l0_default_active_minus1 <= 31, TEXT("num_ref_idx_l0_default_active_minus1"));
				pps.num_ref_idx_l1_default_active_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(pps.num_ref_idx_l1_default_active_minus1 <= 31, TEXT("num_ref_idx_l1_default_active_minus1"));
				pps.weighted_pred_flag = br.GetBits(1);
				pps.weighted_bipred_idc = br.GetBits(2);
				RANGE_CHECK_FAILURE(pps.weighted_bipred_idc <= 2, TEXT("weighted_bipred_idc"));
				pps.pic_init_qp_minus26 = br.se_v();
				pps.pic_init_qs_minus26 = br.se_v();
				RANGE_CHECK_FAILURE(pps.pic_init_qs_minus26 >= -26 && pps.pic_init_qs_minus26 <= 25, TEXT("pic_init_qs_minus26"));
				pps.chroma_qp_index_offset = br.se_v();
				RANGE_CHECK_FAILURE(pps.chroma_qp_index_offset >= -12 && pps.chroma_qp_index_offset <= 12, TEXT("chroma_qp_index_offset"));
				pps.second_chroma_qp_index_offset = pps.chroma_qp_index_offset;
				pps.deblocking_filter_control_present_flag = br.GetBits(1);
				pps.constrained_intra_pred_flag = br.GetBits(1);
				pps.redundant_pic_cnt_present_flag = br.GetBits(1);

				if (br.more_rbsp_data())
				{
					pps.transform_8x8_mode_flag = br.GetBits(1);
					pps.pic_scaling_matrix_present_flag = br.GetBits(1);
					if (pps.pic_scaling_matrix_present_flag)
					{
						if (!ParseScalingLists(&sps, &pps, pps.pic_scaling_list_present_flag, pps.ScalingList4x4, pps.ScalingList8x8, br))
						{
							UE_LOGF(LogElectraDecoders, Error, "Failed to parse PPS scaling list!");
							return false;
						}
					}
					pps.second_chroma_qp_index_offset = br.se_v();
					RANGE_CHECK_FAILURE(pps.second_chroma_qp_index_offset >= -12 && pps.second_chroma_qp_index_offset <= 12, TEXT("second_chroma_qp_index_offset"));
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}


			bool ParseSliceHeader(TUniquePtr<FRBSP>& OutRBSP, FBitstreamReaderH264& OutRBSPReader, FSliceHeader& OutSlice, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const TMap<uint32, FPictureParameterSet>& InPictureParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)															\
					if (!(expr))																				\
					{																							\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in slice header", elem);	\
						return false;																			\
					}

				OutRBSP = MakeRBSP(InBitstream, InBitstreamLenInBytes);
				OutRBSPReader.SetData(OutRBSP->Data.GetData(), OutRBSP->Data.Num());
				FBitstreamReaderH264& br(OutRBSPReader);

				// Verify that this is in fact a (supported) slice.
				uint32 nut = br.GetBits(8);
				uint32 NaluRefIdc = (nut >> 5) & 3;
				uint32 NaluSliceType = nut & 0x1f;
				if (!((NaluSliceType >= 1 && NaluSliceType <= 5) || (NaluSliceType == 20 || NaluSliceType == 21)))
				{
					UE_LOGF(LogElectraDecoders, Error, "Not a supported slice type (%d)!", NaluSliceType);
					return false;
				}

				OutSlice.first_mb_in_slice = br.ue_v();
				OutSlice.slice_type = br.ue_v();
				RANGE_CHECK_FAILURE(OutSlice.slice_type <= 9, TEXT("slice_type"));
				OutSlice.pic_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(OutSlice.pic_parameter_set_id <= 255, TEXT("pic_parameter_set_id"));

				const FPictureParameterSet* ppsPtr = InPictureParameterSets.Find(OutSlice.pic_parameter_set_id);
				if (!ppsPtr)
				{
					UE_LOGF(LogElectraDecoders, Error, "Reference picture parameter set not found!");
					return false;
				}
				const FPictureParameterSet& pps = *ppsPtr;

				const FSequenceParameterSet* spsPtr = InSequenceParameterSets.Find(pps.seq_parameter_set_id);
				if (!spsPtr)
				{
					UE_LOGF(LogElectraDecoders, Error, "Reference sequence parameter set not found!");
					return false;
				}
				const FSequenceParameterSet& sps = *spsPtr;

				if (sps.separate_colour_plane_flag)
				{
					OutSlice.colour_plane_id = br.GetBits(2);
					RANGE_CHECK_FAILURE(OutSlice.colour_plane_id <= 2, TEXT("colour_plane_id"));
				}

				OutSlice.frame_num = br.GetBits(sps.log2_max_frame_num_minus4 + 4);

				if (!sps.frame_mbs_only_flag)
				{
					OutSlice.field_pic_flag = br.GetBits(1);
					if (OutSlice.field_pic_flag)
					{
						OutSlice.bottom_field_flag = br.GetBits(1);
					}
				}

				bool IdrPicFlag = NaluSliceType == 5;
				if (IdrPicFlag)
				{
					OutSlice.idr_pic_id = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.idr_pic_id <= 65535, TEXT("idr_pic_id"));
				}

				if (sps.pic_order_cnt_type == 0)
				{
					OutSlice.pic_order_cnt_lsb = br.GetBits(sps.log2_max_pic_order_cnt_lsb_minus4 + 4);
					if (pps.bottom_field_pic_order_in_frame_present_flag && !OutSlice.field_pic_flag)
					{
						OutSlice.delta_pic_order_cnt_bottom = br.se_v();
					}
				}
				if (sps.pic_order_cnt_type == 1 && !sps.delta_pic_order_always_zero_flag)
				{
					OutSlice.delta_pic_order_cnt[0] = br.se_v();
					if (pps.bottom_field_pic_order_in_frame_present_flag && !OutSlice.field_pic_flag)
					{
						OutSlice.delta_pic_order_cnt[1] = br.se_v();
					}
				}

				if (pps.redundant_pic_cnt_present_flag)
				{
					OutSlice.redundant_pic_cnt = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.redundant_pic_cnt <= 127, TEXT("redundant_pic_cnt"));
				}

				const bool bIsP = OutSlice.slice_type == 0 || OutSlice.slice_type == 5;
				const bool bIsB = OutSlice.slice_type == 1 || OutSlice.slice_type == 6;
				const bool bIsI = OutSlice.slice_type == 2 || OutSlice.slice_type == 7;
				const bool bIsSP = OutSlice.slice_type == 3 || OutSlice.slice_type == 8;
				const bool bIsSI = OutSlice.slice_type == 4 || OutSlice.slice_type == 9;

				if (bIsB)
				{
					OutSlice.direct_spatial_mv_pred_flag = br.GetBits(1);
				}

				OutSlice.num_ref_idx_l0_active_minus1 = pps.num_ref_idx_l0_default_active_minus1;
				OutSlice.num_ref_idx_l1_active_minus1 = pps.num_ref_idx_l1_default_active_minus1;
				if (bIsP || bIsSP || bIsB)
				{
					OutSlice.num_ref_idx_active_override_flag = br.GetBits(1);
					if (OutSlice.num_ref_idx_active_override_flag)
					{
						OutSlice.num_ref_idx_l0_active_minus1 = br.ue_v();
						RANGE_CHECK_FAILURE((OutSlice.field_pic_flag == 1) || (OutSlice.field_pic_flag == 0 && OutSlice.num_ref_idx_l0_active_minus1 <= 15), TEXT("num_ref_idx_l0_active_minus1"));
						if (bIsB)
						{
							OutSlice.num_ref_idx_l1_active_minus1 = br.ue_v();
							RANGE_CHECK_FAILURE((OutSlice.field_pic_flag == 1) || (OutSlice.field_pic_flag == 0 && OutSlice.num_ref_idx_l1_active_minus1 <= 15), TEXT("num_ref_idx_l1_active_minus1"));
						}
					}
				}

				// Slice extensions
				if (NaluSliceType == 20 || NaluSliceType == 21)
				{
					UE_LOGF(LogElectraDecoders, Error, "MVC slice extensions are not supported!");
					return false;
				}
				else
				{
					// ref_pic_list_modification()
					if (OutSlice.slice_type % 5 != 2 &&  OutSlice.slice_type % 5 != 4)
					{
						OutSlice.ref_pic_list_modification_flag_l0 = br.GetBits(1);
						if (OutSlice.ref_pic_list_modification_flag_l0)
						{
							while(1)
							{
								uint32 modification_of_pic_nums_idc = br.ue_v();
								if (modification_of_pic_nums_idc == 3)
								{
									break;
								}
								FSliceHeader::FRefPicModification& refpic = OutSlice.RefPicListModifications[0].Emplace_GetRef();
								refpic.modification_of_pic_nums_idc = modification_of_pic_nums_idc;
								if (refpic.modification_of_pic_nums_idc == 0 || refpic.modification_of_pic_nums_idc == 1)
								{
									refpic.abs_diff_pic_num_minus1 = br.ue_v();
								}
								else if (refpic.modification_of_pic_nums_idc == 2)
								{
									refpic.long_term_pic_num = br.ue_v();
								}
								else
								{
									UE_LOGF(LogElectraDecoders, Error, "Invalid modification_of_pic_nums_idc!");
									return false;
								}
							}
						}
					}

					if (OutSlice.slice_type % 5 == 1)
					{
						OutSlice.ref_pic_list_modification_flag_l1 = br.GetBits(1);
						if (OutSlice.ref_pic_list_modification_flag_l1)
						{
							while(1)
							{
								uint32 modification_of_pic_nums_idc = br.ue_v();
								if (modification_of_pic_nums_idc == 3)
								{
									break;
								}
								FSliceHeader::FRefPicModification& refpic = OutSlice.RefPicListModifications[1].Emplace_GetRef();
								refpic.modification_of_pic_nums_idc = modification_of_pic_nums_idc;
								if (refpic.modification_of_pic_nums_idc == 0 || refpic.modification_of_pic_nums_idc == 1)
								{
									refpic.abs_diff_pic_num_minus1 = br.ue_v();
								}
								else if (refpic.modification_of_pic_nums_idc == 2)
								{
									refpic.long_term_pic_num = br.ue_v();
								}
								else
								{
									UE_LOGF(LogElectraDecoders, Error, "Invalid modification_of_pic_nums_idc!");
									return false;
								}
							}
						}
					}
				}

				if ((pps.weighted_pred_flag && (bIsP || bIsSP)) || (pps.weighted_bipred_idc == 1 && bIsB))
				{
					// pred_weight_table();
					OutSlice.luma_log2_weight_denom = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.luma_log2_weight_denom <= 7, TEXT("luma_log2_weight_denom"));
					int32 ChromaArrayType = sps.separate_colour_plane_flag == 0 ? (int32)sps.chroma_format_idc : 0;
					if (ChromaArrayType != 0)
					{
						OutSlice.chroma_log2_weight_denom = br.ue_v();
						RANGE_CHECK_FAILURE(OutSlice.chroma_log2_weight_denom <= 7, TEXT("chroma_log2_weight_denom"));
					}
					int32 LumaDefault = 1 << OutSlice.luma_log2_weight_denom;
					int32 ChromaDefault = 1 << OutSlice.chroma_log2_weight_denom;
					for(int32 list=0; list<2; ++list)
					{
						FSliceHeader::PredWeightTable_t& PredWeightTable = list == 0 ? OutSlice.PredWeightTable0 : OutSlice.PredWeightTable1;
						int32 iMax = (list == 0 ? OutSlice.num_ref_idx_l0_active_minus1 : OutSlice.num_ref_idx_l1_active_minus1) + 1;
						PredWeightTable.luma_weight_l.AddDefaulted(iMax);
						PredWeightTable.luma_offset_l.AddDefaulted(iMax);
						PredWeightTable.chroma_weight_l[0].AddDefaulted(iMax);
						PredWeightTable.chroma_offset_l[0].AddDefaulted(iMax);
						PredWeightTable.chroma_weight_l[1].AddDefaulted(iMax);
						PredWeightTable.chroma_offset_l[1].AddDefaulted(iMax);
						uint32 luma_weight_l_flag;
						uint32 chroma_weight_l_flag;
						for(int32 i=0; i<iMax; ++i)
						{
							luma_weight_l_flag = br.GetBits(1);
							if (luma_weight_l_flag)
							{
								PredWeightTable.luma_weight_l[i] = br.se_v();
								RANGE_CHECK_FAILURE(PredWeightTable.luma_weight_l[i] >= -128 && PredWeightTable.luma_weight_l[i] <= 127, TEXT("luma_weight_l[i]"));
								PredWeightTable.luma_offset_l[i] = br.se_v();
								RANGE_CHECK_FAILURE(PredWeightTable.luma_offset_l[i] >= -128 && PredWeightTable.luma_offset_l[i] <= 127, TEXT("luma_offset_l[i]"));
							}
							else
							{
								PredWeightTable.luma_weight_l[i] = LumaDefault;
								PredWeightTable.luma_offset_l[i] = 0;
							}
							if (ChromaArrayType != 0)
							{
								chroma_weight_l_flag = br.GetBits(1);
								if (chroma_weight_l_flag)
								{
									for(int32 j=0; j<2; ++j)
									{
										PredWeightTable.chroma_weight_l[j][i] = br.se_v();
										RANGE_CHECK_FAILURE(PredWeightTable.chroma_weight_l[j][i] >= -128 && PredWeightTable.chroma_weight_l[j][i] <= 127, TEXT("chroma_weight_l[j][i]"));
										PredWeightTable.chroma_offset_l[j][i] = br.se_v();
										RANGE_CHECK_FAILURE(PredWeightTable.chroma_offset_l[j][i] >= -128 && PredWeightTable.chroma_offset_l[j][i] <= 127, TEXT("chroma_offset_l[j][i]"));
									}
								}
								else
								{
									PredWeightTable.chroma_weight_l[0][i] =
									PredWeightTable.chroma_weight_l[1][i] = ChromaDefault;
									PredWeightTable.chroma_offset_l[0][i] =
									PredWeightTable.chroma_offset_l[1][i] = 0;
								}
							}
						}

						if (!(OutSlice.slice_type % 5 == 1))
						{
							break;
						}
					}
				}

				if (NaluRefIdc != 0)
				{
					// dec_ref_pic_marking()
					if (IdrPicFlag)
					{
						OutSlice.no_output_of_prior_pic_flag = br.GetBits(1);
						OutSlice.long_term_reference_flag = br.GetBits(1);
					}
					else
					{
						OutSlice.adaptive_ref_pic_marking_mode_flag = br.GetBits(1);
						for(;OutSlice.adaptive_ref_pic_marking_mode_flag;)
						{
							FSliceHeader::MemoryManagementControl_t& mmco = OutSlice.MemoryManagementControl.Emplace_GetRef();
							mmco.memory_management_control_operation = br.ue_v();
							if (mmco.memory_management_control_operation == 1 || mmco.memory_management_control_operation == 3)
							{
								mmco.difference_of_pic_nums_minus1 = br.ue_v();
							}
							if (mmco.memory_management_control_operation == 2)
							{
								mmco.long_term_pic_num = br.ue_v();
							}
							if (mmco.memory_management_control_operation == 3 || mmco.memory_management_control_operation == 6)
							{
								mmco.long_term_frame_idx = br.ue_v();
							}
							if (mmco.memory_management_control_operation == 4)
							{
								mmco.max_long_term_frame_idx_plus1 = br.ue_v();
							}
							if (mmco.memory_management_control_operation == 0)
							{
								break;
							}
							if (mmco.memory_management_control_operation > 6)
							{
								UE_LOGF(LogElectraDecoders, Error, "Found invalid slice mmc operation %u", mmco.memory_management_control_operation);
								return false;
							}
						}
					}
				}

				if (pps.entropy_coding_mode_flag && !bIsI && !bIsSI)
				{
					OutSlice.cabac_init_idc = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.cabac_init_idc <= 2, TEXT("cabac_init_idc"));
				}

				OutSlice.slice_qp_delta = br.se_v();

				if (bIsSP || bIsSI)
				{
					if (bIsSP)
					{
						OutSlice.sp_for_switch_flag = br.GetBits(1);
					}
					OutSlice.slice_qs_delta = br.se_v();
				}

				if (pps.deblocking_filter_control_present_flag)
				{
					OutSlice.disable_deblocking_filter_idc = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.disable_deblocking_filter_idc <= 2, TEXT("disable_deblocking_filter_idc"));
					if (OutSlice.disable_deblocking_filter_idc != 1)
					{
						OutSlice.slice_alpha_c0_offset_div2 = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_alpha_c0_offset_div2 >= -6 && OutSlice.slice_alpha_c0_offset_div2 <= 6, TEXT("slice_alpha_c0_offset_div2"));
						OutSlice.slice_beta_offset_div2 = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_beta_offset_div2 >= -6 && OutSlice.slice_beta_offset_div2 <= 6, TEXT("slice_beta_offset_div2"));
					}
				}

				if (pps.num_slice_groups_minus1 > 0
					&& pps.slice_group_map_type >= 3 && pps.slice_group_map_type <= 5)
				{
					const uint32 NumBits = FMath::CeilLogTwo((pps.pic_size_in_map_units_minus1 + 1) / ( pps.slice_group_change_rate_minus1 + 2) );
					OutSlice.slice_group_change_cycle = br.GetBits(NumBits);
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}





			void FSlicePOCVars::UpdateRefLists()
			{
				ShortTermRefs.Empty();
				LongTermRefs.Empty();
				for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
				{
					FrameDPBInfos[i]->DPBIndex = i;
					if (FrameDPBInfos[i]->bIsUsedForReference)
					{
						(FrameDPBInfos[i]->bIsLongTermReference ? LongTermRefs : ShortTermRefs).Emplace(FrameDPBInfos[i]);
					}
				}
			}
			FSlicePOCVars::FSmallestPOC FSlicePOCVars::GetSmallestPOC()
			{
				FSmallestPOC sp;
				for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
				{
					if (FrameDPBInfos[i]->FramePOC < sp.POC && !FrameDPBInfos[i]->bHasBeenOutput)
					{
						sp.POC = FrameDPBInfos[i]->FramePOC;
						sp.Pos = i;
					}
				}
				return sp;
			}

			bool FSlicePOCVars::BeginFrame(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet, const FPictureParameterSet& InPictureParameterSet)
			{
				check(InPictureParameterSet.pic_parameter_set_id == InSliceHeader.pic_parameter_set_id);
				if (InPictureParameterSet.pic_parameter_set_id != InSliceHeader.pic_parameter_set_id)
				{
					return SetLastError(FString::Printf(TEXT("Non matching PPS passed for slice.")));
				}
				check(InSequenceParameterSet.seq_parameter_set_id == InPictureParameterSet.seq_parameter_set_id);
				if (InSequenceParameterSet.seq_parameter_set_id != InPictureParameterSet.seq_parameter_set_id)
				{
					return SetLastError(FString::Printf(TEXT("Non matching SPS passed for slice PPS.")));
				}

				bool bIsIDR = InNalUnitType == 5;
				if (bIsIDR && InSliceHeader.frame_num != 0)
				{
					return SetLastError(FString::Printf(TEXT("frame_num not zero in IDR slice header")));
				}
				if (bIsIDR && InNalRefIdc == 0)
				{
					return SetLastError(FString::Printf(TEXT("IDR frame_num has a ref_idc of zero")));
				}
				if (InSequenceParameterSet.pic_order_cnt_type > 2)
				{
					return SetLastError(FString::Printf(TEXT("Invalid poc type %u in SPS."), InSequenceParameterSet.pic_order_cnt_type));
				}
				if (InSliceHeader.field_pic_flag)
				{
					return SetLastError(FString::Printf(TEXT("Interlaced video is not supported.")));
				}

				// Init simulation DPB if necessary.
				if (max_num_ref_frames == 0 || max_frame_num == 0)
				{
					max_num_ref_frames = InSequenceParameterSet.max_num_ref_frames;
					MaxDPBSize = InSequenceParameterSet.GetDPBSize();
					if (MaxDPBSize <= 0)
					{
						return false;
					}
					if (MaxDPBSize < max_num_ref_frames)
					{
						return SetLastError(FString::Printf(TEXT("DPB size is smaller than number of reference frames.")));
					}
					max_frame_num = 1 << (InSequenceParameterSet.log2_max_frame_num_minus4 + 4);
				}
				return true;
			}

			void FSlicePOCVars::UndoPOCUpdate()
			{
				CurrentPOC = PreviousPOC;
			}

			bool FSlicePOCVars::HandleMissingFrames(TArray<FOutputFrameInfo>& OutOutputFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos, uint8 InNalUnitType, uint8 /*InNalRefIdc*/, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet)
			{
				bool bIsIDR = InNalUnitType == 5;
				// Do we need to check?
				if (bLastHadMMCO5 || bIsIDR)
				{
					// No need. Just continue.
					return true;
				}
				// Is there a frame missing?
				if (!(InSliceHeader.frame_num != CurrentPOC.prev_frame_num && InSliceHeader.frame_num != (CurrentPOC.prev_frame_num + 1) % max_frame_num))
				{
					// No. Continue.
					return true;
				}
				/*
					Decoding process for gaps in frame_num, Section 8.2.5.2
						the current picture is considered to be a picture considered having frame_num inferred to be equal to UnusedShortTermFrameNum,
						nal_ref_idc inferred to be not equal to 0,
						nal_unit_type inferred to be not equal to 5,
						IdrPicFlag inferred to be equal to 0,
						field_pic_flag inferred to be equal to 0,
						adaptive_ref_pic_marking_mode_flag inferred to be equal to 0,
						delta_pic_order_cnt[ 0 ] (if needed) inferred to be equal to 0, and delta_pic_order_cnt[ 1 ] (if needed) inferred to be equal to 0.
				*/
				InNalUnitType = 1;
				uint8 InNalRefIdc = 1;
				FSliceHeader TempSliceHeader(InSliceHeader);
				TempSliceHeader.delta_pic_order_cnt[0] = TempSliceHeader.delta_pic_order_cnt[1] = 0;
				TempSliceHeader.adaptive_ref_pic_marking_mode_flag = 0;
				TempSliceHeader.MemoryManagementControl.Empty();
				TempSliceHeader.field_pic_flag = 0;
				TempSliceHeader.long_term_reference_flag = 0;
				uint32 UnusedShortTermFrameNum = (CurrentPOC.prev_frame_num + 1) % max_frame_num;
				FOutputFrameInfo NoInfo;
				NoInfo.bDoNotOutput = true;
				for(int32 nm=0; nm<max_frame_num && InSliceHeader.frame_num != UnusedShortTermFrameNum; ++nm)
				{
					TempSliceHeader.frame_num = UnusedShortTermFrameNum;
					if (InSequenceParameterSet.pic_order_cnt_type != 0)
					{
						UpdatePOCInternal(InNalUnitType, InNalRefIdc, TempSliceHeader, InSequenceParameterSet);
					}
					CurrentPOC.prev_frame_num = UnusedShortTermFrameNum;
					UnusedShortTermFrameNum = (UnusedShortTermFrameNum + 1) % max_frame_num;
					if (!EndFrame(OutOutputFrameInfos, OutUnrefFrameInfos, NoInfo, InNalUnitType, InNalRefIdc, TempSliceHeader, true))
					{
						return false;
					}
				}
				return true;
			}

			bool FSlicePOCVars::UpdatePOC(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet)
			{
				PreviousPOC = CurrentPOC;
				bool bIsIDR = InNalUnitType == 5;

				// Check for gaps in frame_num (Section 8.2.5.2).
				// We cannot easily conceal missing frames so we treat them as an error.
				if (bLastHadMMCO5)
				{
					CurrentPOC.prev_frame_num = 0;
				}
				if (bIsIDR)
				{
					CurrentPOC.prev_frame_num = InSliceHeader.frame_num;
				}
				// Missing frames should have been covered in HandleMissingFrames() already!
				if (InSliceHeader.frame_num != CurrentPOC.prev_frame_num && InSliceHeader.frame_num != (CurrentPOC.prev_frame_num + 1) % max_frame_num)
				{
					return SetLastError(FString::Printf(TEXT("Gap in frame_num detected. Cannot conceal error.")));
				}
				UpdatePOCInternal(InNalUnitType, InNalRefIdc, InSliceHeader, InSequenceParameterSet);
				return true;
			}

			void FSlicePOCVars::UpdatePOCInternal(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet)
			{
				bool bIsIDR = InNalUnitType == 5;
				// Decoding process for picture order count, Section 8.2.1
				if (InSequenceParameterSet.pic_order_cnt_type == 0)
				{
					if (bIsIDR)
					{
						CurrentPOC.prev_pic_order_cnt_msb = 0;
						CurrentPOC.prev_pic_order_cnt_lsb = 0;
					}
					else if (bLastHadMMCO5)
					{
						CurrentPOC.prev_pic_order_cnt_msb = 0;
						CurrentPOC.prev_pic_order_cnt_lsb = CurrentPOC.TopPOC;
					}

					uint32 MaxPicOrderCntLsb = 1U << (InSequenceParameterSet.log2_max_pic_order_cnt_lsb_minus4 + 4);

					// Calculate the MSBs of current picture
					if (InSliceHeader.pic_order_cnt_lsb < CurrentPOC.prev_pic_order_cnt_lsb && (CurrentPOC.prev_pic_order_cnt_lsb - InSliceHeader.pic_order_cnt_lsb) >= MaxPicOrderCntLsb / 2)
					{
						CurrentPOC.pic_order_cnt_msb = CurrentPOC.prev_pic_order_cnt_msb + MaxPicOrderCntLsb;
					}
					else if (InSliceHeader.pic_order_cnt_lsb > CurrentPOC.prev_pic_order_cnt_lsb && (InSliceHeader.pic_order_cnt_lsb - CurrentPOC.prev_pic_order_cnt_lsb) > MaxPicOrderCntLsb / 2)
					{
						CurrentPOC.pic_order_cnt_msb = CurrentPOC.prev_pic_order_cnt_msb - MaxPicOrderCntLsb;
					}
					else
					{
						CurrentPOC.pic_order_cnt_msb = CurrentPOC.prev_pic_order_cnt_msb;
					}

					// Note: We only handle frames, not fields.
					check(InSliceHeader.field_pic_flag == 0);
					CurrentPOC.TopPOC = CurrentPOC.pic_order_cnt_msb + (int32) InSliceHeader.pic_order_cnt_lsb;
					CurrentPOC.BottomPOC = CurrentPOC.TopPOC + InSliceHeader.delta_pic_order_cnt_bottom;
					CurrentPOC.FramePOC = Min(CurrentPOC.TopPOC, CurrentPOC.BottomPOC);
					if (InNalRefIdc)
					{
						CurrentPOC.prev_pic_order_cnt_lsb = InSliceHeader.pic_order_cnt_lsb;
						CurrentPOC.prev_pic_order_cnt_msb = CurrentPOC.pic_order_cnt_msb;
					}
					CurrentPOC.prev_frame_num = InSliceHeader.frame_num;
				}
				else if (InSequenceParameterSet.pic_order_cnt_type == 1)
				{
					if (bIsIDR)
					{
						CurrentPOC.FrameNumOffset = 0;
					}
					else
					{
						if (bLastHadMMCO5)
						{
							CurrentPOC.PrevFrameNumOffset = 0;
							CurrentPOC.prev_frame_num = 0;
						}
						if (InSliceHeader.frame_num < CurrentPOC.prev_frame_num)
						{
							CurrentPOC.FrameNumOffset = CurrentPOC.PrevFrameNumOffset + (uint32)max_frame_num;
						}
						else
						{
							CurrentPOC.FrameNumOffset = CurrentPOC.PrevFrameNumOffset;
						}
					}
					uint32 absFrameNum = 0;
					if (InSequenceParameterSet.num_ref_frames_in_pic_order_cnt_cycle)
					{
						absFrameNum = CurrentPOC.FrameNumOffset + InSliceHeader.frame_num;
					}
					if (InNalRefIdc == 0 && absFrameNum > 0)
					{
						--absFrameNum;
					}
					uint32 picOrderCntCycleCnt = 0;
					uint32 frameNumInPicOrderCntCycle = 0;
					int32 expectedPicOrderCnt = 0;
					if (absFrameNum > 0)
					{
						picOrderCntCycleCnt = (absFrameNum - 1) / InSequenceParameterSet.num_ref_frames_in_pic_order_cnt_cycle;
						frameNumInPicOrderCntCycle = (absFrameNum - 1) % InSequenceParameterSet.num_ref_frames_in_pic_order_cnt_cycle;
						expectedPicOrderCnt = (int32)picOrderCntCycleCnt * InSequenceParameterSet.ExpectedDeltaPerPicOrderCntCycle;
						for(uint32 i=0; i<=frameNumInPicOrderCntCycle; ++i)
						{
							expectedPicOrderCnt += InSequenceParameterSet.offset_for_ref_frame[i];
						}
					}
					if (InNalRefIdc == 0)
					{
						expectedPicOrderCnt += InSequenceParameterSet.offset_for_non_ref_pic;
					}
					if (InSliceHeader.field_pic_flag == 0)
					{
						CurrentPOC.TopPOC = expectedPicOrderCnt + InSliceHeader.delta_pic_order_cnt[0];
						CurrentPOC.BottomPOC = CurrentPOC.TopPOC + InSequenceParameterSet.offset_for_top_to_bottom_field + InSliceHeader.delta_pic_order_cnt[1];
						CurrentPOC.FramePOC = Min(CurrentPOC.TopPOC, CurrentPOC.BottomPOC);
					}
					else if (InSliceHeader.bottom_field_flag == 0)
					{
						CurrentPOC.FramePOC = CurrentPOC.TopPOC = expectedPicOrderCnt + InSliceHeader.delta_pic_order_cnt[0];
					}
					else
					{
						CurrentPOC.FramePOC = CurrentPOC.BottomPOC = expectedPicOrderCnt + InSequenceParameterSet.offset_for_top_to_bottom_field + InSliceHeader.delta_pic_order_cnt[0];
					}
					CurrentPOC.prev_frame_num = InSliceHeader.frame_num;
					CurrentPOC.PrevFrameNumOffset = CurrentPOC.FrameNumOffset;
				}
				else if (InSequenceParameterSet.pic_order_cnt_type == 2)
				{
					if (bIsIDR)
					{
						CurrentPOC.FrameNumOffset = 0;
						CurrentPOC.TopPOC = CurrentPOC.BottomPOC = CurrentPOC.FramePOC = 0;
					}
					else
					{
						if (bLastHadMMCO5)
						{
							CurrentPOC.prev_frame_num = 0;
							CurrentPOC.PrevFrameNumOffset = 0;
						}
						if (InSliceHeader.frame_num < CurrentPOC.prev_frame_num)
						{
							CurrentPOC.FrameNumOffset = CurrentPOC.PrevFrameNumOffset + max_frame_num;
						}
						else
						{
							CurrentPOC.FrameNumOffset = CurrentPOC.PrevFrameNumOffset;
						}

						uint32 AbsFrameNum = CurrentPOC.FrameNumOffset + InSliceHeader.frame_num;
						if (InNalRefIdc)
						{
							CurrentPOC.FramePOC = AbsFrameNum * 2;
						}
						else
						{
							CurrentPOC.FramePOC = AbsFrameNum * 2 - 1;
						}
						CurrentPOC.TopPOC = CurrentPOC.BottomPOC = CurrentPOC.FramePOC;

						CurrentPOC.prev_frame_num = InSliceHeader.frame_num;
						CurrentPOC.PrevFrameNumOffset = CurrentPOC.FrameNumOffset;
					}
				}

				// Update PicNum
				// Section 8.2.4.1
				for(auto &it : FrameDPBInfos)
				{
					if (it->bIsUsedForReference && !it->bIsLongTermReference)
					{
						it->FrameNumWrap = it->FrameNum > InSliceHeader.frame_num ? (int32)it->FrameNum - max_frame_num : (int32)it->FrameNum;
						it->PicNum = it->FrameNumWrap;
					}
					else if (it->bIsLongTermReference)
					{
						it->LongTermPicNum = it->LongTermFrameIndex;
					}
				}
			}

			void FSlicePOCVars::GetCurrentReferenceFrames(TArray<FReferenceFrameListEntry>& OutCurrentReferenceFrames)
			{
				for(auto& it : ShortTermRefs)
				{
					OutCurrentReferenceFrames.Emplace(*it);
				}
				for(auto& it : LongTermRefs)
				{
					OutCurrentReferenceFrames.Emplace(*it);
				}
			}

			bool FSlicePOCVars::GetReferenceFrameLists(TArray<FReferenceFrameListEntry>& OutReferenceFrameList0, TArray<FReferenceFrameListEntry>& OutReferenceFrameList1, const FSliceHeader& InSliceHeader)
			{
				// I slice?
				if (InSliceHeader.slice_type == 2 || InSliceHeader.slice_type == 7)
				{
					OutReferenceFrameList0.Empty();
					OutReferenceFrameList1.Empty();
					// No references, no reordering. We are done here.
					return true;
				}
				// P slice?
				else if (InSliceHeader.slice_type == 0 || InSliceHeader.slice_type == 5)
				{
					// 8.2.4.2.1 Initialization process for the reference picture list for P and SP slices in frames
					TArray<FReferenceFrameListEntry> ltList;
					// Add short term references to list0
					for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
					{
						if (FrameDPBInfos[i]->bIsUsedForReference && !FrameDPBInfos[i]->bIsLongTermReference)
						{
							FReferenceFrameListEntry& le = OutReferenceFrameList0.Emplace_GetRef(*FrameDPBInfos[i]);
							le.DPBIndex = i;
						}
						if (FrameDPBInfos[i]->bIsLongTermReference)
						{
							check(FrameDPBInfos[i]->bIsUsedForReference);
							FReferenceFrameListEntry& le = ltList.Emplace_GetRef(*FrameDPBInfos[i]);
							le.DPBIndex = i;
						}
					}
					OutReferenceFrameList0.Sort([this](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
					{
						return FrameDPBInfos[a.DPBIndex]->PicNum > FrameDPBInfos[b.DPBIndex]->PicNum;
					});
					ltList.Sort([this](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
					{
						return FrameDPBInfos[a.DPBIndex]->LongTermPicNum < FrameDPBInfos[b.DPBIndex]->LongTermPicNum;
					});
					OutReferenceFrameList0.Append(ltList);
					// A requirement of 8.2.4.2 is to truncate the list if it contains excess elements.
					if ((uint32)OutReferenceFrameList0.Num() > InSliceHeader.num_ref_idx_l0_active_minus1 + 1)
					{
						OutReferenceFrameList0.SetNum((int32)InSliceHeader.num_ref_idx_l0_active_minus1 + 1);
					}
				}
				// B slice?
				else if (InSliceHeader.slice_type == 1 || InSliceHeader.slice_type == 6)
				{
					// 8.2.4.2.3 Initialization process for reference picture lists for B slices in frames
					TArray<FReferenceFrameListEntry> ltList;
					TArray<FReferenceFrameListEntry> smallerPOCList;
					TArray<FReferenceFrameListEntry> largerPOCList;
					for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
					{
						if (FrameDPBInfos[i]->bIsUsedForReference && !FrameDPBInfos[i]->bIsLongTermReference)
						{
							FReferenceFrameListEntry& le = CurrentPOC.FramePOC >= FrameDPBInfos[i]->FramePOC ? smallerPOCList.Emplace_GetRef() : largerPOCList.Emplace_GetRef();
							le = *FrameDPBInfos[i];
							le.DPBIndex = i;
						}
						if (FrameDPBInfos[i]->bIsLongTermReference)
						{
							check(FrameDPBInfos[i]->bIsUsedForReference);
							FReferenceFrameListEntry& le = ltList.Emplace_GetRef(*FrameDPBInfos[i]);
							le.DPBIndex = i;
						}
					}
					smallerPOCList.Sort([this](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
					{
						return FrameDPBInfos[a.DPBIndex]->FramePOC > FrameDPBInfos[b.DPBIndex]->FramePOC;
					});
					largerPOCList.Sort([this](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
					{
						return FrameDPBInfos[a.DPBIndex]->FramePOC < FrameDPBInfos[b.DPBIndex]->FramePOC;
					});
					ltList.Sort([this](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
					{
						return FrameDPBInfos[a.DPBIndex]->LongTermPicNum < FrameDPBInfos[b.DPBIndex]->LongTermPicNum;
					});
					OutReferenceFrameList0 = smallerPOCList;
					OutReferenceFrameList0.Append(largerPOCList);
					OutReferenceFrameList0.Append(ltList);
					OutReferenceFrameList1 = largerPOCList;
					OutReferenceFrameList1.Append(smallerPOCList);
					OutReferenceFrameList1.Append(ltList);
					// Check for list equality if list 1 has more than one entry.
					if (OutReferenceFrameList1.Num() > 1 && OutReferenceFrameList0.Num() == OutReferenceFrameList1.Num())
					{
						bool bSame = true;
						for(int32 i=0; i<OutReferenceFrameList0.Num(); ++i)
						{
							if (OutReferenceFrameList0[i] != OutReferenceFrameList1[i])
							{
								bSame = false;
								break;
							}
						}
						if (bSame)
						{
							Swap(OutReferenceFrameList1[0], OutReferenceFrameList1[1]);
						}
					}
					// A requirement of 8.2.4.2 is to truncate the list if it contains excess elements.
					if (OutReferenceFrameList0.Num() > (int32)InSliceHeader.num_ref_idx_l0_active_minus1 + 1)
					{
						OutReferenceFrameList0.SetNum((int32)InSliceHeader.num_ref_idx_l0_active_minus1 + 1);
					}
					if (OutReferenceFrameList1.Num() > (int32)InSliceHeader.num_ref_idx_l1_active_minus1 + 1)
					{
						OutReferenceFrameList1.SetNum((int32)InSliceHeader.num_ref_idx_l1_active_minus1 + 1);
					}
				}
				else
				{
					return SetLastError(FString::Printf(TEXT("Unhandled slice type")));
				}

				// Reorder the lists if necessary.
				// P or B slice list may modify list 0.
				if (InSliceHeader.slice_type != 2 && InSliceHeader.slice_type != 7)
				{
					if (InSliceHeader.ref_pic_list_modification_flag_l0)
					{
						if (!ReorderRefPicList(OutReferenceFrameList0, InSliceHeader, 0))
						{
							return false;
						}
					}
					if ((int32)InSliceHeader.num_ref_idx_l0_active_minus1 >= OutReferenceFrameList0.Num())
					{
						return SetLastError(FString::Printf(TEXT("RefPicList0[num_ref_idx_l0_active_minus1] has no reference picture.")));
					}
					if (OutReferenceFrameList0.Num() > (int32)InSliceHeader.num_ref_idx_l0_active_minus1 + 1)
					{
						OutReferenceFrameList0.SetNum((int32)InSliceHeader.num_ref_idx_l0_active_minus1 + 1);
					}
				}
				// B slice list?
				if (InSliceHeader.slice_type == 1 || InSliceHeader.slice_type == 6)
				{
					if (InSliceHeader.ref_pic_list_modification_flag_l1)
					{
						if (!ReorderRefPicList(OutReferenceFrameList1, InSliceHeader, 1))
						{
							return false;
						}
					}
					if ((int32)InSliceHeader.num_ref_idx_l1_active_minus1 >= OutReferenceFrameList1.Num())
					{
						return SetLastError(FString::Printf(TEXT("RefPicList0[num_ref_idx_l1_active_minus1] has no reference picture.")));
					}
					if (OutReferenceFrameList1.Num() > (int32)InSliceHeader.num_ref_idx_l1_active_minus1 + 1)
					{
						OutReferenceFrameList1.SetNum((int32)InSliceHeader.num_ref_idx_l1_active_minus1 + 1);
					}
				}

#if 0
				FString ls;
				for(auto& it : OutReferenceFrameList0)
				{
					if (ls.Len()) ls += TEXT(", ");
					//ls += FString::Printf(TEXT("%d [%d]"), FrameDPBInfos[it.DPBIndex]->PicNum, FrameDPBInfos[it.DPBIndex]->POC);
					ls += FString::Printf(TEXT("%d"), FrameDPBInfos[it.DPBIndex]->PicNum);
				}
				UE_LOGF(LogElectraDecoders, Log, "refpiclist 0: %ls", *ls);
				if (OutReferenceFrameList1.Num())
				{
					ls.Empty();
					for(auto& it : OutReferenceFrameList1)
					{
						if (ls.Len()) ls += TEXT(", ");
						//ls += FString::Printf(TEXT("%d [%d]"), FrameDPBInfos[it.DPBIndex]->PicNum, FrameDPBInfos[it.DPBIndex]->POC);
						ls += FString::Printf(TEXT("%d"), FrameDPBInfos[it.DPBIndex]->PicNum);
					}
					UE_LOGF(LogElectraDecoders, Log, "refpiclist 1: %ls", *ls);
				}
#endif
				return true;
			}

			bool FSlicePOCVars::ReorderRefPicList(TArray<FReferenceFrameListEntry>& InOutReferenceFrameList, const FSliceHeader& InSliceHeader, int32 InListNum)
			{
				auto FindShortTermPic = [this](int32 InPicNum) -> FReferenceFrameListEntry
				{
					FReferenceFrameListEntry e;
					for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
					{
						if (FrameDPBInfos[i]->bIsUsedForReference && !FrameDPBInfos[i]->bIsLongTermReference && FrameDPBInfos[i]->PicNum == InPicNum)
						{
							e = *FrameDPBInfos[i];
							e.DPBIndex = i;
							break;
						}
					}
					return e;
				};
				auto FindLongTermPic = [this](int32 InPicNum) -> FReferenceFrameListEntry
				{
					FReferenceFrameListEntry e;
					for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
					{
						if (FrameDPBInfos[i]->bIsLongTermReference && FrameDPBInfos[i]->LongTermPicNum == InPicNum)
						{
							e = *FrameDPBInfos[i];
							e.DPBIndex = i;
							break;
						}
					}
					return e;
				};

				check(InListNum == 0 || InListNum == 1);
				const TArray<FSliceHeader::FRefPicModification>& RefPicListModifications = InSliceHeader.RefPicListModifications[InListNum];
				const int32 maxPicNum = max_frame_num;
				const int32 currPicNum = InSliceHeader.frame_num;
				const int32 num_ref_idx_lX_active_minus1 = (int32)(InListNum == 0 ? InSliceHeader.num_ref_idx_l0_active_minus1 : InSliceHeader.num_ref_idx_l1_active_minus1);
				int32 picNumLXPred = currPicNum;
				int32 picNumLXNoWrap = 0;
				int32 picNumLX = 0;
				int32 refIdxLX = 0;
				for(int32 i=0; i<RefPicListModifications.Num(); ++i)
				{
					check(RefPicListModifications[i].modification_of_pic_nums_idc <= 2);	// idc 3 to end the loop has not been added to the list.
					if (RefPicListModifications[i].modification_of_pic_nums_idc < 2)
					{
						if (RefPicListModifications[i].modification_of_pic_nums_idc == 0)
						{
							if (picNumLXPred - ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1) < 0)
							{
								picNumLXNoWrap = picNumLXPred - ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1) + maxPicNum;
							}
							else
							{
								picNumLXNoWrap = picNumLXPred - ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1);
							}
						}
						else
						{
							if (picNumLXPred + ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1) >= maxPicNum)
							{
								picNumLXNoWrap = picNumLXPred + ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1) - maxPicNum;
							}
							else
							{
								picNumLXNoWrap = picNumLXPred + ((int32)RefPicListModifications[i].abs_diff_pic_num_minus1 + 1);
							}
						}
						picNumLXPred = picNumLXNoWrap;
						if (picNumLXNoWrap > currPicNum)
						{
							picNumLX = picNumLXNoWrap - maxPicNum;
						}
						else
						{
							picNumLX = picNumLXNoWrap;
						}

						FReferenceFrameListEntry stp = FindShortTermPic(picNumLX);
						check(stp.DPBIndex >= 0);
						if (stp.DPBIndex < 0)
						{
							return SetLastError(FString::Printf(TEXT("picNumLX for reordering does not exist as a reference in the DPB.")));
						}
						InOutReferenceFrameList.Insert(stp, refIdxLX);
						for(int32 j=++refIdxLX; j<InOutReferenceFrameList.Num(); ++j)
						{
							if (!(InOutReferenceFrameList[j].bIsLongTerm || FrameDPBInfos[InOutReferenceFrameList[j].DPBIndex]->PicNum != picNumLX))
							{
								InOutReferenceFrameList.RemoveAt(j);
								--j;
							}
						}
					}
					else if (RefPicListModifications[i].modification_of_pic_nums_idc == 2)
					{
						picNumLX = (int32)RefPicListModifications[i].long_term_pic_num;
						FReferenceFrameListEntry stp = FindLongTermPic(picNumLX);
						check(stp.DPBIndex >= 0);
						if (stp.DPBIndex < 0)
						{
							return SetLastError(FString::Printf(TEXT("long_term_pic_num for reordering does not exist as a reference in the DPB.")));
						}
						InOutReferenceFrameList.Insert(stp, refIdxLX);
						for(int32 j=++refIdxLX; j<InOutReferenceFrameList.Num(); ++j)
						{
							if (!(!InOutReferenceFrameList[j].bIsLongTerm || FrameDPBInfos[InOutReferenceFrameList[j].DPBIndex]->LongTermPicNum != picNumLX))
							{
								InOutReferenceFrameList.RemoveAt(j);
								--j;
							}
						}
					}
					else
					{
						break;
					}

				}
				if (InOutReferenceFrameList.Num() > num_ref_idx_lX_active_minus1 + 1)
				{
					InOutReferenceFrameList.SetNum(num_ref_idx_lX_active_minus1 + 1);
				}
				return true;
			}


			bool FSlicePOCVars::EndFrame(TArray<FOutputFrameInfo>& OutOutputFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos, const FOutputFrameInfo& InOutputFrameInfo, uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, bool bIsNonExisting)
			{
				bool bIsIDR = InNalUnitType == 5;
				bLastHadMMCO5 = false;
				TSharedPtr<FFrameInDPBInfo> dinf = MakeShared<FFrameInDPBInfo>();
				dinf->UserFrameInfo = InOutputFrameInfo;
				dinf->bIsUsedForReference = !!InNalRefIdc;
				dinf->bIsLongTermReference = !!InSliceHeader.long_term_reference_flag;
				dinf->LongTermFrameIndex = 0;
				dinf->FrameNum = InSliceHeader.frame_num;
				dinf->FramePOC = CurrentPOC.FramePOC;
				dinf->TopPOC = CurrentPOC.TopPOC;
				dinf->BottomPOC = CurrentPOC.BottomPOC;
				if ((dinf->bIsNonExisting = bIsNonExisting) == true)
				{
					dinf->bHasBeenOutput = true;
				}


				// Decoded reference picture marking process
				// Section 8.2.5
				if (InNalRefIdc)
				{
					// Section 8.2.5.1
					if (bIsIDR)
					{
						if (InSliceHeader.no_output_of_prior_pic_flag)
						{
							// Add all frames to the output list but tag them as NOT to be used for output.
							for(auto& it : FrameDPBInfos)
							{
								if (!it->bHasBeenOutput)
								{
									it->UserFrameInfo.bDoNotOutput = true;
									OutOutputFrameInfos.Emplace(it->UserFrameInfo);
								}
								OutUnrefFrameInfos.Emplace(it->UserFrameInfo);
							}
							FrameDPBInfos.Empty();
						}
						else
						{
							FlushDPB(OutOutputFrameInfos, OutUnrefFrameInfos);
						}
						ShortTermRefs.Empty();
						LongTermRefs.Empty();
						max_long_term_pic_index = InSliceHeader.long_term_reference_flag ? 0 : -1;
					}
					else
					{
						if (InSliceHeader.adaptive_ref_pic_marking_mode_flag == 0)
						{
							// Section 8.2.5.3, Sliding Window
							if (ShortTermRefs.Num() + LongTermRefs.Num() == Max(1, max_num_ref_frames))
							{
								check(ShortTermRefs.Num());
								for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
								{
									if (FrameDPBInfos[i]->bIsUsedForReference && !FrameDPBInfos[i]->bIsLongTermReference)
									{
										FrameDPBInfos[i]->bIsUsedForReference = false;
										UpdateRefLists();
										break;
									}
								}
							}
						}
						else
						{
							// Section 8.2.5.4
							bLastHadMMCO5 = false;
							for(auto& mmco : InSliceHeader.MemoryManagementControl)
							{
								check(mmco.memory_management_control_operation <= 6);
								if (mmco.memory_management_control_operation > 6)
								{
									return SetLastError(FString::Printf(TEXT("Invalid slice mmco type")));
								}
								switch(mmco.memory_management_control_operation)
								{
									default:
									{
										break;
									}
									case 1:
									{
										uint32 picNumX = InSliceHeader.frame_num - (mmco.difference_of_pic_nums_minus1 + 1);
										for(auto& it : FrameDPBInfos)
										{
											if (it->bIsUsedForReference && !it->bIsLongTermReference && it->PicNum == picNumX)
											{
												it->bIsUsedForReference = false;
												break;
											}
										}
										UpdateRefLists();
										break;
									}
									case 2:
									{
										for(auto& it : LongTermRefs)
										{
											if (it->LongTermPicNum == (int32)mmco.long_term_pic_num)
											{
												it->bIsUsedForReference = false;
												it->bIsLongTermReference = false;
											}
										}
										UpdateRefLists();
										break;
									}
									case 3:
									{
										uint32 picNumX = InSliceHeader.frame_num - (mmco.difference_of_pic_nums_minus1 + 1);
										for(auto& it : LongTermRefs)
										{
											if (it->LongTermFrameIndex == (int32)mmco.long_term_frame_idx)
											{
												it->bIsUsedForReference = false;
												it->bIsLongTermReference = false;
											}
										}

										for(auto& it : ShortTermRefs)
										{
											check(it->PicNum != picNumX || (it->PicNum == picNumX && !it->bIsLongTermReference)); // check if for some reason the long term flag is set
											if (it->PicNum == picNumX && !it->bIsLongTermReference)
											{
												it->LongTermFrameIndex = it->LongTermPicNum = (int32)mmco.long_term_frame_idx;
												it->bIsLongTermReference = true;
												break;
											}
										}
										UpdateRefLists();
										break;
									}
									case 4:
									{
										max_long_term_pic_index = mmco.max_long_term_frame_idx_plus1 - 1;
										for(auto& it : LongTermRefs)
										{
											if (it->LongTermFrameIndex > max_long_term_pic_index)
											{
												it->bIsUsedForReference = false;
												it->bIsLongTermReference = false;
												break;
											}
										}
										UpdateRefLists();
										break;
									}
									case 5:
									{
										for(auto& it : FrameDPBInfos)
										{
											it->bIsUsedForReference = false;
											it->bIsLongTermReference = false;
										}
										max_long_term_pic_index = -1;
										UpdateRefLists();
										bLastHadMMCO5 = true;
										break;
									}
									case 6:
									{
										for(auto& it : LongTermRefs)
										{
											if (it->LongTermFrameIndex == (int32)mmco.long_term_frame_idx)
											{
												it->bIsUsedForReference = false;
												it->bIsLongTermReference = false;
											}
										}
										dinf->bIsLongTermReference = true;
										dinf->LongTermFrameIndex = (int32)mmco.long_term_frame_idx;
										break;
									}
								}

							}
							if (bLastHadMMCO5)
							{
								dinf->FrameNum = dinf->PicNum = 0;
								CurrentPOC.TopPOC -= dinf->FramePOC;
								CurrentPOC.BottomPOC -= dinf->FramePOC;
								dinf->FramePOC = Min(CurrentPOC.TopPOC, CurrentPOC.BottomPOC);
								FlushDPB(OutOutputFrameInfos, OutUnrefFrameInfos);
							}
						}
					}
				}

				// If the DPB is full first try to remove a frame that has been output and is no longer referenced.
				if (FrameDPBInfos.Num() == MaxDPBSize)
				{
					for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
					{
						if (FrameDPBInfos[i]->bHasBeenOutput && !FrameDPBInfos[i]->bIsUsedForReference)
						{
							OutUnrefFrameInfos.Emplace(FrameDPBInfos[i]->UserFrameInfo);
							FrameDPBInfos.RemoveAt(i);
							break;
						}
					}
				}

				// While the DPB is full we need to find a frame to output.
				while(FrameDPBInfos.Num() == MaxDPBSize)
				{
					FSmallestPOC SmallestPOC = GetSmallestPOC();

					if (InNalRefIdc == 0)
					{
						if (SmallestPOC.Pos < 0 || dinf->FramePOC < SmallestPOC.POC)
						{
							// This frame won't be stored in the DPB and needs to be output immediately.
							OutOutputFrameInfos.Emplace(InOutputFrameInfo);
							return true;
						}
					}

					if (SmallestPOC.Pos >= 0)
					{
						OutOutputFrameInfos.Emplace(FrameDPBInfos[SmallestPOC.Pos]->UserFrameInfo);
						FrameDPBInfos[SmallestPOC.Pos]->bHasBeenOutput = true;
						if (!FrameDPBInfos[SmallestPOC.Pos]->bIsUsedForReference)
						{
							OutUnrefFrameInfos.Emplace(FrameDPBInfos[SmallestPOC.Pos]->UserFrameInfo);
							FrameDPBInfos.RemoveAt(SmallestPOC.Pos);
							break;
						}
					}
					else
					{
						check(!"how could this happen?");
						return SetLastError(FString::Printf(TEXT("No frame found in DPB to output.")));
					}
				}

				// Check that the frame number doesn't appear twice in the short term references
				if (InNalRefIdc && InSliceHeader.long_term_reference_flag == 0)
				{
					for(auto& it : ShortTermRefs)
					{
						if (it->FrameNum == InSliceHeader.frame_num)
						{
							return SetLastError(FString::Printf(TEXT("Duplicate frame_num in short term reference picture buffer.")));
						}
					}
				}

				// Add to DPB
				FrameDPBInfos.Emplace(MoveTemp(dinf));
				// Update references
				UpdateRefLists();
				if (ShortTermRefs.Num() + LongTermRefs.Num() > Max(1, max_num_ref_frames))
				{
					return SetLastError(FString::Printf(TEXT("Exceeded maximum number of reference frames.")));
				}
#if 0
				// dump dpb for debugging
				UE_LOGF(LogElectraDecoders, Log, "Current DPB POC");
				for(auto& it : FrameDPBInfos)
				{
					UE_LOGF(LogElectraDecoders, Log, "fn=%u  poc=%d  ref=%d  ltref=%d  out=%d", it->FrameNum, it->FramePOC, it->bIsUsedForReference, it->bIsLongTermReference, it->bHasBeenOutput);
				}
#endif
				return true;
			}

			void FSlicePOCVars::FlushDPB(TArray<FOutputFrameInfo>& InOutInfos, TArray<FOutputFrameInfo>& InUnrefInfos)
			{
				// Mark all frames as unused for reference and remove all that have already been output.
				for(int32 i=0; i<FrameDPBInfos.Num(); ++i)
				{
					FrameDPBInfos[i]->bIsUsedForReference = false;
					if (FrameDPBInfos[i]->bHasBeenOutput)
					{
						InUnrefInfos.Emplace(FrameDPBInfos[i]->UserFrameInfo);
						FrameDPBInfos.RemoveAt(i);
						--i;
					}
				}
				// Output frames in POC order
				while(FrameDPBInfos.Num())
				{
					FSmallestPOC SmallestPOC = GetSmallestPOC();
					if (SmallestPOC.Pos >= 0)
					{
						InOutInfos.Emplace(FrameDPBInfos[SmallestPOC.Pos]->UserFrameInfo);
						FrameDPBInfos[SmallestPOC.Pos]->bHasBeenOutput = true;
						InUnrefInfos.Emplace(FrameDPBInfos[SmallestPOC.Pos]->UserFrameInfo);
						FrameDPBInfos.RemoveAt(SmallestPOC.Pos);
					}
					else
					{
						break;
					}
				}
			}

			void FSlicePOCVars::Flush(TArray<FOutputFrameInfo>& OutRemainingFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos)
			{
				FlushDPB(OutRemainingFrameInfos, OutUnrefFrameInfos);
				Reset();
			}

			void FSlicePOCVars::Reset()
			{
				CurrentPOC.Reset();
				PreviousPOC.Reset();
				MaxDPBSize = 0;
				max_num_ref_frames = 0;
				max_frame_num = 0;
				max_long_term_pic_index = -1;
				FrameDPBInfos.Empty();
				ShortTermRefs.Empty();
				LongTermRefs.Empty();
				LastErrorMsg.Empty();
			}

			bool FSlicePOCVars::SetLastError(const FString& InLastErrorMsg)
			{
				LastErrorMsg = InLastErrorMsg;
				UE_LOGF(LogElectraDecoders, Error, "%ls", *InLastErrorMsg);
				return false;
			}



			int32 ELECTRADECODERS_API CalcNumBytesInNAL(int32 InProfileLevel, int32 InFrameWidth, int32 InFrameHeight, double InFrameRate)
			{
				int32 MaxMBPS = 0;
				int32 MinCR = InProfileLevel == 31 || InProfileLevel == 32 || InProfileLevel == 40 ? 4 : 2;
				double fR = InProfileLevel >= 60 ? 1/300.0 : 1/172.0;
				if (InProfileLevel <= 10) MaxMBPS = 1485;
				else if (InProfileLevel == 11) MaxMBPS = 3000;
				else if (InProfileLevel == 12) MaxMBPS = 6000;
				else if (InProfileLevel == 13) MaxMBPS = 11880;
				else if (InProfileLevel == 20) MaxMBPS = 11880;
				else if (InProfileLevel == 21) MaxMBPS = 19800;
				else if (InProfileLevel == 22) MaxMBPS = 20250;
				else if (InProfileLevel == 30) MaxMBPS = 40500;
				else if (InProfileLevel == 31) MaxMBPS = 108000;
				else if (InProfileLevel == 32) MaxMBPS = 216000;
				else if (InProfileLevel == 40) MaxMBPS = 245760;
				else if (InProfileLevel == 41) MaxMBPS = 245760;
				else if (InProfileLevel == 42) MaxMBPS = 522240;
				else if (InProfileLevel == 50) MaxMBPS = 589824;
				else if (InProfileLevel == 51) MaxMBPS = 983040;
				else if (InProfileLevel == 52) MaxMBPS = 2073600;
				else if (InProfileLevel == 60) MaxMBPS = 4177920;
				else if (InProfileLevel == 61) MaxMBPS = 8355840;
				else if (InProfileLevel == 62) MaxMBPS = 16711680;
				if (MaxMBPS == 0 || InFrameWidth <= 0 || InFrameHeight <= 0 || InFrameRate <= 1.0)
				{
					return 1536 << 10;
				}
				int32 PicSizeInMbs = Align(InFrameWidth, 16) * Align(InFrameHeight, 16) / 256;
				int32 NumBytesInNAL = (int32)(384.0 * (FMath::Max((double)PicSizeInMbs, fR * (double)MaxMBPS) + ((double)MaxMBPS * 1.0 / InFrameRate)) / (double)MinCR);
				return NumBytesInNAL;
			}


			bool FAVCDecoderConfigurationRecord::Parse(const TConstArrayView<uint8>& InDCR)
			{
				CSD.Empty();
				RawData.Empty();

				if (!InDCR.Num())
				{
					return false;
				}

				RawData = InDCR;
				Electra::FElectraByteReader ByteReader(InDCR.GetData(), InDCR.Num());

				if (!ByteReader.ReadByte(ConfigurationVersion) || ConfigurationVersion != 1)
				{
					return false;
				}
				if (!ByteReader.ReadByte(AVCProfileIndication))
				{
					return false;
				}
				if (!ByteReader.ReadByte(ProfileCompatibility))
				{
					return false;
				}
				if (!ByteReader.ReadByte(AVCLevelIndication))
				{
					return false;
				}
				if (!ByteReader.ReadByte(NALUnitLength))
				{
					return false;
				}
				NALUnitLength = (NALUnitLength & 3) + 1;
				uint8 nSPS = 0;
				if (!ByteReader.ReadByte(nSPS))
				{
					return false;
				}
				nSPS &= 31;
				if (nSPS)
				{
					SequenceParameterSets.Reserve(nSPS);
				}
				int32 TotalSPSSize = 0;
				for(int32 i=0; i<nSPS; ++i)
				{
					uint16 spsLen = 0;
					if (!ByteReader.ReadByte(spsLen))
					{
						return false;
					}
					TArray<uint8>& sps = SequenceParameterSets.AddDefaulted_GetRef();
					sps.Reserve(spsLen);
					sps.SetNumUninitialized(spsLen);
					if (!ByteReader.ReadBytes(sps.GetData(), spsLen))
					{
						return false;
					}
					TotalSPSSize += 4 + spsLen;		// 4 because we always use 32 bit startcode and not NALUnitLength
				}
				uint8 nPPS = 0;
				if (!ByteReader.ReadByte(nPPS))
				{
					return false;
				}
				if (nPPS)
				{
					PictureParameterSets.Reserve(nPPS);
				}
				int32 TotalPPSSize = 0;
				for(int32 i=0; i<nPPS; ++i)
				{
					uint16 ppsLen = 0;
					if (!ByteReader.ReadByte(ppsLen))
					{
						return false;
					}
					TArray<uint8>& pps = PictureParameterSets.AddDefaulted_GetRef();
					pps.Reserve(ppsLen);
					pps.SetNumUninitialized(ppsLen);
					if (!ByteReader.ReadBytes(pps.GetData(), ppsLen))
					{
						return false;
					}
					TotalPPSSize += 4 + ppsLen;		// 4 because we always use 32 bit startcode and not NALUnitLength
				}

				if (AVCProfileIndication != 66 && AVCProfileIndication != 77 && AVCProfileIndication != 88)
				{
					// At least according to the standard these values must appear.
					// However, there are files that are of profile 100 but omit these values.
					// Therefore let's do a quick check if we can read at least 4 more bytes.
					if (ByteReader.BytesRemaining() >= 4)
					{
						if (!ByteReader.ReadByte(ChromaFormat))
						{
							return false;
						}
						ChromaFormat &= 3;
						if (!ByteReader.ReadByte(BitDepthLumaMinus8))
						{
							return false;
						}
						BitDepthLumaMinus8 &= 7;
						if (!ByteReader.ReadByte(BitDepthChromaMinus8))
						{
							return false;
						}
						BitDepthChromaMinus8 &= 7;
						uint8 nSPSExt;
						if (!ByteReader.ReadByte(nSPSExt))
						{
							return false;
						}
						if (nSPSExt)
						{
							SequenceParameterSetsExt.Reserve(nSPSExt);
						}
						for(int32 i=0; i<nSPSExt; ++i)
						{
							uint16 spsLenExt = 0;
							if (!ByteReader.ReadByte(spsLenExt))
							{
								return false;
							}
							TArray<uint8>& spsExt = SequenceParameterSetsExt.AddDefaulted_GetRef();
							spsExt.Reserve(spsLenExt);
							spsExt.SetNumUninitialized(spsLenExt);
							if (!ByteReader.ReadBytes(spsExt.GetData(), spsLenExt))
							{
								return false;
							}
						}
						bHaveAdditionalProfileIndication = true;
					}
				}

				int32 TotalCSDSize = TotalSPSSize + TotalPPSSize;
				if (TotalCSDSize)
				{
					const TArray<uint8> StartCode { 0,0,0,1 };
					CSD.Reserve(TotalCSDSize);
					for(int32 i=0; i<SequenceParameterSets.Num(); ++i)
					{
						CSD.Append(StartCode);
						CSD.Append(SequenceParameterSets[i]);
					}
					for(int32 i=0; i<PictureParameterSets.Num(); ++i)
					{
						CSD.Append(StartCode);
						CSD.Append(PictureParameterSets[i]);
					}
				}
				return true;
			}

			bool FAVCDecoderConfigurationRecord::CreateFromCodecSpecificData(const TConstArrayView<uint8>& InCSD)
			{
				CSD.Empty();
				RawData.Empty();
				if (InCSD.IsEmpty())
				{
					return false;
				}

				CSD = InCSD;
				TArray<FNaluInfo> Nalus;
				if (!ParseBitstreamForNALUs(Nalus, InCSD.GetData(), InCSD.Num()))
				{
					return false;
				}

				struct FArray
				{
					TArray<TArray<uint8>> NALUs;
				};
				FArray aSPS, aPPS;
				for(int32 i=0; i<Nalus.Num(); ++i)
				{
					if (Nalus[i].Type == 7)
					{
						FSequenceParameterSet sps;
						if (ParseSequenceParameterSet(sps, InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size))
						{
							AVCProfileIndication = sps.profile_idc;
							ProfileCompatibility = (sps.constraint_set0_flag << 7) | (sps.constraint_set1_flag << 6) | (sps.constraint_set2_flag << 5) |
													(sps.constraint_set3_flag << 4) | (sps.constraint_set4_flag << 3) | (sps.constraint_set5_flag << 2);
							AVCLevelIndication = sps.level_idc;
							NALUnitLength = 4;
							ChromaFormat = sps.chroma_format_idc;
							BitDepthLumaMinus8 = sps.bit_depth_luma_minus8;
							BitDepthChromaMinus8 = sps.bit_depth_chroma_minus8;
							bHaveAdditionalProfileIndication = false;

							TArray<uint8>& NewSPS(SequenceParameterSets.Emplace_GetRef());
							NewSPS.Append(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
						}
						else
						{
							return false;
						}
						aSPS.NALUs.Emplace_GetRef() = MakeArrayView<const uint8>(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
					}
					else if (Nalus[i].Type == 8)
					{
						aPPS.NALUs.Emplace_GetRef() = MakeArrayView<const uint8>(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
					}
				}
				if (aSPS.NALUs.IsEmpty() || aPPS.NALUs.IsEmpty())
				{
					return false;
				}

				// Create the raw configuration record now.
				ConfigurationVersion = 1;
				Electra::FBitstreamWriter wr;
				wr.PutBits(ConfigurationVersion, 8);
				wr.PutBits(AVCProfileIndication, 8);
				wr.PutBits(ProfileCompatibility, 8);
				wr.PutBits(AVCLevelIndication, 8);
				wr.PutBits(63U, 6);
				wr.PutBits(NALUnitLength-1, 2);
				wr.PutBits(7U, 3);
				wr.PutBits(static_cast<uint32>(aSPS.NALUs.Num()), 5);
				for(int32 j=0; j<aSPS.NALUs.Num(); ++j)
				{
					const TArray<uint8>& d(aSPS.NALUs[j]);
					wr.PutBits(static_cast<uint32>(d.Num()), 16);
					for(int32 k=0; k<d.Num(); ++k)
					{
						wr.PutBits(d[k], 8);
					}
				}
				wr.PutBits(static_cast<uint32>(aPPS.NALUs.Num()), 8);
				for(int32 j=0; j<aPPS.NALUs.Num(); ++j)
				{
					const TArray<uint8>& d(aPPS.NALUs[j]);
					wr.PutBits(static_cast<uint32>(d.Num()), 16);
					for(int32 k=0; k<d.Num(); ++k)
					{
						wr.PutBits(d[k], 8);
					}
				}
				if (AVCProfileIndication != 66 && AVCProfileIndication != 77 && AVCProfileIndication != 88)
				{
					wr.PutBits(63U, 6);
					wr.PutBits(ChromaFormat, 2);
					wr.PutBits(31U, 5);
					wr.PutBits(BitDepthLumaMinus8, 3);
					wr.PutBits(31U, 5);
					wr.PutBits(BitDepthChromaMinus8, 3);
					wr.PutBits(0U, 8);
				}
				wr.GetArray(RawData);
				return true;
			}

			FString FAVCDecoderConfigurationRecord::GetFormatInfo() const
			{
				FString fi(TEXT("AVC, "));
				fi.Append(FSequenceParameterSet::GetFormatInfo(AVCProfileIndication, AVCLevelIndication, 0));
				return fi;
			}

		} // namespace H264
	} // namespace MPEG
} // namespace ElectraDecodersUtil
