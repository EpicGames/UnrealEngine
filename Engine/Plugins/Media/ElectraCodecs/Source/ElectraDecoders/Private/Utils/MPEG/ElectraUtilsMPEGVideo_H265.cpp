// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/MPEG/ElectraUtilsMPEGVideo_H265.h"
#include "Utilities/ElectraBitstream.h"
#include "ElectraDecodersUtils.h"
#include "ElectraDecodersModule.h"

#define LOG_DPB_FRAME_OUTPUT 0

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		namespace H265
		{
			static inline int32 GetMaxDPBSize(int32 InGeneralLevelIdc, uint32 PicSizeInSamplesY, uint8 sps_curr_pic_ref_enabled_flag)
			{
				const int32 maxDpbPicBuf = sps_curr_pic_ref_enabled_flag ? 7 : 6;
				int32 MaxDpbSize = 16;
				// Table A-8
				uint32 MaxLumaPs = 0;
				switch(InGeneralLevelIdc)
				{
					#define LEVEL(a,b) a*30+(b*3)
					case LEVEL(1,0):	MaxLumaPs = 36864;		break;
					case LEVEL(2,0):	MaxLumaPs = 122880;		break;
					case LEVEL(2,1):	MaxLumaPs = 245760;		break;
					case LEVEL(3,0):	MaxLumaPs = 552960;		break;
					case LEVEL(3,1):	MaxLumaPs = 983040;		break;
					case LEVEL(4,0):
					case LEVEL(4,1):	MaxLumaPs = 2228224;	break;
					case LEVEL(5,0):
					case LEVEL(5,1):
					case LEVEL(5,2):	MaxLumaPs = 8912896;	break;
					case LEVEL(6,0):
					case LEVEL(6,1):
					case LEVEL(6,2):	MaxLumaPs = 35651584;	break;
					case LEVEL(6,3):	MaxLumaPs = 80216064;	break;
					case LEVEL(7,0):
					case LEVEL(7,1):
					case LEVEL(7,2):	MaxLumaPs = 142606336;	break;
					default:
						return MaxDpbSize;
					#undef LEVEL
				}
				if (PicSizeInSamplesY <= (MaxLumaPs >> 2))
				{
					MaxDpbSize = Min(4 * maxDpbPicBuf, 16);
				}
				else if (PicSizeInSamplesY <= (MaxLumaPs >> 1))
				{
					MaxDpbSize = Min(2 * maxDpbPicBuf, 16);
				}
				else if (PicSizeInSamplesY <= ((3 * MaxLumaPs) >> 2))
				{
					MaxDpbSize = Min((4 * maxDpbPicBuf) / 3, 16);
				}
				else
				{
					MaxDpbSize = maxDpbPicBuf;
				}
				return MaxDpbSize;
			}

			// Default scaling list values, already in diagonal scan order (Rec. ITU-T H.265 (V9) tables 7-5 and 7-6)
			static const uint8 Default_4x4_Diag[16] = { 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16 };
			static const uint8 Default_8x8_Intra_Diag[64] = { 16, 16, 16, 16, 17, 18, 21, 24, 16, 16, 16, 16, 17, 19, 22, 25,
															  16, 16, 17, 18, 20, 22, 25, 29, 16, 16, 18, 21, 24, 27, 31, 36,
															  17, 17, 20, 24, 30, 35, 41, 47, 18, 19, 22, 27, 35, 44, 54, 65,
															  21, 22, 25, 31, 41, 54, 70, 88, 24, 25, 29, 36, 47, 65, 88, 115 };
			static const uint8 Default_8x8_Inter_Diag[64] = { 16, 16, 16, 16, 17, 18, 20, 24, 16, 16, 16, 17, 18, 20, 24, 25,
															  16, 16, 17, 18, 20, 24, 25, 28, 16, 17, 18, 20, 24, 25, 28, 33,
															  17, 18, 20, 24, 25, 28, 33, 41, 18, 20, 24, 25, 28, 33, 41, 54,
															  20, 24, 25, 28, 33, 41, 54, 71, 24, 25, 28, 33, 41, 54, 71, 91 };

			// Precalculated mapping tables from sequential to diagonal scan order for 4x4 and 8x8 matrices (Rec. ITU-T H.265 (V9) section 6.5.3)
			static const uint8 ScanOrderDiag4[16] = { 0, 4, 1, 8, 5, 2, 12, 9, 6, 3, 13, 10, 7, 14, 11, 15 };
			static const uint8 ScanOrderDiag8[64] = { 0, 8, 1, 16, 9, 2, 24, 17, 10, 3, 32, 25, 18, 11, 4, 40,
													  33, 26, 19, 12, 5, 48, 41, 34, 27, 20, 13, 6, 56, 49, 42, 35,
													  28, 21, 14, 7, 57, 50, 43, 36, 29, 22, 15, 58, 51, 44, 37, 30,
													  23, 59, 52, 45, 38, 31, 60, 53, 46, 39, 61, 54, 47, 62, 55, 63 };


			uint64 EBSPtoRBSP(TArray<uint64>& OutRemovedInBufPos, uint8* OutBuf, const uint8* InBuf, uint64 InNumBytes)
			{
				// Scan the input in steps of 3.
				const uint8* OutBase = OutBuf;
				uint64 Prev=0;
				for(int64 i=0,iMax=(int64)InNumBytes-2; i<iMax; )
				{
					// If the 3rd byte is larger than a 3 then we are not in a '00 00 03' sequence.
					if (InBuf[i+2] > 3)
					{
						i += 3;
						continue;
					}
					if (InBuf[i+2] != 3)
					{
						++i;
						continue;
					}
					// Sequence found?
					if (InBuf[i] == 0 && InBuf[i+1] == 0)
					{
						const uint32 np = i+2;
						OutRemovedInBufPos.Emplace(np);
						uint32 nb = np-Prev;
						FMemory::Memcpy(OutBuf, InBuf + Prev, nb);
						OutBuf += nb;
						Prev = np+1;
					}
					i += 3;
				}
				if (Prev < InNumBytes)
				{
					uint32 nb = InNumBytes-Prev;
					FMemory::Memcpy(OutBuf, InBuf + Prev, nb);
					OutBuf += nb;
				}
				return OutBuf - OutBase;
			}
			TUniquePtr<FRBSP> MakeRBSP(const uint8* InData, uint64 InDataSize)
			{
				auto Ptr = MakeUnique<FRBSP>(InDataSize);
				Ptr->Size = EBSPtoRBSP(Ptr->RemovedAtSrc, Ptr->Data, InData, InDataSize);
				return Ptr;
			}

			bool ParseBitstreamForNALUs(TArray<FNaluInfo>& OutNALUs, const uint8* InBitstream, uint64 InBitstreamLength)
			{
				if (!InBitstream)
				{
					return false;
				}
				else if (InBitstreamLength < 3)
				{
					UE_LOGF(LogElectraDecoders, Error, "Bitstream too short to parse for NALUs");
					return false;
				}
				// Skip over stream in intervals of 3 until InBitstream[i + 2] is either 0 or 1
				uint64 i=0;
				for(uint64 iMax=InBitstreamLength-2; i<iMax; )
				{
					if (InBitstream[i+2] > 1)
					{
						i += 3;
					}
					else if (InBitstream[i+2] == 1)
					{
						if (InBitstream[i+1] == 0 && InBitstream[i] == 0)
						{
							// Found start sequence but we don't know if it has a 3 or 4 byte start code so we check.
							const bool bOff1 = i > 0 && InBitstream[i-1] == 0;
							uint64 Offset = bOff1 ? i-1 : i;
							// Update length of previous entry.
							if (OutNALUs.Num())
							{
								OutNALUs.Last().Size = Offset - (OutNALUs.Last().Offset + OutNALUs.Last().UnitLength);
							}

							FNaluInfo& NalInfo = OutNALUs.Emplace_GetRef();
							NalInfo.Offset = Offset;
							NalInfo.UnitLength = bOff1 ? 4 : 3;
							uint32 nut = ((uint32)InBitstream[NalInfo.Offset + NalInfo.UnitLength] << 8) + InBitstream[NalInfo.Offset + NalInfo.UnitLength + 1];
							if ((nut & 0x8000U) != 0)
							{
								UE_LOGF(LogElectraDecoders, Error, "Forbidden zero bit in NAL header is not zero!");
								return false;
							}
							NalInfo.Type = nut >> 9;
							NalInfo.LayerId = (nut >> 3) & 0x3f;
							NalInfo.TemporalIdPlusOne = nut & 7;
						}
						i += 3;
					}
					else
					{
						++i;
					}
				}
				if (OutNALUs.IsEmpty())
				{
					UE_LOGF(LogElectraDecoders, Error, "No NALUs found in bitstream!");
					return false;
				}
				OutNALUs.Last().Size = InBitstreamLength - (OutNALUs.Last().Offset + OutNALUs.Last().UnitLength);
				return true;
			}

			int32 FSequenceParameterSet::GetMaxDPBSize() const
			{
				return ElectraDecodersUtil::MPEG::H265::GetMaxDPBSize(profile_tier_level.general_level_idc, PicSizeInSamplesY, sps_scc_extension.sps_curr_pic_ref_enabled_flag);
			}
			int32 FSequenceParameterSet::GetDPBSize() const
			{
				int32 MaxDPB = GetMaxDPBSize();
				uint32 max_dec_buffering = 0;
				for(int32 i=0; i<UE_ARRAY_COUNT(sps_max_dec_pic_buffering_minus1); ++i)
				{
					max_dec_buffering = max_dec_buffering > sps_max_dec_pic_buffering_minus1[i] ? max_dec_buffering : sps_max_dec_pic_buffering_minus1[i];
				}
				++max_dec_buffering;
				check((int32)max_dec_buffering <= MaxDPB);
				return (int32)max_dec_buffering;
			}
			int32 FSequenceParameterSet::GetMinCbSizeY() const
			{
				return MinCbSizeY;
			}
			int32 FSequenceParameterSet::GetWidth() const
			{
				return (int32) pic_width_in_luma_samples;
			}
			int32 FSequenceParameterSet::GetHeight() const
			{
				return (int32) pic_height_in_luma_samples;
			}
			void FSequenceParameterSet::GetDisplaySize(int32& OutWidth, int32& OutHeight) const
			{
				int32 cl,cr,ct,cb;
				GetCrop(cl,cr,ct,cb);
				OutWidth = GetWidth() - cl - cr;
				OutHeight = GetHeight() - ct - cb;
			}
			void FSequenceParameterSet::GetDisplaySize(uint32& OutWidth, uint32& OutHeight) const
			{
				int32 w,h;
				GetDisplaySize(w, h);
				OutWidth = (uint32) w;
				OutHeight = (uint32) h;
			}
			void FSequenceParameterSet::GetCrop(int32& OutLeft, int32& OutRight, int32& OutTop, int32& OutBottom) const
			{
				OutLeft = conf_win_left_offset * SubWidthC;
				OutRight = conf_win_right_offset * SubWidthC;
				OutTop	= conf_win_top_offset * SubHeightC;
				OutBottom = conf_win_bottom_offset * SubHeightC;
			}
			void FSequenceParameterSet::GetAspect(int32& OutSarW, int32& OutSarH) const
			{
				if (vui_parameters_present_flag && vui_parameters.aspect_ratio_info_present_flag)
				{
					switch(vui_parameters.aspect_ratio_idc)
					{
						default:	OutSarW = OutSarH = 0;		break;
						case 0:		OutSarW = OutSarH = 0;		break;
						case 1:		OutSarW = OutSarH = 1;		break;
						case 2:		OutSarW = 12; OutSarH = 11;	break;
						case 3:		OutSarW = 10; OutSarH = 11;	break;
						case 4:		OutSarW = 16; OutSarH = 11;	break;
						case 5:		OutSarW = 40; OutSarH = 33;	break;
						case 6:		OutSarW = 24; OutSarH = 11;	break;
						case 7:		OutSarW = 20; OutSarH = 11;	break;
						case 8:		OutSarW = 32; OutSarH = 11;	break;
						case 9:		OutSarW = 80; OutSarH = 33;	break;
						case 10:	OutSarW = 18; OutSarH = 11;	break;
						case 11:	OutSarW = 15; OutSarH = 11;	break;
						case 12:	OutSarW = 64; OutSarH = 33;	break;
						case 13:	OutSarW = 160; OutSarH = 99;break;
						case 14:	OutSarW = 4; OutSarH = 3;	break;
						case 15:	OutSarW = 3; OutSarH = 2;	break;
						case 16:	OutSarW = 2; OutSarH = 1;	break;
						case 255:	OutSarW = vui_parameters.sar_width; OutSarH = vui_parameters.sar_height;	break;
					}
				}
				else
				{
					OutSarW = OutSarH = 1;
				}
			}
			FFractionalValue FSequenceParameterSet::GetTiming() const
			{
				if (vui_parameters_present_flag && vui_parameters.vui_timing_info_present_flag)
				{
					return FFractionalValue(vui_parameters.vui_time_scale, vui_parameters.vui_num_units_in_tick);
				}
				return FFractionalValue();
			}

			uint64 FSequenceParameterSet::GetConstraintFlags() const
			{
				return (uint64)((profile_tier_level.general_progressive_source_flag << 3) | (profile_tier_level.general_interlaced_source_flag << 2) | (profile_tier_level.general_non_packed_constraint_flag << 1) | profile_tier_level.general_frame_only_constraint_flag) << 44;
			}

			FString FSequenceParameterSet::GetRFC6381(const TCHAR* SampleTypePrefix) const
			{
				// As per ISO/IEC 14496-15:2014 Annex E.3
				uint32 cf = BitReverse32(profile_tier_level.general_profile_compatibility_flags);
				uint64 ConstraintFlags = GetConstraintFlags();
				FString cfs;
				bool bNonZero = false;
				for(int32 i=5; i>=0; --i, ConstraintFlags >>= 8)
				{
					if ((ConstraintFlags & 255) == 0 && !bNonZero)
					{
						continue;
					}
					bNonZero = true;
					cfs = FString::Printf(TEXT(".%02X"), (uint8)(ConstraintFlags & 255)) + cfs;
				}

				if (profile_tier_level.general_profile_space == 0)
				{
					return FString::Printf(TEXT("%s.%d.%X.%c%d%s"), SampleTypePrefix, profile_tier_level.general_profile_idc, cf, profile_tier_level.general_tier_flag ? TCHAR('H') : TCHAR('L'), profile_tier_level.general_level_idc, *cfs);
				}
				else
				{
					return FString::Printf(TEXT("%s.%c%d.%X.%c%d%s"), SampleTypePrefix, TCHAR('A')+profile_tier_level.general_profile_space-1, profile_tier_level.general_profile_idc, cf, profile_tier_level.general_tier_flag ? TCHAR('H') : TCHAR('L'), profile_tier_level.general_level_idc, *cfs);
				}
			}


			FString FSequenceParameterSet::GetFormatInfo() const
			{
				return GetFormatInfo(profile_tier_level.general_profile_idc, profile_tier_level.general_level_idc, GetConstraintFlags());
			}

			FString FSequenceParameterSet::GetFormatInfo(uint32 InProfileIDC, uint32 InLevelIDC, uint64 InConstraintFlags)
			{
				FString fi;
				if (InProfileIDC == 1)
				{
					fi = TEXT("Main");
				}
				else if (InProfileIDC == 2)
				{
					fi = TEXT("Main 10");
				}
				else
				{
					fi = TEXT("Unknown");
				}
				fi.Append(TEXT(" Profile"));
				if (InLevelIDC)
				{
					fi.Append(FString::Printf(TEXT(", level %d.%d"), InLevelIDC/30, (InLevelIDC%30)/3));
				}
				else
				{
					fi.Append(TEXT(" at unknown level"));
				}
				return fi;
			}


			bool FHRDParameters::Parse(FBitstreamReaderH265& br, bool commonInfPresentFlag, uint32 maxNumSubLayersMinus1)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)															\
					if (!(expr))																				\
					{																							\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in hrd_parameters", elem);\
						return false;																			\
					}
				if (commonInfPresentFlag)
				{
					nal_hrd_parameters_present_flag = br.GetBits(1);
					vcl_hrd_parameters_present_flag = br.GetBits(1);
					if (nal_hrd_parameters_present_flag || vcl_hrd_parameters_present_flag)
					{
						sub_pic_hrd_params_present_flag = br.GetBits(1);
						if (sub_pic_hrd_params_present_flag)
						{
							tick_divisor_minus2 = br.GetBits(8);
							du_cpb_removal_delay_increment_length_minus1 = br.GetBits(5);
							sub_pic_cpb_params_in_pic_timing_sei_flag = br.GetBits(1);
							dpb_output_delay_du_length_minus1 = br.GetBits(5);
						}
						bit_rate_scale = br.GetBits(4);
						cpb_size_scale = br.GetBits(4);
						if (sub_pic_hrd_params_present_flag)
						{
							cpb_size_du_scale = br.GetBits(4);
						}
						initial_cpb_removal_delay_length_minus1 = br.GetBits(5);
						au_cpb_removal_delay_length_minus1 = br.GetBits(5);
						dpb_output_delay_length_minus1 = br.GetBits(5);
					}
				}
				SubLayers.SetNum(maxNumSubLayersMinus1+1);
				for(uint32 i=0; i<=maxNumSubLayersMinus1; ++i)
				{
					FSubLayer& sl = SubLayers[i];
					sl.fixed_pic_rate_general_flag = br.GetBits(1);
					sl.fixed_pic_rate_within_cvs_flag = sl.fixed_pic_rate_general_flag ? 1 : br.GetBits(1);
					if (sl.fixed_pic_rate_within_cvs_flag)
					{
						sl.elemental_duration_in_tc_minus1 = br.ue_v();
					}
					else
					{
						sl.low_delay_hrd_flag = br.GetBits(1);
					}
					if (!sl.low_delay_hrd_flag)
					{
						sl.cpb_cnt_minus1 = br.ue_v();
					}
					uint32 CpbCnt = sl.cpb_cnt_minus1 + 1;
					if (nal_hrd_parameters_present_flag)
					{
						sl.nal_sub_layer_hrd_parameters.cpb.SetNum(CpbCnt);
						for(uint32 j=0; j<CpbCnt; ++j)
						{
							sl.nal_sub_layer_hrd_parameters.cpb[j].bit_rate_value_minus1 = br.ue_v();
							sl.nal_sub_layer_hrd_parameters.cpb[j].cpb_size_value_minus1 = br.ue_v();
							if (sub_pic_hrd_params_present_flag)
							{
								sl.nal_sub_layer_hrd_parameters.cpb[j].cpb_size_du_value_minus1 = br.ue_v();
								sl.nal_sub_layer_hrd_parameters.cpb[j].bit_rate_du_value_minus1 = br.ue_v();
							}
							sl.nal_sub_layer_hrd_parameters.cpb[j].cbr_flag = br.GetBits(1);
						}
					}
					if (vcl_hrd_parameters_present_flag)
					{
						sl.vcl_sub_layer_hrd_parameters.cpb.SetNum(CpbCnt);
						for(uint32 j=0; j<CpbCnt; ++j)
						{
							sl.vcl_sub_layer_hrd_parameters.cpb[j].bit_rate_value_minus1 = br.ue_v();
							sl.vcl_sub_layer_hrd_parameters.cpb[j].cpb_size_value_minus1 = br.ue_v();
							if (sub_pic_hrd_params_present_flag)
							{
								sl.vcl_sub_layer_hrd_parameters.cpb[j].cpb_size_du_value_minus1 = br.ue_v();
								sl.vcl_sub_layer_hrd_parameters.cpb[j].bit_rate_du_value_minus1 = br.ue_v();
							}
							sl.vcl_sub_layer_hrd_parameters.cpb[j].cbr_flag = br.GetBits(1);
						}
					}
				}

				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool FVUIParameters::Parse(FBitstreamReaderH265& br, uint32 In_sps_max_sub_layers_minus1)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)																\
					if (!(expr))																					\
					{																								\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in vui_parameters", elem);	\
						return false;																				\
					}

				aspect_ratio_info_present_flag = br.GetBits(1);
				if (aspect_ratio_info_present_flag)
				{
					aspect_ratio_idc = br.GetBits(8);
					if (aspect_ratio_idc >= 17 && aspect_ratio_idc <= 254)
					{
						aspect_ratio_idc = 0;
					}
					if (aspect_ratio_idc == 255)
					{
						sar_width = br.GetBits(16);
						sar_height = br.GetBits(16);
					}
				}
				overscan_info_present_flag = br.GetBits(1);
				if (overscan_info_present_flag)
				{
					overscan_appropriate_flag = br.GetBits(1);
				}
				video_signal_type_present_flag = br.GetBits(1);
				if (video_signal_type_present_flag)
				{
					video_format = br.GetBits(3);
					video_format = video_format == 6 || video_format == 7 ? 5 : video_format;
					video_full_range_flag = br.GetBits(1);
					colour_description_present_flag = br.GetBits(1);
					if (colour_description_present_flag)
					{
						colour_primaries = br.GetBits(8);
						if (colour_primaries == 0 || colour_primaries == 3 || (colour_primaries >= 13 && colour_primaries <= 21) || colour_primaries >= 23)
						{
							colour_primaries = 2;
						}
						transfer_characteristics = br.GetBits(8);
						if (transfer_characteristics == 0 || transfer_characteristics == 3 || transfer_characteristics >= 19)
						{
							transfer_characteristics = 2;
						}
						matrix_coeffs = br.GetBits(8);
						if (matrix_coeffs == 3 || matrix_coeffs >= 15)
						{
							matrix_coeffs = 2;
						}
					}
				}
				chroma_loc_info_present_flag = br.GetBits(1);
				if (chroma_loc_info_present_flag)
				{
					chroma_sample_loc_type_top_field = br.ue_v();
					RANGE_CHECK_FAILURE(chroma_sample_loc_type_top_field <= 5, TEXT("chroma_sample_loc_type_top_field"));
					chroma_sample_loc_type_bottom_field = br.ue_v();
					RANGE_CHECK_FAILURE(chroma_sample_loc_type_bottom_field <= 5, TEXT("chroma_sample_loc_type_bottom_field"));
				}
				neutral_chroma_indication_flag = br.GetBits(1);
				field_seq_flag = br.GetBits(1);
				frame_field_info_present_flag = br.GetBits(1);
				default_display_window_flag = br.GetBits(1);
				if (default_display_window_flag)
				{
					def_disp_win_left_offset = br.ue_v();
					def_disp_win_right_offset = br.ue_v();
					def_disp_win_top_offset = br.ue_v();
					def_disp_win_bottom_offset = br.ue_v();
				}
				vui_timing_info_present_flag = br.GetBits(1);
				if (vui_timing_info_present_flag)
				{
					vui_num_units_in_tick = br.GetBits(32);
					vui_time_scale = br.GetBits(32);
					vui_poc_proportional_to_timing_flag = br.GetBits(1);
					if (vui_poc_proportional_to_timing_flag)
					{
						vui_num_ticks_poc_diff_one_minus1 = br.ue_v();
					}
					vui_hrd_parameters_present_flag = br.GetBits(1);
					if (vui_hrd_parameters_present_flag)
					{
						hrd_parameters.Parse(br, 1, In_sps_max_sub_layers_minus1);
					}
				}
				bitstream_restriction_flag = br.GetBits(1);
				if (bitstream_restriction_flag)
				{
					tiles_fixed_structure_flag = br.GetBits(1);
					motion_vectors_over_pic_boundaries_flag = br.GetBits(1);
					restricted_ref_pic_lists_flag = br.GetBits(1);
					min_spatial_segmentation_idc = br.ue_v();
					RANGE_CHECK_FAILURE(min_spatial_segmentation_idc <= 4095, TEXT("min_spatial_segmentation_idc"));
					max_bytes_per_pic_denom = br.ue_v();
					RANGE_CHECK_FAILURE(max_bytes_per_pic_denom <= 16, TEXT("max_bytes_per_pic_denom"));
					max_bits_per_min_cu_denom = br.ue_v();
					RANGE_CHECK_FAILURE(max_bits_per_min_cu_denom <= 16, TEXT("max_bits_per_min_cu_denom"));
					log2_max_mv_length_horizontal = br.ue_v();
					RANGE_CHECK_FAILURE(log2_max_mv_length_horizontal <= 15, TEXT("log2_max_mv_length_horizontal"));
					log2_max_mv_length_vertical = br.ue_v();
					RANGE_CHECK_FAILURE(log2_max_mv_length_vertical <= 15, TEXT("log2_max_mv_length_vertical"));
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool FScalingListData::Parse(FBitstreamReaderH265& br)
			{
				for(uint32 sizeId=0; sizeId<4; ++sizeId)
				{
					const uint8* ScanOrder = sizeId==0 ? ScanOrderDiag4 : ScanOrderDiag8;
					for(uint32 matrixId=0; matrixId<6; ++matrixId)
					{
						if (sizeId == 3 && matrixId != 0 && matrixId != 3)
						{
							scaling_list[sizeId][matrixId] = scaling_list[sizeId - 1][matrixId];
							scaling_list_dc[sizeId][matrixId] = scaling_list_dc[sizeId - 1][matrixId];
						}
						else
						{
							uint32 scaling_list_pred_mode_flag = br.GetBits(1);
							uint8* ScalingList = scaling_list[sizeId][matrixId].GetData();
							// Prediction or copy mode?
							if (!scaling_list_pred_mode_flag)
							{
								uint32 scaling_list_pred_matrix_id_delta = br.ue_v();
								if (sizeId == 3)
								{
									scaling_list_pred_matrix_id_delta *= 3;
								}
								int32 ref_matrix_id = (int32)matrixId - (int32)scaling_list_pred_matrix_id_delta;
								check(ref_matrix_id >= 0);
								if (sizeId > 1)
								{
									scaling_list_dc[sizeId][matrixId] = matrixId==ref_matrix_id ? 16 : scaling_list_dc[sizeId][ref_matrix_id];
								}
								if (matrixId == ref_matrix_id)
								{
									if (sizeId == 0)
									{
										FMemory::Memcpy(ScalingList, Default_4x4_Diag, sizeof(Default_4x4_Diag));
									}
									else if (matrixId < 3)
									{
										FMemory::Memcpy(ScalingList, Default_8x8_Intra_Diag, sizeof(Default_8x8_Intra_Diag));
									}
									else
									{
										FMemory::Memcpy(ScalingList, Default_8x8_Inter_Diag, sizeof(Default_8x8_Inter_Diag));
									}
								}
								else
								{
									scaling_list[sizeId][matrixId] = scaling_list[sizeId][ref_matrix_id];
								}
							}
							else
							{
								const int32 coefNum = sizeId==0 ? 16 : 64;
								int32 nextCoef = 8;
								if (sizeId > 1)
								{
									int32 scaling_list_dc_coef_minus8 = br.se_v();
									nextCoef = scaling_list_dc_coef_minus8 + 8;
									check(nextCoef >=0 && nextCoef <= 255);
									scaling_list_dc[sizeId][matrixId] = (uint8) nextCoef;
								}
								for(int32 i=0; i<coefNum; ++i)
								{
									int32 scaling_list_delta_coef = br.se_v();
									nextCoef = (nextCoef + scaling_list_delta_coef + 256) % 256;
									check(nextCoef >=0 && nextCoef <= 255);
									ScalingList[ScanOrder[i]] = (uint8) nextCoef;
								}
							}
						}
					}
				}
				return true;
			}

			void FScalingListData::SetDefaults()
			{
				FMemory::Memset(scaling_list_dc, 16);
				check(scaling_list[0][0].Num());	// Arrays must have been created already
				for(int32 i=0; i<6; ++i)
				{
					FMemory::Memset(scaling_list[0][i].GetData(), 16, 16);
				}
				for(int32 i=0; i<6; ++i)
				{
					if (i < 3)
					{
						FMemory::Memcpy(scaling_list[1][i].GetData(), Default_8x8_Intra_Diag, 64);
						FMemory::Memcpy(scaling_list[2][i].GetData(), Default_8x8_Intra_Diag, 64);
						FMemory::Memcpy(scaling_list[3][i].GetData(), Default_8x8_Intra_Diag, 64);
					}
					else
					{
						FMemory::Memcpy(scaling_list[1][i].GetData(), Default_8x8_Inter_Diag, 64);
						FMemory::Memcpy(scaling_list[2][i].GetData(), Default_8x8_Inter_Diag, 64);
						FMemory::Memcpy(scaling_list[3][i].GetData(), Default_8x8_Inter_Diag, 64);
					}
				}
			}

			bool FProfileTierLevel::Parse(FBitstreamReaderH265& br, bool profilePresentFlag, uint32 maxNumSubLayersMinus1)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)																\
					if (!(expr))																					\
					{																								\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in ProfileTierLevel", elem);	\
						return false;																				\
					}
				if (profilePresentFlag)
				{
					general_profile_space = br.GetBits(2);
					general_tier_flag = br.GetBits(1);
					general_profile_idc = br.GetBits(5);
					general_profile_compatibility_flags = br.PeekBits(32);
					for(int32 i=0; i<32; ++i)
					{
						general_profile_compatibility_flag[i] = br.GetBits(1);
					}
					general_constraint_indicator_flags = FBitstreamReaderH265(br).GetBits64(48);	// elaborate PeekBits(48)
					general_progressive_source_flag = br.GetBits(1);
					general_interlaced_source_flag = br.GetBits(1);
					general_non_packed_constraint_flag = br.GetBits(1);
					general_frame_only_constraint_flag = br.GetBits(1);
					if (general_profile_idc == 4 || general_profile_compatibility_flag[4] ||
						general_profile_idc == 5 || general_profile_compatibility_flag[5] ||
						general_profile_idc == 6 || general_profile_compatibility_flag[6] ||
						general_profile_idc == 7 || general_profile_compatibility_flag[7] ||
						general_profile_idc == 8 || general_profile_compatibility_flag[8] ||
						general_profile_idc == 9 || general_profile_compatibility_flag[9] ||
						general_profile_idc == 10 || general_profile_compatibility_flag[10] ||
						general_profile_idc == 11 || general_profile_compatibility_flag[11])
					{
						general_max_12bit_constraint_flag = br.GetBits(1);
						general_max_10bit_constraint_flag = br.GetBits(1);
						general_max_8bit_constraint_flag = br.GetBits(1);
						general_max_422chroma_constraint_flag = br.GetBits(1);
						general_max_420chroma_constraint_flag = br.GetBits(1);
						general_max_monochrome_constraint_flag = br.GetBits(1);
						general_intra_constraint_flag = br.GetBits(1);
						general_one_picture_only_constraint_flag = br.GetBits(1);
						general_lower_bit_rate_constraint_flag = br.GetBits(1);
						if (general_profile_idc == 5 || general_profile_compatibility_flag[5] ||
							general_profile_idc == 9 || general_profile_compatibility_flag[9] ||
							general_profile_idc == 10 || general_profile_compatibility_flag[10] ||
							general_profile_idc == 11 || general_profile_compatibility_flag[11])
						{
							general_max_14bit_constraint_flag = br.GetBits(1);
							uint64 general_reserved_zero_33bits = br.GetBits64(33);
							RANGE_CHECK_FAILURE(general_reserved_zero_33bits == 0, TEXT("general_reserved_zero_33bits"));
						}
						else
						{
							uint64 general_reserved_zero_34bits = br.GetBits64(34);
							RANGE_CHECK_FAILURE(general_reserved_zero_34bits == 0, TEXT("general_reserved_zero_34bits"));
						}
					}
					else if (general_profile_idc == 2 || general_profile_compatibility_flag[2])
					{
						uint8 general_reserved_zero_7bits = br.GetBits(7);
						RANGE_CHECK_FAILURE(general_reserved_zero_7bits == 0, TEXT("general_reserved_zero_7bits"));
						general_one_picture_only_constraint_flag = br.GetBits(1);
						uint64 general_reserved_zero_35bits = br.GetBits64(35);
						RANGE_CHECK_FAILURE(general_reserved_zero_35bits == 0, TEXT("general_reserved_zero_35bits"));
					}
					else
					{
						uint64 general_reserved_zero_43bits = br.GetBits64(43);
						RANGE_CHECK_FAILURE(general_reserved_zero_43bits == 0, TEXT("general_reserved_zero_43bits"));
					}
					if (general_profile_idc == 1 || general_profile_compatibility_flag[1] ||
						general_profile_idc == 2 || general_profile_compatibility_flag[2] ||
						general_profile_idc == 3 || general_profile_compatibility_flag[3] ||
						general_profile_idc == 4 || general_profile_compatibility_flag[4] ||
						general_profile_idc == 5 || general_profile_compatibility_flag[5] ||
						general_profile_idc == 9 || general_profile_compatibility_flag[9] ||
						general_profile_idc == 11 || general_profile_compatibility_flag[11])
					{
						general_inbld_flag = br.GetBits(1);
					}
					else
					{
						uint8 general_reserved_zero_bit = br.GetBits(1);
						RANGE_CHECK_FAILURE(general_reserved_zero_bit == 0, TEXT("general_reserved_zero_bit"));
					}
				}
				general_level_idc = br.GetBits(8);
				for(uint32 i=0; i<maxNumSubLayersMinus1; ++i)
				{
					sub_layer_profile_present_flag[i] = br.GetBits(1);
					sub_layer_level_present_flag[i] = br.GetBits(1);
				}
				if (maxNumSubLayersMinus1)
				{
					for(uint32 i=maxNumSubLayersMinus1; i<8; ++i)
					{
						uint8 reserved_zero_2bits = br.GetBits(2);
						RANGE_CHECK_FAILURE(reserved_zero_2bits == 0, TEXT("reserved_zero_2bits"));
					}
				}
				for(uint32 i=0; i<maxNumSubLayersMinus1; ++i)
				{
					if (sub_layer_profile_present_flag[i])
					{
						FSubLayer& sl = sub_layers[i];
						sl.sub_layer_profile_space = br.GetBits(2);
						sl.sub_layer_tier_flag = br.GetBits(1);
						sl.sub_layer_profile_idc = br.GetBits(5);
						for(int32 j=0; j<32; ++j)
						{
							sl.sub_layer_profile_compatibility_flag[j] = br.GetBits(1);
						}
						sl.sub_layer_progressive_source_flag = br.GetBits(1);
						sl.sub_layer_interlaced_source_flag = br.GetBits(1);
						sl.sub_layer_non_packed_constraint_flag = br.GetBits(1);
						sl.sub_layer_frame_only_constraint_flag = br.GetBits(1);
						if (sl.sub_layer_profile_idc == 4 || sl.sub_layer_profile_compatibility_flag[4] ||
							sl.sub_layer_profile_idc == 5 || sl.sub_layer_profile_compatibility_flag[5] ||
							sl.sub_layer_profile_idc == 6 || sl.sub_layer_profile_compatibility_flag[6] ||
							sl.sub_layer_profile_idc == 7 || sl.sub_layer_profile_compatibility_flag[7] ||
							sl.sub_layer_profile_idc == 8 || sl.sub_layer_profile_compatibility_flag[8] ||
							sl.sub_layer_profile_idc == 9 || sl.sub_layer_profile_compatibility_flag[9] ||
							sl.sub_layer_profile_idc == 10 || sl.sub_layer_profile_compatibility_flag[10] ||
							sl.sub_layer_profile_idc == 11 || sl.sub_layer_profile_compatibility_flag[11])
						{
							sl.sub_layer_max_12bit_constraint_flag = br.GetBits(1);
							sl.sub_layer_max_10bit_constraint_flag = br.GetBits(1);
							sl.sub_layer_max_8bit_constraint_flag = br.GetBits(1);
							sl.sub_layer_max_422chroma_constraint_flag = br.GetBits(1);
							sl.sub_layer_max_420chroma_constraint_flag = br.GetBits(1);
							sl.sub_layer_max_monochrome_constraint_flag = br.GetBits(1);
							sl.sub_layer_intra_constraint_flag = br.GetBits(1);
							sl.sub_layer_one_picture_only_constraint_flag = br.GetBits(1);
							sl.sub_layer_lower_bit_rate_constraint_flag = br.GetBits(1);
							if (sl.sub_layer_profile_idc == 5 || sl.sub_layer_profile_compatibility_flag[5] ||
								sl.sub_layer_profile_idc == 9 || sl.sub_layer_profile_compatibility_flag[9] ||
								sl.sub_layer_profile_idc == 10 || sl.sub_layer_profile_compatibility_flag[10] ||
								sl.sub_layer_profile_idc == 11 || sl.sub_layer_profile_compatibility_flag[11])
							{
								sl.sub_layer_max_14bit_constraint_flag = br.GetBits(1);
								uint64 sub_layer_reserved_zero_33bits = br.GetBits64(33);
								RANGE_CHECK_FAILURE(sub_layer_reserved_zero_33bits == 0, TEXT("sub_layer_reserved_zero_33bits"));
							}
							else
							{
								uint64 sub_layer_reserved_zero_34bits = br.GetBits64(34);
								RANGE_CHECK_FAILURE(sub_layer_reserved_zero_34bits == 0, TEXT("sub_layer_reserved_zero_34bits"));
							}
						}
						else if (sl.sub_layer_profile_idc == 2 || sl.sub_layer_profile_compatibility_flag[2])
						{
							uint8 sub_layer_reserved_zero_7bits = br.GetBits(7);
							RANGE_CHECK_FAILURE(sub_layer_reserved_zero_7bits == 0, TEXT("sub_layer_reserved_zero_7bits"));
							sl.sub_layer_one_picture_only_constraint_flag = br.GetBits(1);
							uint64 sub_layer_reserved_zero_35bits = br.GetBits64(35);
							RANGE_CHECK_FAILURE(sub_layer_reserved_zero_35bits == 0, TEXT("sub_layer_reserved_zero_35bits"));
						}
						else
						{
							uint64 sub_layer_reserved_zero_43bits = br.GetBits64(43);
							RANGE_CHECK_FAILURE(sub_layer_reserved_zero_43bits == 0, TEXT("sub_layer_reserved_zero_43bits"));
						}
						if (sl.sub_layer_profile_idc == 1 || sl.sub_layer_profile_compatibility_flag[1] ||
							sl.sub_layer_profile_idc == 2 || sl.sub_layer_profile_compatibility_flag[2] ||
							sl.sub_layer_profile_idc == 3 || sl.sub_layer_profile_compatibility_flag[3] ||
							sl.sub_layer_profile_idc == 4 || sl.sub_layer_profile_compatibility_flag[4] ||
							sl.sub_layer_profile_idc == 5 || sl.sub_layer_profile_compatibility_flag[5] ||
							sl.sub_layer_profile_idc == 9 || sl.sub_layer_profile_compatibility_flag[9] ||
							sl.sub_layer_profile_idc == 11 || sl.sub_layer_profile_compatibility_flag[11])
						{
							sl.sub_layer_inbld_flag = br.GetBits(1);
						}
						else
						{
							uint8 sub_layer_reserved_zero_bit = br.GetBits(1);
							RANGE_CHECK_FAILURE(sub_layer_reserved_zero_bit == 0, TEXT("sub_layer_reserved_zero_bit"));
						}
					}
					if (sub_layer_level_present_flag[i])
					{
						sub_layers[i].sub_layer_level_idc = br.GetBits(8);
					}
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool FStRefPicSet::Parse(FBitstreamReaderH265& br, int32 stRpsIdx, const TArray<FStRefPicSet>& InReferencePicSetList, int32 InReferencePicSetListNum)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)																\
					if (!(expr))																					\
					{																								\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in st_ref_pic_set", elem);	\
						return false;																				\
					}
				inter_ref_pic_set_prediction_flag = stRpsIdx == 0 ? 0 : br.GetBits(1);
				if (inter_ref_pic_set_prediction_flag)
				{
					bool bParseFromSliceHeader = stRpsIdx == InReferencePicSetListNum;
					delta_idx_minus1 = bParseFromSliceHeader ? br.ue_v() : 0U;
					RANGE_CHECK_FAILURE((int32)delta_idx_minus1 < stRpsIdx, TEXT("delta_idx_minus1"));
					int32 RefRpsIdx = stRpsIdx - 1 - (int32)delta_idx_minus1;
					RANGE_CHECK_FAILURE(RefRpsIdx >= 0 && RefRpsIdx < stRpsIdx, TEXT("delta_idx_minus1"));
					const FStRefPicSet& RefPicSet = InReferencePicSetList[RefRpsIdx];
					NumDeltaPocsInSliceReferencedSet = bParseFromSliceHeader ? RefPicSet.NumDeltaPocs() : 0;

					delta_rps_sign = br.GetBits(1);
					abs_delta_rps_minus1 = br.ue_v();
					RANGE_CHECK_FAILURE(abs_delta_rps_minus1 <= 32767, TEXT("abs_delta_rps_minus1"));
					int32 delta_rps = (1 - (int32)(delta_rps_sign << 1)) * (int32)(abs_delta_rps_minus1 + 1);
					uint32 k=0, k0=0;
					for(uint32 i=0, iMax=RefPicSet.NumDeltaPocs(); i<=iMax; ++i)
					{
						used_by_curr_pic_flag[k] = br.GetBits(1);
						raw_values.use_delta_flag[i] = !used_by_curr_pic_flag[k] ? br.GetBits(1) : 1;
						if (used_by_curr_pic_flag[k] != 0 || raw_values.use_delta_flag[i] != 0)
						{
							int32 deltaPOC = delta_rps + (i<iMax ? RefPicSet.GetDeltaPOC(i) : 0);
							delta_poc[k] = deltaPOC;
							if (deltaPOC < 0)
							{
								++k0;
							}
							++k;
						}
					}
					num_negative_pics = k0;
					num_positive_pics = k - k0;
					SortDeltaPOC();
				}
				else
				{
					num_negative_pics = br.ue_v();
					num_positive_pics = br.ue_v();
					RANGE_CHECK_FAILURE(num_negative_pics <= 16, TEXT("num_negative_pics"));
					RANGE_CHECK_FAILURE(num_positive_pics <= 16, TEXT("num_positive_pics"));

					int32 PrevPOC = 0;
					for(uint32 i=0; i<num_negative_pics; ++i)
					{
						raw_values.delta_poc_s0_minus1[i] = br.ue_v();
						RANGE_CHECK_FAILURE(raw_values.delta_poc_s0_minus1[i] <= 32767, TEXT("delta_poc_s0_minus1"));
						PrevPOC -= (int32)raw_values.delta_poc_s0_minus1[i] + 1;
						delta_poc[i] = PrevPOC;
						used_by_curr_pic_flag[i] = raw_values.used_by_curr_pic_s0_flag[i] = br.GetBits(1);
					}
					PrevPOC = 0;
					for(uint32 i=0; i<num_positive_pics; ++i)
					{
						raw_values.delta_poc_s1_minus1[i] = br.ue_v();
						RANGE_CHECK_FAILURE(raw_values.delta_poc_s1_minus1[i] <= 32767, TEXT("delta_poc_s1_minus1"));
						PrevPOC += (int32)raw_values.delta_poc_s1_minus1[i] + 1;
						delta_poc[i + num_negative_pics] = PrevPOC;
						used_by_curr_pic_flag[i + num_negative_pics] = raw_values.used_by_curr_pic_s1_flag[i] = br.GetBits(1);
					}
				}
				#undef RANGE_CHECK_FAILURE
				return true;
			}

  			void FStRefPicSet::SortDeltaPOC()
			{
				// Sort all by delta_poc in ascending order.
				if (NumDeltaPocs())
				{
					for(int32 i=1, iMax=(int32)NumDeltaPocs(); i<iMax; ++i)
					{
						const int32 dp = delta_poc[i];
						const uint8 temp_used_by_curr_pic_flag = used_by_curr_pic_flag[i];
						for(int32 j=i-1; j>=0; --j)
						{
							const int32 temp_delta_poc = delta_poc[j];
							if (dp < temp_delta_poc)
							{
								delta_poc[j+1] = temp_delta_poc;
								used_by_curr_pic_flag[j+1] = used_by_curr_pic_flag[j];
								delta_poc[j] = dp;
								used_by_curr_pic_flag[j] = temp_used_by_curr_pic_flag;
							}
						}
					}
				}
				// Sort the negative delta_pocs in descending order
				const int32 numNegPics = (int32)NumNegativePics();
				for(int32 i=0, j=numNegPics-1, iMax=numNegPics>>1; i<iMax; ++i, --j)
				{
					int32 temp_delta_poc = delta_poc[i];
					uint8 temp_used_by_curr_pic_flag = used_by_curr_pic_flag[i];
					delta_poc[i] = delta_poc[j];
					used_by_curr_pic_flag[i] = used_by_curr_pic_flag[j];
					delta_poc[j] = temp_delta_poc;
					used_by_curr_pic_flag[j] = temp_used_by_curr_pic_flag;
				}
			}

			bool ELECTRADECODERS_API ParseVideoParameterSet(TMap<uint32, FVideoParameterSet>& InOutVideoParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)													\
					if (!(expr))																		\
					{																					\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in VPS", elem);	\
						return false;																	\
					}
				TUniquePtr<FRBSP> RBSP(MakeRBSP(InBitstream, InBitstreamLenInBytes));
				FBitstreamReaderH265 br(RBSP->Data, RBSP->Size);

				// Verify that this is in fact a VPS NALU.
				if ((br.GetBits(16) >> 9) != 32)
				{
					UE_LOGF(LogElectraDecoders, Error, "Not a VPS NALU!");
					return false;
				}
				if (br.GetRemainingByteLength() < 4)
				{
					UE_LOGF(LogElectraDecoders, Error, "Insufficient bytes in bitstream to be a VPS NALU!");
					return false;
				}

				uint32 temp_vps_video_parameter_set_id = br.GetBits(4);

				FVideoParameterSet* vpsPtr = InOutVideoParameterSets.Find(temp_vps_video_parameter_set_id);
				if (!vpsPtr)
				{
					vpsPtr = &InOutVideoParameterSets.Emplace(temp_vps_video_parameter_set_id);
					check(vpsPtr);
				}
				else
				{
					vpsPtr->Reset();
				}
				FVideoParameterSet& vps = *vpsPtr;

				vps.vps_video_parameter_set_id = (uint8) temp_vps_video_parameter_set_id;
				vps.vps_base_layer_internal_flag = br.GetBits(1);
				vps.vps_base_layer_available_flag = br.GetBits(1);
				if (!vps.vps_base_layer_internal_flag && vps.vps_base_layer_available_flag)
				{
					UE_LOGF(LogElectraDecoders, Error, "vps_base_layer_internal_flag==0 && vps_base_layer_available_flag==1 is not supported!");
					return false;
				}
				vps.vps_max_layers_minus1 = br.GetBits(6);
				vps.vps_max_sub_layers_minus1 = br.GetBits(3);
				RANGE_CHECK_FAILURE(vps.vps_max_sub_layers_minus1 <= 6, TEXT("vps_max_sub_layers_minus1"));
				vps.vps_temporal_id_nesting_flag = br.GetBits(1);
				uint32 vps_reserved_0xffff_16bits = br.GetBits(16);
				//RANGE_CHECK_FAILURE(vps_reserved_0xffff_16bits == 0xffff, TEXT("vps_reserved_0xffff_16bits"));
				// Profile tier level
				if (!vps.profile_tier_level.Parse(br, true, vps.vps_max_sub_layers_minus1))
				{
					return false;
				}
				vps.vps_sub_layer_ordering_info_present_flag = br.GetBits(1);
				for(uint32 i=vps.vps_sub_layer_ordering_info_present_flag ? 0 : vps.vps_max_sub_layers_minus1; i<=vps.vps_max_sub_layers_minus1; ++i)
				{
					vps.vps_max_dec_pic_buffering_minus1[i] = br.ue_v();
					vps.vps_max_num_reorder_pics[i] = br.ue_v();
					vps.vps_max_latency_increase_plus1[i] = br.ue_v();
				}
				vps.vps_max_layer_id = br.GetBits(6);
				vps.vps_num_layer_sets_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(vps.vps_num_layer_sets_minus1 <= 1023, TEXT("vps_num_layer_sets_minus1"));
				vps.layer_id_included_flag.SetNum(vps.vps_num_layer_sets_minus1 + 1);
				vps.layer_id_included_flag[0].SetNum(1);
				vps.layer_id_included_flag[0][0] = 1;
				for(uint32 i=1; i<=vps.vps_num_layer_sets_minus1; ++i)
				{
					vps.layer_id_included_flag[i].SetNum(vps.vps_max_layer_id + 1);
					for(uint32 j=0; j<=vps.vps_max_layer_id; ++j)
					{
						vps.layer_id_included_flag[i][j] = br.GetBits(1);
					}
				}

				vps.vps_timing_info_present_flag = br.GetBits(1);
				if (vps.vps_timing_info_present_flag)
				{
					vps.vps_num_units_in_tick = br.GetBits(32);
					vps.vps_time_scale = br.GetBits(32);
					vps.vps_poc_proportional_to_timing_flag = br.GetBits(1);
					if (vps.vps_poc_proportional_to_timing_flag)
					{
						vps.vps_num_ticks_poc_diff_one_minus1 = br.ue_v();
					}
					uint32 vps_num_hrd_parameters = br.ue_v();
					RANGE_CHECK_FAILURE(vps_num_hrd_parameters <= vps.vps_num_layer_sets_minus1+1, TEXT("vps_num_hrd_parameters"));
					vps.ListOf_hrd_parameters.SetNum(vps_num_hrd_parameters);
					for(uint32 i=0; i<vps_num_hrd_parameters; ++i)
					{
						vps.ListOf_hrd_parameters[i].hrd_layer_set_idx = br.ue_v();
						if (i == 0)
						{
							vps.ListOf_hrd_parameters[i].cprms_present_flag = 1;
						}
						else
						{
							vps.ListOf_hrd_parameters[i].cprms_present_flag = br.GetBits(1);
						}
						if (!vps.ListOf_hrd_parameters[i].hrd_parameter.Parse(br, !!vps.ListOf_hrd_parameters[i].cprms_present_flag, vps.vps_max_sub_layers_minus1))
						{
							return false;
						}
					}
				}

				vps.vps_extension_flag = br.GetBits(1);
				if (vps.vps_extension_flag)
				{
					while(br.more_rbsp_data())
					{
						/*uint8 vps_extension_data_flag =*/ br.GetBits(1);
					}
				}

				if (!br.rbsp_trailing_bits())
				{
					UE_LOGF(LogElectraDecoders, Error, "Bad rbsp_trailing_bits in VPS NALU!");
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool ParseVideoParameterSet(FVideoParameterSet& OutVideoParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				TMap<uint32, FVideoParameterSet> vpss;
				if (!ParseVideoParameterSet(vpss, InBitstream, InBitstreamLenInBytes))
				{
					return false;
				}
				OutVideoParameterSet = vpss.CreateConstIterator().Value();
				return true;
			}

			bool ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				TMap<uint32, FSequenceParameterSet> spss;
				if (!ParseSequenceParameterSet(spss, InBitstream, InBitstreamLenInBytes))
				{
					return false;
				}
				OutSequenceParameterSet = spss.CreateConstIterator().Value();
				return true;
			}

			bool ParseSequenceParameterSet(TMap<uint32, FSequenceParameterSet>& InOutSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)													\
					if (!(expr))																		\
					{																					\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in SPS", elem);	\
						return false;																	\
					}

				TUniquePtr<FRBSP> RBSP(MakeRBSP(InBitstream, InBitstreamLenInBytes));
				FBitstreamReaderH265 br(RBSP->Data, RBSP->Size);

				// Verify that this is in fact an SPS NALU.
				if ((br.GetBits(16) >> 9) != 33)
				{
					UE_LOGF(LogElectraDecoders, Error, "Not an SPS NALU!");
					return false;
				}
				if (br.GetRemainingByteLength() < 4)
				{
					UE_LOGF(LogElectraDecoders, Error, "Insufficient bytes in bitstream to be an SPS NALU!");
					return false;
				}

				uint8 temp_sps_video_parameter_set_id = br.GetBits(4);
				uint8 temp_sps_max_sub_layers_minus1 = br.GetBits(3);
				RANGE_CHECK_FAILURE(temp_sps_max_sub_layers_minus1 <= 6, TEXT("sps_max_sub_layers_minus1"));
				uint8 temp_sps_temporal_id_nesting_flag = br.GetBits(1);
				FProfileTierLevel temp_profile_tier_level;
				if (!temp_profile_tier_level.Parse(br, true, temp_sps_max_sub_layers_minus1))
				{
					return false;
				}
				uint8 temp_sps_seq_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(temp_sps_seq_parameter_set_id <= 15, TEXT("seq_parameter_set_id"));

				FSequenceParameterSet* spsPtr = InOutSequenceParameterSets.Find(temp_sps_seq_parameter_set_id);
				if (!spsPtr)
				{
					spsPtr = &InOutSequenceParameterSets.Emplace(temp_sps_seq_parameter_set_id);
					check(spsPtr);
				}
				else
				{
					spsPtr->Reset();
				}
				FSequenceParameterSet& sps = *spsPtr;
				sps.sps_video_parameter_set_id = temp_sps_video_parameter_set_id;
				sps.sps_max_sub_layers_minus1 = temp_sps_max_sub_layers_minus1;
				sps.sps_temporal_id_nesting_flag = temp_sps_temporal_id_nesting_flag;
				sps.profile_tier_level = temp_profile_tier_level;
				sps.sps_seq_parameter_set_id = temp_sps_seq_parameter_set_id;
				sps.chroma_format_idc = br.ue_v();
				RANGE_CHECK_FAILURE(sps.chroma_format_idc <= 3, TEXT("chroma_format_idc"));
				if (sps.chroma_format_idc == 3)
				{
					sps.separate_colour_plane_flag = br.GetBits(1);
				}
				sps.pic_width_in_luma_samples = br.ue_v();
				RANGE_CHECK_FAILURE(sps.pic_width_in_luma_samples != 0, TEXT("pic_width_in_luma_samples"));
				sps.pic_height_in_luma_samples = br.ue_v();
				RANGE_CHECK_FAILURE(sps.pic_height_in_luma_samples != 0, TEXT("pic_height_in_luma_samples"));
				sps.conformance_window_flag = br.GetBits(1);
				if (sps.conformance_window_flag)
				{
					sps.conf_win_left_offset = br.ue_v();
					sps.conf_win_right_offset = br.ue_v();
					sps.conf_win_top_offset = br.ue_v();
					sps.conf_win_bottom_offset = br.ue_v();
				}
				sps.bit_depth_luma_minus8 = br.ue_v();
				RANGE_CHECK_FAILURE(sps.bit_depth_luma_minus8 <= 8, TEXT("bit_depth_luma_minus8"));
				sps.bit_depth_chroma_minus8 = br.ue_v();
				RANGE_CHECK_FAILURE(sps.bit_depth_chroma_minus8 <= 8, TEXT("bit_depth_chroma_minus8"));
				sps.log2_max_pic_order_cnt_lsb_minus4 = br.ue_v();
				RANGE_CHECK_FAILURE(sps.log2_max_pic_order_cnt_lsb_minus4 <= 12, TEXT("log2_max_pic_order_cnt_lsb_minus4"));
				sps.sps_sub_layer_ordering_info_present_flag = br.GetBits(1);
				for(int32 i=sps.sps_sub_layer_ordering_info_present_flag ? 0 : sps.sps_max_sub_layers_minus1; i<= sps.sps_max_sub_layers_minus1; ++i)
				{
					// Note: `sps_max_dec_pic_buffering_minus1` has to be within 0 to MaxDpbSize-1
					sps.sps_max_dec_pic_buffering_minus1[i] = br.ue_v();
					sps.sps_max_num_reorder_pics[i] = br.ue_v();
					RANGE_CHECK_FAILURE(sps.sps_max_num_reorder_pics[i] <= sps.sps_max_dec_pic_buffering_minus1[i], TEXT("sps_max_num_reorder_pics"));
					RANGE_CHECK_FAILURE(i==0 ||(sps.sps_max_num_reorder_pics[i] >= sps.sps_max_num_reorder_pics[i-1]), TEXT("sps_max_num_reorder_pics"));
					sps.sps_max_latency_increase_plus1[i] = br.ue_v();
				}
				// Set default layer values if sps_sub_layer_ordering_info_present_flag == 0
				if(!sps.sps_sub_layer_ordering_info_present_flag)
				{
					for(int32 i=0; i<UE_ARRAY_COUNT(sps.sps_max_dec_pic_buffering_minus1); ++i)
					{
						if (i != sps.sps_max_sub_layers_minus1)
						{
							sps.sps_max_dec_pic_buffering_minus1[i] = sps.sps_max_dec_pic_buffering_minus1[sps.sps_max_sub_layers_minus1];
							sps.sps_max_num_reorder_pics[i] = sps.sps_max_num_reorder_pics[sps.sps_max_sub_layers_minus1];
							sps.sps_max_latency_increase_plus1[i] = sps.sps_max_latency_increase_plus1[sps.sps_max_sub_layers_minus1];
						}
					}
				}
				sps.log2_min_luma_coding_block_size_minus3 = br.ue_v();
				uint32 MinCbSizeY = 1U << (sps.log2_min_luma_coding_block_size_minus3 + 3);
				RANGE_CHECK_FAILURE(sps.pic_width_in_luma_samples % MinCbSizeY == 0, TEXT("pic_width_in_luma_samples"));
				RANGE_CHECK_FAILURE(sps.pic_height_in_luma_samples % MinCbSizeY == 0, TEXT("pic_height_in_luma_samples"));
				sps.log2_diff_max_min_luma_coding_block_size = br.ue_v();
				sps.log2_min_luma_transform_block_size_minus2 = br.ue_v();
				sps.log2_diff_max_min_luma_transform_block_size = br.ue_v();
				sps.max_transform_hierarchy_depth_inter = br.ue_v();
				sps.max_transform_hierarchy_depth_intra = br.ue_v();
				sps.scaling_list_enabled_flag = br.GetBits(1);
				if (sps.scaling_list_enabled_flag)
				{
					sps.sps_scaling_list_data_present_flag = br.GetBits(1);
					if (sps.sps_scaling_list_data_present_flag)
					{
						if (!sps.scaling_list_data.Parse(br))
						{
							return false;
						}
					}
					else
					{
						sps.scaling_list_data.SetDefaults();
					}
				}
				sps.amp_enabled_flag = br.GetBits(1);
				sps.sample_adaptive_offset_enabled_flag = br.GetBits(1);
				sps.pcm_enabled_flag = br.GetBits(1);
				if (sps.pcm_enabled_flag)
				{
					sps.pcm_sample_bit_depth_luma_minus1 = br.GetBits(4);
					sps.pcm_sample_bit_depth_chroma_minus1 = br.GetBits(4);
					sps.log2_min_pcm_luma_coding_block_size_minus3 = br.ue_v();
					sps.log2_diff_max_min_pcm_luma_coding_block_size = br.ue_v();
					sps.pcm_loop_filter_disabled_flag = br.GetBits(1);
				}
				sps.num_short_term_ref_pic_sets = br.ue_v();
				RANGE_CHECK_FAILURE(sps.num_short_term_ref_pic_sets <= 64, TEXT("num_short_term_ref_pic_sets"));
				// IMPORTANT:
				//	 As per NOTE 5 of section 7.4.3.2.1 General sequence parameter set RBSP semantics
				//   we add +1 to the size of the array.
				//	 Therefore make sure to always use either `sps.num_short_term_ref_pic_sets` or `sps.st_ref_pic_set.Num()-1`
				//	 when the number of elements in the SPS is to be used!
				sps.st_ref_pic_set.SetNum(sps.num_short_term_ref_pic_sets + 1);
				for(uint32 i=0; i<sps.num_short_term_ref_pic_sets; ++i)
				{
					if (!sps.st_ref_pic_set[i].Parse(br, i, sps.st_ref_pic_set, sps.num_short_term_ref_pic_sets))
					{
						return false;
					}
				}

				sps.long_term_ref_pics_present_flag = br.GetBits(1);
				if (sps.long_term_ref_pics_present_flag)
				{
					sps.num_long_term_ref_pics_sps = br.ue_v();
					RANGE_CHECK_FAILURE(sps.num_long_term_ref_pics_sps <= 32, TEXT("num_long_term_ref_pics_sps"));
					sps.long_term_ref_pics.SetNum(sps.num_long_term_ref_pics_sps);
					for(uint32 i=0; i<sps.num_long_term_ref_pics_sps; ++i)
					{
						sps.long_term_ref_pics[i].lt_ref_pic_poc_lsb_sps = br.GetBits(sps.log2_max_pic_order_cnt_lsb_minus4 + 4);
						sps.long_term_ref_pics[i].used_by_curr_pic_lt_sps_flag = br.GetBits(1);
					}
				}

				sps.sps_temporal_mvp_enabled_flag = br.GetBits(1);
				sps.strong_intra_smoothing_enabled_flag = br.GetBits(1);
				sps.vui_parameters_present_flag = br.GetBits(1);
				if (sps.vui_parameters_present_flag)
				{
					if (!sps.vui_parameters.Parse(br, sps.sps_max_sub_layers_minus1))
					{
						return false;
					}
				}

				sps.sps_extension_present_flag = br.GetBits(1);
				if (sps.sps_extension_present_flag)
				{
					sps.sps_range_extension_flag = br.GetBits(1);
					sps.sps_multilayer_extension_flag = br.GetBits(1);
					sps.sps_3d_extension_flag = br.GetBits(1);
					sps.sps_scc_extension_flag = br.GetBits(1);
					sps.sps_extension_4bits = br.GetBits(4);
				}

				if (sps.sps_range_extension_flag)
				{
					UE_LOGF(LogElectraDecoders, Error, "Range extensions are not supported");
					return false;
				}

				if (sps.sps_multilayer_extension_flag)
				{
					UE_LOGF(LogElectraDecoders, Error, "Multilayer extensions are not supported");
					return false;
				}

				if (sps.sps_3d_extension_flag)
				{
					UE_LOGF(LogElectraDecoders, Error, "3D extensions are not supported");
					return false;
				}

				if (sps.sps_scc_extension_flag)
				{
					UE_LOGF(LogElectraDecoders, Error, "Screen content coding extensions are not supported");
					return false;
				}

				if (sps.sps_extension_4bits)
				{
					while(br.more_rbsp_data())
					{
						/*uint8 sps_extension_data_flag =*/ br.GetBits(1);
					}
				}

				if (!br.rbsp_trailing_bits())
				{
					UE_LOGF(LogElectraDecoders, Error, "Bad rbsp_trailing_bits in SPS NALU!");
					return false;
				}

				// Set up common values
				sps.SubWidthC = (sps.chroma_format_idc==1 || sps.chroma_format_idc==2) && sps.separate_colour_plane_flag==0 ? 2 : 1;
				sps.SubHeightC = sps.chroma_format_idc==1 && sps.separate_colour_plane_flag==0 ? 2 : 1;
				sps.ChromaArrayType = sps.separate_colour_plane_flag ? 0 : sps.chroma_format_idc;
				sps.MinCbLog2SizeY = sps.log2_min_luma_coding_block_size_minus3 + 3;
				sps.CtbLog2SizeY = sps.MinCbLog2SizeY + sps.log2_diff_max_min_luma_coding_block_size;
				sps.MinCbSizeY = 1 << sps.MinCbLog2SizeY;
				sps.CtbSizeY = 1 << sps.CtbLog2SizeY;
				sps.PicWidthInMinCbsY = sps.pic_width_in_luma_samples / sps.MinCbSizeY;
				sps.PicWidthInCtbsY = (sps.pic_width_in_luma_samples + sps.CtbSizeY - 1) / sps.CtbSizeY;
				sps.PicHeightInMinCbsY = sps.pic_height_in_luma_samples / sps.MinCbSizeY;
				sps.PicHeightInCtbsY = (sps.pic_height_in_luma_samples + sps.CtbSizeY - 1) / sps.CtbSizeY;
				sps.PicSizeInMinCbsY = sps.PicWidthInMinCbsY * sps.PicHeightInMinCbsY;
				sps.PicSizeInCtbsY = sps.PicWidthInCtbsY * sps.PicHeightInCtbsY;
				sps.PicSizeInSamplesY = sps.pic_width_in_luma_samples * sps.pic_height_in_luma_samples;
				sps.PicWidthInSamplesC = sps.pic_width_in_luma_samples / sps.SubWidthC;
				sps.PicHeightInSamplesC = sps.pic_height_in_luma_samples / sps.SubHeightC;
				sps.CtbWidthC = sps.chroma_format_idc == 0 ? 0 : sps.CtbSizeY / sps.SubWidthC;
				sps.CtbHeightC = sps.chroma_format_idc == 0 ? 0 : sps.CtbSizeY / sps.SubHeightC;

				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool ParsePictureParameterSet(TMap<uint32, FPictureParameterSet>& InOutPictureParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)													\
					if (!(expr))																		\
					{																					\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in PPS", elem);	\
						return false;																	\
					}

				TUniquePtr<FRBSP> RBSP(MakeRBSP(InBitstream, InBitstreamLenInBytes));
				FBitstreamReaderH265 br(RBSP->Data, RBSP->Size);

				// Verify that this is in fact an PPS NALU.
				if ((br.GetBits(16) >> 9) != 34)
				{
					UE_LOGF(LogElectraDecoders, Error, "Not a PPS NALU!");
					return false;
				}
				if (br.GetRemainingByteLength() < 4)
				{
					UE_LOGF(LogElectraDecoders, Error, "Insufficient bytes in bitstream to be a PPS NALU!");
					return false;
				}

				uint32 temp_pps_pic_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(temp_pps_pic_parameter_set_id <= 63, TEXT("pps_pic_parameter_set_id"));

				FPictureParameterSet* ppsPtr = InOutPictureParameterSets.Find(temp_pps_pic_parameter_set_id);
				if (!ppsPtr)
				{
					ppsPtr = &InOutPictureParameterSets.Emplace(temp_pps_pic_parameter_set_id);
					check(ppsPtr);
				}
				else
				{
					ppsPtr->Reset();
				}
				FPictureParameterSet& pps = *ppsPtr;

				pps.pps_pic_parameter_set_id = temp_pps_pic_parameter_set_id;
				pps.pps_seq_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(pps.pps_seq_parameter_set_id <= 15, TEXT("pps_seq_parameter_set_id"));
				pps.dependent_slice_segments_enabled_flag = br.GetBits(1);
				pps.output_flag_present_flag = br.GetBits(1);
				pps.num_extra_slice_header_bits = br.GetBits(3);
				//RANGE_CHECK_FAILURE(pps.num_extra_slice_header_bits <= 2, TEXT("num_extra_slice_header_bits"));
				pps.sign_data_hiding_enabled_flag = br.GetBits(1);
				pps.cabac_init_present_flag = br.GetBits(1);
				pps.num_ref_idx_l0_default_active_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(pps.num_ref_idx_l0_default_active_minus1 <= 14, TEXT("num_ref_idx_l0_default_active_minus1"));
				pps.num_ref_idx_l1_default_active_minus1 = br.ue_v();
				RANGE_CHECK_FAILURE(pps.num_ref_idx_l1_default_active_minus1 <= 14, TEXT("num_ref_idx_l1_default_active_minus1"));
				pps.init_qp_minus26 = br.se_v();
				RANGE_CHECK_FAILURE(pps.init_qp_minus26 >= -26 && pps.init_qp_minus26 <= 25, TEXT("init_qp_minus26"));
				pps.constrained_intra_pred_flag = br.GetBits(1);
				pps.transform_skip_enabled_flag = br.GetBits(1);
				pps.cu_qp_delta_enabled_flag = br.GetBits(1);
				if (pps.cu_qp_delta_enabled_flag)
				{
					pps.diff_cu_qp_delta_depth = br.ue_v();
				}
				pps.pps_cb_qp_offset = br.se_v();
				RANGE_CHECK_FAILURE(pps.pps_cb_qp_offset >= -12 && pps.pps_cb_qp_offset <= 12, TEXT("pps_cb_qp_offset"));
				pps.pps_cr_qp_offset = br.se_v();
				RANGE_CHECK_FAILURE(pps.pps_cr_qp_offset >= -12 && pps.pps_cr_qp_offset <= 12, TEXT("pps_cr_qp_offset"));
				pps.pps_slice_chroma_qp_offsets_present_flag = br.GetBits(1);
				pps.weighted_pred_flag = br.GetBits(1);
				pps.weighted_bipred_flag = br.GetBits(1);
				pps.transquant_bypass_enabled_flag = br.GetBits(1);
				pps.tiles_enabled_flag = br.GetBits(1);
				pps.entropy_coding_sync_enabled_flag = br.GetBits(1);
				if (pps.tiles_enabled_flag)
				{
					pps.num_tile_columns_minus1 = br.ue_v();
					pps.num_tile_rows_minus1 = br.ue_v();
					RANGE_CHECK_FAILURE(pps.num_tile_columns_minus1 + pps.num_tile_rows_minus1 > 0, TEXT("num_tile_columns_minus1 + num_tile_rows_minus1"));
					pps.uniform_spacing_flag = br.GetBits(1);
					if (!pps.uniform_spacing_flag)
					{
						pps.column_width_minus1.SetNum(pps.num_tile_columns_minus1);
						for(uint32 i=0; i<pps.num_tile_columns_minus1; ++i)
						{
							pps.column_width_minus1[i] = br.ue_v();
						}
						pps.row_height_minus1.SetNum(pps.num_tile_rows_minus1);
						for(uint32 i=0; i<pps.num_tile_rows_minus1; ++i)
						{
							pps.row_height_minus1[i] = br.ue_v();
						}
					}
					pps.loop_filter_across_tiles_enabled_flag = br.GetBits(1);
				}
				pps.pps_loop_filter_across_slices_enabled_flag = br.GetBits(1);
				pps.deblocking_filter_control_present_flag = br.GetBits(1);
				if (pps.deblocking_filter_control_present_flag)
				{
					pps.deblocking_filter_override_enabled_flag = br.GetBits(1);
					pps.pps_deblocking_filter_disabled_flag = br.GetBits(1);
					if (!pps.pps_deblocking_filter_disabled_flag)
					{
						pps.pps_beta_offset_div2 = br.se_v();
						RANGE_CHECK_FAILURE(pps.pps_beta_offset_div2 >= -6 && pps.pps_beta_offset_div2 <= 6, TEXT("pps_beta_offset_div2"));
						pps.pps_tc_offset_div2 = br.se_v();
						RANGE_CHECK_FAILURE(pps.pps_tc_offset_div2 >= -6 && pps.pps_tc_offset_div2 <= 6, TEXT("pps_tc_offset_div2"));
					}
				}
				pps.pps_scaling_list_data_present_flag = br.GetBits(1);
				if (pps.pps_scaling_list_data_present_flag)
				{
					if (!pps.scaling_list_data.Parse(br))
					{
						return false;
					}
				}
				pps.lists_modification_present_flag = br.GetBits(1);
				pps.log2_parallel_merge_level_minus2 = br.ue_v();

				pps.slice_segment_header_extension_present_flag = br.GetBits(1);
				pps.pps_extension_present_flag = br.GetBits(1);
				if (pps.pps_extension_present_flag)
				{
					pps.pps_range_extension_flag = br.GetBits(1);
					pps.pps_multilayer_extension_flag = br.GetBits(1);
					pps.pps_3d_extension_flag = br.GetBits(1);
					pps.pps_scc_extension_flag = br.GetBits(1);
					pps.pps_extension_4bits = br.GetBits(4);
				}

				if (pps.pps_range_extension_flag)
				{
					if (!pps.pps_range_extension.Parse(br))
					{
						return false;
					}
				}
				if (pps.pps_multilayer_extension_flag)
				{
					if (!pps.pps_multilayer_extension.Parse(br))
					{
						return false;
					}
				}
				if (pps.pps_3d_extension_flag)
				{
					if (!pps.pps_3d_extension.Parse(br))
					{
						return false;
					}
				}
				if (pps.pps_scc_extension_flag)
				{
					if (!pps.pps_scc_extension.Parse(br))
					{
						return false;
					}
				}
				if (pps.pps_extension_4bits)
				{
					while(br.more_rbsp_data())
					{
						uint8 pps_extension_data_flag = br.GetBits(1);
					}
				}
				if (!br.rbsp_trailing_bits())
				{
					UE_LOGF(LogElectraDecoders, Error, "Bad rbsp_trailing_bits in PPS NALU!");
					return false;
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}

			bool FPictureParameterSet::FPPSRangeExtension::Parse(FBitstreamReaderH265& br)
			{
				UE_LOGF(LogElectraDecoders, Error, "Range extension is not supported");
				return false;
			}

			bool FPictureParameterSet::FPPSMultilayerExtension::Parse(FBitstreamReaderH265& br)
			{
				UE_LOGF(LogElectraDecoders, Error, "Multilayer extension is not supported");
				return false;
				/*
				#define RANGE_CHECK_FAILURE(expr, elem)																		\
					if (!(expr))																							\
					{																										\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in pps_multilayer_extension(", elem);	\
						return false;																						\
					}

				poc_reset_info_present_flag = br.GetBits(1);
				pps_infer_scaling_list_flag = br.GetBits(1);
				if (pps_infer_scaling_list_flag)
				{
					pps_scaling_list_ref_layer_id = br.GetBits(6);
					RANGE_CHECK_FAILURE(pps_scaling_list_ref_layer_id <= 62, TEXT("pps_scaling_list_ref_layer_id"));
				}
				num_ref_loc_offsets = br.ue_v();
				rlos.SetNum(num_ref_loc_offsets);
				for(uint32 i=0; i<num_ref_loc_offsets; ++i)
				{
					rlos[i].ref_loc_offset_layer_id = br.GetBits(6);
					rlos[i].scaled_ref_layer_offset_present_flag = br.GetBits(1);
					if (rlos[i].scaled_ref_layer_offset_present_flag)
					{
						rlos[i].scaled_ref_layer_left_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].scaled_ref_layer_left_offset >= -16384 && rlos[i].scaled_ref_layer_left_offset <= 16383, TEXT("scaled_ref_layer_left_offset"));
						rlos[i].scaled_ref_layer_top_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].scaled_ref_layer_top_offset >= -16384 && rlos[i].scaled_ref_layer_top_offset <= 16383, TEXT("scaled_ref_layer_top_offset"));
						rlos[i].scaled_ref_layer_right_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].scaled_ref_layer_right_offset >= -16384 && rlos[i].scaled_ref_layer_right_offset <= 16383, TEXT("scaled_ref_layer_right_offset"));
						rlos[i].scaled_ref_layer_bottom_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].scaled_ref_layer_bottom_offset >= -16384 && rlos[i].scaled_ref_layer_bottom_offset <= 16383, TEXT("scaled_ref_layer_bottom_offset"));
					}
					rlos[i].ref_region_offset_present_flag = br.GetBits(1);
					if (rlos[i].ref_region_offset_present_flag)
					{
						rlos[i].ref_region_left_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].ref_region_left_offset >= -16384 && rlos[i].ref_region_left_offset <= 16383, TEXT("ref_region_left_offset"));
						rlos[i].ref_region_top_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].ref_region_top_offset >= -16384 && rlos[i].ref_region_top_offset <= 16383, TEXT("ref_region_top_offset"));
						rlos[i].ref_region_right_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].ref_region_right_offset >= -16384 && rlos[i].ref_region_right_offset <= 16383, TEXT("ref_region_right_offset"));
						rlos[i].ref_region_bottom_offset = br.se_v();
						RANGE_CHECK_FAILURE(rlos[i].ref_region_bottom_offset >= -16384 && rlos[i].ref_region_bottom_offset <= 16383, TEXT("ref_region_bottom_offset"));
					}
					rlos[i].resample_phase_set_present_flag = br.GetBits(1);
					if (rlos[i].resample_phase_set_present_flag)
					{
						rlos[i].phase_hor_luma = br.ue_v();
						RANGE_CHECK_FAILURE(rlos[i].phase_hor_luma <= 31, TEXT("phase_hor_luma"));
						rlos[i].phase_ver_luma = br.ue_v();
						RANGE_CHECK_FAILURE(rlos[i].phase_ver_luma <= 31, TEXT("phase_ver_luma"));
						rlos[i].phase_hor_chroma_plus8 = br.ue_v();
						RANGE_CHECK_FAILURE(rlos[i].phase_hor_chroma_plus8 <= 63, TEXT("phase_hor_chroma_plus8"));
						rlos[i].phase_ver_chroma_plus8 = br.ue_v();
						RANGE_CHECK_FAILURE(rlos[i].phase_ver_chroma_plus8 <= 63, TEXT("phase_ver_chroma_plus8"));
					}
				}
				colour_mapping_enabled_flag = br.GetBits(1);
				if (colour_mapping_enabled_flag)
				{
					// colour_mapping_table
				}
				return true;
				#undef RANGE_CHECK_FAILURE
				*/
			}

			bool FPictureParameterSet::FPPS3DExtension::Parse(FBitstreamReaderH265& br)
			{
				UE_LOGF(LogElectraDecoders, Error, "3D extension is not supported");
				return false;
			}

			bool FPictureParameterSet::FPPSSCCExtension::Parse(FBitstreamReaderH265& br)
			{
				UE_LOGF(LogElectraDecoders, Error, "SCC extension is not supported");
				return false;
			}

			bool ParseSliceHeader(TUniquePtr<FRBSP>& OutRBSP, FBitstreamReaderH265& OutRBSPReader, FSliceSegmentHeader& OutSlice, const TMap<uint32, FVideoParameterSet>& InVideoParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const TMap<uint32, FPictureParameterSet>& InPictureParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes)
			{
				#define RANGE_CHECK_FAILURE(expr, elem)																	\
					if (!(expr))																						\
					{																									\
						UE_LOGF(LogElectraDecoders, Error, "Invalid value for %ls in slice_segment_header", elem);	\
						return false;																					\
					}

				OutRBSP = MakeRBSP(InBitstream, InBitstreamLenInBytes);
				OutRBSPReader.SetData(OutRBSP->Data, OutRBSP->Size);
				FBitstreamReaderH265& br(OutRBSPReader);

				if (br.GetBits(1))	// forbidden_zero_bit
				{
					RANGE_CHECK_FAILURE(false, TEXT("forbidden_zero_bit"));
				}
				uint32 nal_unit_type = br.GetBits(6);
				uint32 nuh_layer_id = br.GetBits(6);
				uint32 nuh_temporal_id_plus1 = br.GetBits(3);

				// Verify that this is in fact a slice.
				if (!((nal_unit_type >= 0 && nal_unit_type <= 9) || (nal_unit_type >= 16 && nal_unit_type <= 21)))
				{
					UE_LOGF(LogElectraDecoders, Error, "Not a supported slice type (%d)!", nal_unit_type);
					return false;
				}
				// Sub-layer-non-reference (SLNR)?
				// "If a picture has nal_unit_type equal to TRAIL_N, TSA_N, STSA_N, RADL_N, RASL_N, RSV_VCL_N10, RSV_VCL_N12 or RSV_VCL_N14, the picture is an SLNR picture."
				bool bIsSLNR = nal_unit_type == 0/*TRAIL_N*/ || nal_unit_type == 2/*TSA_N*/ || nal_unit_type == 4/*STSA_N*/ || nal_unit_type == 6/*RADL_N*/ || nal_unit_type == 8/*RASL_N*/ || nal_unit_type == 12/*RSV_VCL_N12*/ || nal_unit_type == 14/*RSV_VCL_N14*/;
				bool bIsIRAP = nal_unit_type >= 16 && nal_unit_type <= 23;
				bool bIsBLA = nal_unit_type >= 16 && nal_unit_type <= 18;
				bool bIsIDR = nal_unit_type == 19 || nal_unit_type == 20;
				bool bIsCRA = nal_unit_type == 21;
				bool bIsReferenceNalu = (nal_unit_type <= 15 && (nal_unit_type & 1) != 0) || (nal_unit_type >= 16 && nal_unit_type <= 23);

				OutSlice.first_slice_segment_in_pic_flag = br.GetBits(1);
				if (bIsIRAP)
				{
					OutSlice.no_output_of_prior_pics_flag = br.GetBits(1);
				}
				OutSlice.slice_pic_parameter_set_id = br.ue_v();
				RANGE_CHECK_FAILURE(OutSlice.slice_pic_parameter_set_id <= 63, TEXT("slice_pic_parameter_set_id"));

				const FPictureParameterSet* ppsPtr = InPictureParameterSets.Find(OutSlice.slice_pic_parameter_set_id);
				if (!ppsPtr)
				{
					UE_LOGF(LogElectraDecoders, Error, "Reference picture parameter set not found!");
					return false;
				}
				const FPictureParameterSet& pps = *ppsPtr;

				const FSequenceParameterSet* spsPtr = InSequenceParameterSets.Find(pps.pps_seq_parameter_set_id);
				if (!spsPtr)
				{
					UE_LOGF(LogElectraDecoders, Error, "Reference sequence parameter set not found!");
					return false;
				}
				const FSequenceParameterSet& sps = *spsPtr;

				// Setup convenience values
				OutSlice.NumBitsForPOCValues = sps.log2_max_pic_order_cnt_lsb_minus4 + 4;
				OutSlice.NalUnitType = nal_unit_type;
				OutSlice.NuhLayerId = nuh_layer_id;
				OutSlice.NuhTemporalIdPlus1 = nuh_temporal_id_plus1;
				OutSlice.bIsIDR = bIsIDR;
				OutSlice.bIsCRA = bIsCRA;
				OutSlice.bIsIRAP = bIsIRAP;
				OutSlice.bIsBLA = bIsBLA;
				OutSlice.bIsReferenceNalu = bIsReferenceNalu;
				OutSlice.bIsSLNR = bIsSLNR;
				OutSlice.CurrRpsIdx = sps.num_short_term_ref_pic_sets;	// start with SPS default
				OutSlice.HighestTid = sps.sps_max_sub_layers_minus1;	// see 8.1
				OutSlice.sps_max_num_reorder_pics_HighestTid = sps.sps_max_num_reorder_pics[OutSlice.HighestTid];
				OutSlice.sps_max_latency_increase_plus1_HighestTid = sps.sps_max_latency_increase_plus1[OutSlice.HighestTid];
				OutSlice.sps_max_dec_pic_buffering_minus1_HighestTid = sps.sps_max_dec_pic_buffering_minus1[OutSlice.HighestTid];
				OutSlice.SPSLongTermRefPicsPresentFlag = !!sps.long_term_ref_pics_present_flag;
				OutSlice.SPSLongTermRefPics = sps.long_term_ref_pics;
				// Inherit defaults from PPS
				OutSlice.slice_deblocking_filter_disabled_flag = pps.pps_deblocking_filter_disabled_flag;
				OutSlice.slice_beta_offset_div2 = pps.pps_beta_offset_div2;
				OutSlice.slice_tc_offset_div2 = pps.pps_tc_offset_div2;
				OutSlice.slice_loop_filter_across_slices_enabled_flag = pps.pps_loop_filter_across_slices_enabled_flag;

				if (!OutSlice.first_slice_segment_in_pic_flag)
				{
					if (pps.dependent_slice_segments_enabled_flag)
					{
						OutSlice.dependent_slice_segment_flag = br.GetBits(1);
					}
					const uint32 NumBits = FMath::CeilLogTwo(sps.PicSizeInCtbsY);
					OutSlice.slice_segment_address = br.GetBits(NumBits);
				}
				if (!OutSlice.dependent_slice_segment_flag)
				{
					for(uint32 i=0; i<pps.num_extra_slice_header_bits; ++i)
					{
						br.SkipBits(1);
					}
					OutSlice.slice_type = br.ue_v();
					RANGE_CHECK_FAILURE(OutSlice.slice_type <= 2, TEXT("slice_type"));
					if (pps.output_flag_present_flag)
					{
						OutSlice.pic_output_flag = br.GetBits(1);
					}
					if (sps.separate_colour_plane_flag)
					{
						OutSlice.colour_plane_id = br.GetBits(2);
					}
					if (!bIsIDR)
					{
						OutSlice.slice_pic_order_cnt_lsb = br.GetBits(OutSlice.NumBitsForPOCValues);
						OutSlice.short_term_ref_pic_set_sps_flag = br.GetBits(1);
						uint64 NumBitsRemainingBefore = br.GetRemainingBits();
						if (!OutSlice.short_term_ref_pic_set_sps_flag)
						{
							if (!OutSlice.st_ref_pic_set.Parse(br, sps.num_short_term_ref_pic_sets, sps.st_ref_pic_set, sps.num_short_term_ref_pic_sets))
							{
								return false;
							}
						}
						else if (sps.num_short_term_ref_pic_sets)
						{
							const uint32 NumBits = FMath::CeilLogTwo(sps.num_short_term_ref_pic_sets);
							OutSlice.short_term_ref_pic_set_idx = br.GetBits(NumBits);
							RANGE_CHECK_FAILURE((int32)OutSlice.short_term_ref_pic_set_idx < sps.num_short_term_ref_pic_sets, TEXT("short_term_ref_pic_set_idx"));
							OutSlice.st_ref_pic_set = sps.st_ref_pic_set[OutSlice.short_term_ref_pic_set_idx];
						}
						else
						{
							RANGE_CHECK_FAILURE(false, TEXT("sps.num_short_term_ref_pic_sets is 0"));
						}
						uint64 NumBitsRemainingAfter = br.GetRemainingBits();
						OutSlice.NumBitsForShortTermRefs = NumBitsRemainingBefore - NumBitsRemainingAfter;

						// Set `CurrRpsIdx` convenience value.
						if (OutSlice.short_term_ref_pic_set_sps_flag)
						{
							OutSlice.CurrRpsIdx = OutSlice.short_term_ref_pic_set_idx;
						}

						NumBitsRemainingBefore = NumBitsRemainingAfter;
						if (sps.long_term_ref_pics_present_flag)
						{
							const uint32 NumBits = FMath::CeilLogTwo(sps.num_long_term_ref_pics_sps);
							if (sps.num_long_term_ref_pics_sps > 0)
							{
								OutSlice.num_long_term_sps = br.ue_v();
								RANGE_CHECK_FAILURE(OutSlice.num_long_term_sps <= sps.num_long_term_ref_pics_sps, TEXT("num_long_term_sps"));
							}
							OutSlice.num_long_term_pics = br.ue_v();
							OutSlice.long_term_ref.SetNum(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);
							for(uint32 i=0; i<OutSlice.num_long_term_sps + OutSlice.num_long_term_pics; ++i)
							{
								if (i < OutSlice.num_long_term_sps)
								{
									if (sps.num_long_term_ref_pics_sps > 1)
									{
										OutSlice.long_term_ref[i].lt_idx_sps = br.GetBits(NumBits);
										RANGE_CHECK_FAILURE(OutSlice.long_term_ref[i].lt_idx_sps <= sps.num_long_term_ref_pics_sps, TEXT("lt_idx_sps"));
									}
								}
								else
								{
									OutSlice.long_term_ref[i].poc_lsb_lt = br.GetBits(OutSlice.NumBitsForPOCValues);
									OutSlice.long_term_ref[i].used_by_curr_pic_lt_flag = br.GetBits(1);
								}
								OutSlice.long_term_ref[i].delta_poc_msb_present_flag = br.GetBits(1);
								if (OutSlice.long_term_ref[i].delta_poc_msb_present_flag)
								{
									OutSlice.long_term_ref[i].delta_poc_msb_cycle_lt = br.ue_v();
								}
							}
						}
						NumBitsRemainingAfter = br.GetRemainingBits();
						OutSlice.NumBitsForLongTermRefs = NumBitsRemainingBefore - NumBitsRemainingAfter;

						if (sps.sps_temporal_mvp_enabled_flag)
						{
							OutSlice.slice_temporal_mvp_enabled_flag = br.GetBits(1);
						}
					}

					if (sps.sample_adaptive_offset_enabled_flag)
					{
						OutSlice.slice_sao_luma_flag = br.GetBits(1);
						if (sps.ChromaArrayType != 0)
						{
							OutSlice.slice_sao_chroma_flag = br.GetBits(1);
						}
					}
					if (OutSlice.slice_type == 0 /*B*/ || OutSlice.slice_type == 1 /*P*/)
					{
						OutSlice.num_ref_idx_l0_active_minus1 = pps.num_ref_idx_l0_default_active_minus1;
						OutSlice.num_ref_idx_l1_active_minus1 = pps.num_ref_idx_l1_default_active_minus1;
						OutSlice.num_ref_idx_active_override_flag = br.GetBits(1);
						if (OutSlice.num_ref_idx_active_override_flag)
						{
							OutSlice.num_ref_idx_l0_active_minus1 = br.ue_v();
							RANGE_CHECK_FAILURE(OutSlice.num_ref_idx_l0_active_minus1 <= 14, TEXT("num_ref_idx_l0_active_minus1"));
							if (OutSlice.slice_type == 0 /*B*/)
							{
								OutSlice.num_ref_idx_l1_active_minus1 = br.ue_v();
								RANGE_CHECK_FAILURE(OutSlice.num_ref_idx_l1_active_minus1 <= 14, TEXT("num_ref_idx_l1_active_minus1"));
							}
						}
						if (pps.lists_modification_present_flag)
						{
							int32 NumPicTotalCurr = 0;
							for(uint32 i=0,iMax=OutSlice.st_ref_pic_set.NumDeltaPocs(); i<iMax; ++i)
							{
								NumPicTotalCurr += OutSlice.st_ref_pic_set.used_by_curr_pic_flag[i];
							}
							for(int32 i=0; i<OutSlice.long_term_ref.Num(); ++i)
							{
								NumPicTotalCurr += OutSlice.long_term_ref[i].used_by_curr_pic_lt_flag;
							}
							if (pps.pps_scc_extension_flag && pps.pps_scc_extension.pps_curr_pic_ref_enabled_flag)
							{
								++NumPicTotalCurr;
							}
							if (NumPicTotalCurr > 1)
							{
								const uint32 NumBits = FMath::CeilLogTwo(NumPicTotalCurr);
								uint8 ref_pic_list_modification_flag_l0 = br.GetBits(1);
								if (ref_pic_list_modification_flag_l0)
								{
									OutSlice.ref_pic_lists_modification.list_entry_l0.SetNumUninitialized(OutSlice.num_ref_idx_l0_active_minus1 + 1);
									for(uint32 i=0; i<=OutSlice.num_ref_idx_l0_active_minus1; ++i)
									{
										OutSlice.ref_pic_lists_modification.list_entry_l0[i] = br.GetBits(NumBits);
									}
								}
								if (OutSlice.slice_type == 0 /*B*/)
								{
									uint8 ref_pic_list_modification_flag_l1 = br.GetBits(1);
									if (ref_pic_list_modification_flag_l1)
									{
										OutSlice.ref_pic_lists_modification.list_entry_l1.SetNumUninitialized(OutSlice.num_ref_idx_l1_active_minus1 + 1);
										for(uint32 i=0; i<=OutSlice.num_ref_idx_l1_active_minus1; ++i)
										{
											OutSlice.ref_pic_lists_modification.list_entry_l1[i] = br.GetBits(NumBits);
										}
									}
								}
							}
						}

						if (OutSlice.slice_type == 0 /*B*/)
						{
							OutSlice.mvd_l1_zero_flag = br.GetBits(1);
						}
						if (pps.cabac_init_present_flag)
						{
							OutSlice.cabac_init_flag = br.GetBits(1);
						}
						if (OutSlice.slice_temporal_mvp_enabled_flag)
						{
							if (OutSlice.slice_type == 0 /*B*/)
							{
								OutSlice.collocated_from_l0_flag = br.GetBits(1);
							}
							if ((OutSlice.collocated_from_l0_flag && OutSlice.num_ref_idx_l0_active_minus1 > 0) ||
								(!OutSlice.collocated_from_l0_flag && OutSlice.num_ref_idx_l1_active_minus1 > 0))
							{
								OutSlice.collocated_ref_idx = br.ue_v();
								if (OutSlice.slice_type == 1 || (OutSlice.slice_type == 0 && OutSlice.collocated_from_l0_flag))
								{
									RANGE_CHECK_FAILURE(OutSlice.collocated_ref_idx <= OutSlice.num_ref_idx_l0_active_minus1, TEXT("collocated_ref_idx"));
								}
								else if (OutSlice.slice_type == 0 && !OutSlice.collocated_from_l0_flag)
								{
									RANGE_CHECK_FAILURE(OutSlice.collocated_ref_idx <= OutSlice.num_ref_idx_l1_active_minus1, TEXT("collocated_ref_idx"));
								}
							}
						}

						if ((pps.weighted_pred_flag && OutSlice.slice_type == 1) ||
							(pps.weighted_bipred_flag && OutSlice.slice_type == 0))
						{
							auto HasExplicitValue = [](uint32 InI) -> bool
							{
								// FIXME: P-slice: if( ( pic_layer_id( RefPicList0[ i ] ) != nuh_layer_id ) || ( PicOrderCnt(RefPicList0[ i ]) != PicOrderCnt( CurrPic ) ) )
								//		  B-Slice: if( ( pic_layer_id( RefPicList0[ i ] ) != nuh_layer_id ) || ( PicOrderCnt(RefPicList1[ i ]) != PicOrderCnt( CurrPic ) ) )
								return true;
							};
							OutSlice.pred_weight_table.luma_log2_weight_denom = br.ue_v();
							RANGE_CHECK_FAILURE(OutSlice.pred_weight_table.luma_log2_weight_denom <= 7, TEXT("luma_log2_weight_denom"));
							OutSlice.pred_weight_table.delta_chroma_log2_weight_denom = sps.ChromaArrayType != 0 ? br.se_v() : 0;
							const int32 ChromaLog2WeightDenom = (int32)OutSlice.pred_weight_table.luma_log2_weight_denom + OutSlice.pred_weight_table.delta_chroma_log2_weight_denom;
							RANGE_CHECK_FAILURE(ChromaLog2WeightDenom >= 0 && ChromaLog2WeightDenom <= 7, TEXT("luma_log2_weight_denom + delta_chroma_log2_weight_denom"));
							const int32 default_delta_luma_weight_lX = 1 << OutSlice.pred_weight_table.luma_log2_weight_denom;
							const int32 default_delta_chroma_weight_lX = 1 << ChromaLog2WeightDenom;
							for(uint32 nList=0, nListMax=OutSlice.slice_type==0/*B*/?2:1; nList<nListMax; ++nList)
							{
								uint32 num_ref_idx_lX_active_minus1 = nList == 0 ? OutSlice.num_ref_idx_l0_active_minus1 : OutSlice.num_ref_idx_l1_active_minus1;

								// Set calculated default values.
								for(uint32 i=0; i<=num_ref_idx_lX_active_minus1; ++i)
								{
									OutSlice.pred_weight_table.delta_luma_weight_lX[nList][i] = default_delta_luma_weight_lX;
									for(uint32 j=0; j<2; ++j)
									{
										OutSlice.pred_weight_table.delta_chroma_weight_lX[nList][i][j] = default_delta_chroma_weight_lX;
									}
								}

								for(uint32 i=0; i<=num_ref_idx_lX_active_minus1; ++i)
								{
									if (HasExplicitValue(i))
									{
										OutSlice.pred_weight_table.luma_weight_lX_flag[nList][i] = br.GetBits(1);
									}
								}
								if (sps.ChromaArrayType != 0)
								{
									for(uint32 i=0; i<=num_ref_idx_lX_active_minus1; ++i)
									{
										if (HasExplicitValue(i))
										{
											OutSlice.pred_weight_table.chroma_weight_lX_flag[nList][i] = br.GetBits(1);
										}
									}
								}
								for(uint32 i=0; i<=num_ref_idx_lX_active_minus1; ++i)
								{
									if (OutSlice.pred_weight_table.luma_weight_lX_flag[nList][i])
									{
										OutSlice.pred_weight_table.delta_luma_weight_lX[nList][i] = br.se_v();
										OutSlice.pred_weight_table.luma_offset_lX[nList][i] = br.se_v();
									}
									if (OutSlice.pred_weight_table.chroma_weight_lX_flag[nList][i])
									{
										for(uint32 j=0; j<2; ++j)
										{
											OutSlice.pred_weight_table.delta_chroma_weight_lX[nList][i][j] = br.se_v();
											OutSlice.pred_weight_table.delta_chroma_offset_lX[nList][i][j] = br.se_v();
										}
									}
								}
							}
						}

						OutSlice.five_minus_max_num_merge_cand = br.ue_v();
						RANGE_CHECK_FAILURE(OutSlice.five_minus_max_num_merge_cand <= 4, TEXT("five_minus_max_num_merge_cand"));

						if (sps.sps_scc_extension.motion_vector_resolution_control_idc == 2)
						{
							OutSlice.use_integer_mv_flag = br.GetBits(1);
						}
						else
						{
							OutSlice.use_integer_mv_flag = sps.sps_scc_extension.motion_vector_resolution_control_idc ? 1 : 0;
						}
					}

					OutSlice.slice_qp_delta = br.se_v();
					if (pps.pps_slice_chroma_qp_offsets_present_flag)
					{
						OutSlice.slice_cb_qp_offset = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_cb_qp_offset >= -12 && OutSlice.slice_cb_qp_offset <= 12, TEXT("slice_cb_qp_offset"));
						OutSlice.slice_cr_qp_offset = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_cr_qp_offset >= -12 && OutSlice.slice_cr_qp_offset <= 12, TEXT("slice_cr_qp_offset"));
						RANGE_CHECK_FAILURE(pps.pps_cb_qp_offset + OutSlice.slice_cb_qp_offset >= -12 && pps.pps_cb_qp_offset + OutSlice.slice_cb_qp_offset <= 12, TEXT("pps_cb_qp_offset + slice_cb_qp_offset"));
						RANGE_CHECK_FAILURE(pps.pps_cr_qp_offset + OutSlice.slice_cr_qp_offset >= -12 && pps.pps_cr_qp_offset + OutSlice.slice_cr_qp_offset <= 12, TEXT("pps_cr_qp_offset + slice_cr_qp_offset"));
					}
					if (pps.pps_scc_extension.pps_slice_act_qp_offsets_present_flag)
					{
						OutSlice.slice_act_y_qp_offset = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_act_y_qp_offset >= -12 && OutSlice.slice_act_y_qp_offset <= 12, TEXT("slice_act_y_qp_offset"));
						OutSlice.slice_act_cb_qp_offset = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_act_cb_qp_offset >= -12 && OutSlice.slice_act_cb_qp_offset <= 12, TEXT("slice_act_cb_qp_offset"));
						OutSlice.slice_act_cr_qp_offset = br.se_v();
						RANGE_CHECK_FAILURE(OutSlice.slice_act_cr_qp_offset >= -12 && OutSlice.slice_act_cr_qp_offset <= 12, TEXT("slice_act_cr_qp_offset"));
					}
					if (pps.pps_range_extension.chroma_qp_offset_list_enabled_flag)
					{
						OutSlice.cu_chroma_qp_offset_enabled_flag = br.GetBits(1);
					}
					if (pps.deblocking_filter_override_enabled_flag)
					{
						OutSlice.deblocking_filter_override_flag = br.GetBits(1);
					}
					if (OutSlice.deblocking_filter_override_flag)
					{
						OutSlice.slice_deblocking_filter_disabled_flag = br.GetBits(1);
						if (!OutSlice.slice_deblocking_filter_disabled_flag)
						{
							OutSlice.slice_beta_offset_div2 = br.se_v();
							RANGE_CHECK_FAILURE(OutSlice.slice_beta_offset_div2 >= -6 && OutSlice.slice_beta_offset_div2 <= 6, TEXT("slice_beta_offset_div2"));
							OutSlice.slice_tc_offset_div2 = br.se_v();
							RANGE_CHECK_FAILURE(OutSlice.slice_tc_offset_div2 >= -6 && OutSlice.slice_tc_offset_div2 <= 6, TEXT("slice_tc_offset_div2"));
						}
					}
					if (pps.pps_loop_filter_across_slices_enabled_flag && (OutSlice.slice_sao_luma_flag || OutSlice.slice_sao_chroma_flag || !OutSlice.slice_deblocking_filter_disabled_flag))
					{
						OutSlice.slice_loop_filter_across_slices_enabled_flag = br.GetBits(1);
					}
				}

				if (pps.tiles_enabled_flag || pps.entropy_coding_sync_enabled_flag)
				{
					OutSlice.num_entry_point_offsets = br.ue_v();
					if (OutSlice.num_entry_point_offsets > 0)
					{
						OutSlice.offset_len_minus1 = br.ue_v();
						RANGE_CHECK_FAILURE(OutSlice.offset_len_minus1 <= 31, TEXT("offset_len_minus1"));
						OutSlice.entry_point_offset_minus1.SetNumUninitialized(OutSlice.num_entry_point_offsets);
						for(uint32 i=0; i<OutSlice.num_entry_point_offsets; ++i)
						{
							OutSlice.entry_point_offset_minus1[i] = br.GetBits(OutSlice.offset_len_minus1 + 1);
						}
					}
				}

				if (pps.slice_segment_header_extension_present_flag)
				{
					uint32 slice_segment_header_extension_length = br.ue_v();
					RANGE_CHECK_FAILURE(slice_segment_header_extension_length <= 256, TEXT("slice_segment_header_extension_length"));
					for(uint32 i=0; i<slice_segment_header_extension_length; ++i)
					{
						br.SkipBits(8);
					}
				}

				if (!br.byte_alignment())
				{
					UE_LOGF(LogElectraDecoders, Error, "Bad byte_alignment in slice_segment_header!");
					return false;
				}
				return true;
				#undef RANGE_CHECK_FAILURE
			}

			void FSliceSegmentHeaderPOCVars::Update(const FSliceSegmentHeader& InFromSliceHeader)
			{
				if (InFromSliceHeader.first_slice_segment_in_pic_flag && (InFromSliceHeader.bIsIDR || InFromSliceHeader.bIsBLA))
				{
					if (InFromSliceHeader.bIsIDR)
					{
						Reset();
						return;
					}
				}

				// Copy the short-term reference picture set from the slice
				rps = InFromSliceHeader.st_ref_pic_set;

				// 8.3.1 - Decoding process for picture order count
				const int32 pocLsb = (int32) InFromSliceHeader.slice_pic_order_cnt_lsb;
				const int32 prevPoc = prevTid0POC;
				MaxPocLsb = 1 << InFromSliceHeader.NumBitsForPOCValues;
				int32 prevPocLsb = prevPoc & (MaxPocLsb - 1);
				int32 prevPocMsb = prevPoc - prevPocLsb;
				int32 pocMsb = 0;
				if (pocLsb < prevPocLsb && prevPocLsb-pocLsb >= MaxPocLsb/2)
				{
					pocMsb = prevPocMsb + MaxPocLsb;
				}
				else if (pocLsb > prevPocLsb && pocLsb-prevPocLsb > MaxPocLsb/2)
				{
					pocMsb = prevPocMsb - MaxPocLsb;
				}
				else
				{
					pocMsb = prevPocMsb;
				}
				// BLA pictures set msb to 0 and also clear the reference picture set.
				if (InFromSliceHeader.NalUnitType == 16 || InFromSliceHeader.NalUnitType == 17 || InFromSliceHeader.NalUnitType == 18)
				{
					pocMsb = 0;
					SlicePOC = pocMsb + pocLsb;
					rps = FRefPicSet();
				}
				else
				{
					SlicePOC = pocMsb + pocLsb;
					// Update long term reference POCs (not needed for BLA pictures)
					if (!InFromSliceHeader.bIsIDR && InFromSliceHeader.SPSLongTermRefPicsPresentFlag)
					{
						rps.num_long_term_pics = InFromSliceHeader.num_long_term_sps + InFromSliceHeader.num_long_term_pics;
						int32 prev_delta_msb = 0;
						for(uint32 i=0; i<rps.num_long_term_pics; ++i)
						{
							if (i < InFromSliceHeader.num_long_term_sps)
							{
								rps.poc_lt[i] = InFromSliceHeader.SPSLongTermRefPics[ InFromSliceHeader.long_term_ref[i].lt_idx_sps ].lt_ref_pic_poc_lsb_sps;
								rps.used_by_curr_pic_lt_flag[i] = InFromSliceHeader.SPSLongTermRefPics[ InFromSliceHeader.long_term_ref[i].lt_idx_sps ].used_by_curr_pic_lt_sps_flag;
							}
							else
							{
								rps.poc_lt[i] = InFromSliceHeader.long_term_ref[i].poc_lsb_lt;
								rps.used_by_curr_pic_lt_flag[i] = InFromSliceHeader.long_term_ref[i].used_by_curr_pic_lt_flag;
							}
							if ((rps.delta_poc_msb_present_lt_flag[i] = InFromSliceHeader.long_term_ref[i].delta_poc_msb_present_flag) != 0)
							{
								// (7-52)
								int32 delta_poc_msb_cycle_lt = InFromSliceHeader.long_term_ref[i].delta_poc_msb_cycle_lt;
								if (i != 0 && i != InFromSliceHeader.num_long_term_sps)
								{
									delta_poc_msb_cycle_lt += prev_delta_msb;
								}
								rps.poc_lt[i] += SlicePOC - delta_poc_msb_cycle_lt * MaxPocLsb - pocLsb;
								prev_delta_msb = delta_poc_msb_cycle_lt;
							}
						}
					}
				}

				// 8.3.2 - Decoding process for reference picture set
				PocStCurrBefore.Empty();
				PocStCurrAfter.Empty();
				PocStFoll.Empty();
				PocLtCurr.Empty();
				PocLtFoll.Empty();
				CurrDeltaPocMsbPresentFlag.Empty();
				FollDeltaPocMsbPresentFlag.Empty();
				for(uint32 i=0; i<rps.NumDeltaPocs(); ++i)
				{
					int32 ref_pic_poc = SlicePOC + rps.GetDeltaPOC(i);
					TArray<int32>& ref_list = !rps.IsUsed(i) ? PocStFoll : (i < rps.NumNegativePics() ? PocStCurrBefore : PocStCurrAfter);
					ref_list.Emplace(ref_pic_poc);
				}
				for(uint32 i=0; i<rps.GetNumLongTermPics(); ++i)
				{
					int32 ref_pic_poc = rps.GetLongTermPOC(i);
					TArray<int32>& ref_list = rps.IsUsedLongTerm(i) ? PocLtCurr : PocLtFoll;
					ref_list.Emplace(ref_pic_poc);
					TArray<bool>& msb_list = rps.IsUsedLongTerm(i) ? CurrDeltaPocMsbPresentFlag : FollDeltaPocMsbPresentFlag;
					msb_list.Emplace(rps.IsLongTermPOCMSBPresent(i));
				}

				// Finally, update `prevTid0POC` as per 8.3.1:
				// "Let prevTid0Pic be the previous picture in decoding order that has TemporalId equal to 0 and that is not a RASL, RADL or SLNR picture."
				if (InFromSliceHeader.NuhTemporalIdPlus1-1 == 0 && InFromSliceHeader.NalUnitType != 7 && InFromSliceHeader.NalUnitType != 9 && !InFromSliceHeader.bIsSLNR)
				{
					prevTid0POC = SlicePOC;
				}
			}

			FDecodedPictureBuffer::FReferenceFrameListEntry* FDecodedPictureBuffer::GenerateMissingFrame(int32 InPOC, bool bIsLongterm)
			{
				FReferenceFrameListEntry* NewEntry = nullptr;
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex == -1)
					{
						NewEntry = &DPBEntries[i];
						NewEntry->DPBIndex = i;
						break;
					}
				}
				if (!NewEntry)
				{
					SetLastError(FString::Printf(TEXT("DPB overflow creating missing frame")));
					return NewEntry;
				}
				NewEntry->UserFrameInfo = FOutputFrameInfo();
				NewEntry->UserFrameInfo.bDoNotOutput = true;
				NewEntry->POC = InPOC;
				NewEntry->bIsShortTermReference = !bIsLongterm;
				NewEntry->bIsLongTermReference = bIsLongterm;
				NewEntry->bNeededForOutput = false;
				NewEntry->bIsMissing = true;
				NewEntry->PicLatencyCount = 0;
				return NewEntry;
			}

			bool FDecodedPictureBuffer::UpdatePOCandRPS(const FSliceSegmentHeader& InFromSliceHeader)
			{
				// Update POC and RPS (8.3.1 & 8.3.2)
				Update(InFromSliceHeader);

				// Apply the picture marking to the frames in the DPB

				// First, remove all "no reference picture" entries from the DPB
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].bIsMissing)
					{
						DPBEntries[i].bIsMissing = false;
						DPBEntries[i].DPBIndex = -1;
					}
				}
				// Unmark all referenced entries in the DPB
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					DPBEntries[i].bIsShortTermReference = false;
					DPBEntries[i].bIsLongTermReference = false;
				}
				DPBIdxStCurrBefore.Empty();
				DPBIdxStCurrAfter.Empty();
				DPBIdxStFoll.Empty();
				DPBIdxLtCurr.Empty();
				DPBIdxLtFoll.Empty();
				// Perform picture marking.
				for(int32 nList=0; nList<3; ++nList)
				{
					const TArray<int32>& StRefPOCs(nList==0?GetPocStCurrBefore():nList==1?GetPocStCurrAfter():GetPocStFoll());
					TArray<int32>& DpbIdxList(nList==0?DPBIdxStCurrBefore:nList==1?DPBIdxStCurrAfter:DPBIdxStFoll);
					for(int32 i=0; i<StRefPOCs.Num(); ++i)
					{
						bool bFound = false;
						for(int32 j=0; j<DPBEntries.Num(); ++j)
						{
							if (DPBEntries[j].DPBIndex >= 0 && DPBEntries[j].POC == StRefPOCs[i])
							{
								DPBEntries[j].bIsShortTermReference = true;
								DpbIdxList.Emplace(DPBEntries[j].DPBIndex);
								bFound = true;
								break;
							}
						}
						if (!bFound)
						{
							FReferenceFrameListEntry* MissingFrame = GenerateMissingFrame(StRefPOCs[i], false);
							if (!MissingFrame)
							{
								return false;
							}
							DpbIdxList.Emplace(MissingFrame->DPBIndex);
						}
					}
				}
				for(int32 nList=0; nList<2; ++nList)
				{
					const TArray<int32>& ltPoc = nList==0 ? GetPocLtCurr() : GetPocLtFoll();
					const TArray<bool>& ltMSB = nList==0 ? GetCurrDeltaPocMsbPresentFlag() : GetFollDeltaPocMsbPresentFlag();
					TArray<int32>& DpbIdxList(nList==0?DPBIdxLtCurr:DPBIdxLtFoll);
					check(ltPoc.Num() == ltMSB.Num());
					int32 Mask = GetMaxPocLsb() - 1;
					for(int32 j=0; j<ltPoc.Num(); ++j)
					{
						bool bFound = false;
						if (!ltMSB[j])
						{
							for(int32 k=0; k<DPBEntries.Num(); ++k)
							{
								if (DPBEntries[k].DPBIndex >= 0 && (DPBEntries[k].POC & Mask) == ltPoc[j])
								{
									DPBEntries[k].bIsLongTermReference = true;
									DpbIdxList.Emplace(DPBEntries[k].DPBIndex);
									bFound = true;
									break;
								}
							}
						}
						else
						{
							for(int32 k=0; k<DPBEntries.Num(); ++k)
							{
								if (DPBEntries[k].DPBIndex >= 0 && DPBEntries[k].POC == ltPoc[j])
								{
									DPBEntries[k].bIsLongTermReference = true;
									DpbIdxList.Emplace(DPBEntries[k].DPBIndex);
									bFound = true;
									break;
								}
							}
						}
						if (!bFound)
						{
							FReferenceFrameListEntry* MissingFrame = GenerateMissingFrame(ltPoc[j], true);
							if (!MissingFrame)
							{
								return false;
							}
							DpbIdxList.Emplace(MissingFrame->DPBIndex);
						}
					}
				}
				return true;
			}

			void FDecodedPictureBuffer::GetReferenceFramesFromDPB(TArray<FReferenceFrameListEntry>& OutReferenceFrames, TArray<int32> OutDPBIndexLists[eMAX])
			{
				OutReferenceFrames.Empty();
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex >= 0 && (DPBEntries[i].bIsShortTermReference || DPBEntries[i].bIsLongTermReference))
					{
						OutReferenceFrames.Emplace(DPBEntries[i]);
					}
				}
				OutDPBIndexLists[eStCurrBefore] = DPBIdxStCurrBefore;
				OutDPBIndexLists[eStCurrAfter] = DPBIdxStCurrAfter;
				OutDPBIndexLists[eStFoll] = DPBIdxStFoll;
				OutDPBIndexLists[eLtCurr] = DPBIdxLtCurr;
				OutDPBIndexLists[eLtFoll] = DPBIdxLtFoll;
			}

			const FDecodedPictureBuffer::FReferenceFrameListEntry* FDecodedPictureBuffer::GetDPBEntryAtIndex(int32 InIndex) const
			{
				if (InIndex >= 0 && InIndex < DPBEntries.Num())
				{
					if (DPBEntries[InIndex].DPBIndex >= 0)
					{
						return &DPBEntries[InIndex];
					}
				}
				return nullptr;
			}

			void FDecodedPictureBuffer::ProcessFirstSliceOfFrame(TArray<FDPBOutputFrame>& OutFrames, const FSliceSegmentHeader& InFromSliceHeader, bool bIsFirstInSequence)
			{
				/**
				 * 8.1.3 - Decoding process for a coded picture with nuh_layer_id equal to 0
				 *
				 * "When the current picture is an IRAP picture, the following applies:
				 *   - If the current picture is an IDR picture, a BLA picture, the first picture in the bitstream in decoding order,
				 *     or the first picture that follows an end of sequence NAL unit in decoding order, the variable NoRaslOutputFlag is set equal to 1.
				 *   - Otherwise, the variable HandleCraAsBlaFlag is set equal to 0 and the variable NoRaslOutputFlag is set equal to 0."
				 */
				bNoRaslOutputFlag = false;
				bCRANoRaslOutputFlag = false;
				// We go with the definition from 8.1.3.
				if (InFromSliceHeader.bIsIRAP)
				{
					bNoRaslOutputFlag = InFromSliceHeader.bIsIDR || InFromSliceHeader.bIsBLA || (InFromSliceHeader.bIsCRA && bIsFirstInSequence);
					if (InFromSliceHeader.bIsCRA)
					{
						// For output of RASL pictures
						bCRANoRaslOutputFlag = bNoRaslOutputFlag;
					}
				}

				// C.5.2.2 - Output and removal of pictures from the DPB
				UpdatePOCandRPS(InFromSliceHeader);

				/**
				 * Set up the `NoOutputOfPriorPicsFlag`
				 *
				 *   "If the current picture is an IRAP picture with NoRaslOutputFlag equal to 1 that is not picture 0..."
				 *
				 * We ignore the condition of "picture 0" since this is the only mention of it in the entire stadard and if
				 * this refers to the very first picture the DPB is empty anyway and this entire block does nothing anyway.
				 */
				if (InFromSliceHeader.bIsIRAP && bNoRaslOutputFlag)
				{
					bool bNoOutputOfPriorPicsFlag = false;
					// "If the current picture is a CRA picture, NoOutputOfPriorPicsFlag is set equal to 1 (regardless of the value of no_output_of_prior_pics_flag)."
					if (InFromSliceHeader.bIsCRA)
					{
						bNoOutputOfPriorPicsFlag = true;
					}
					else
					{
						bNoOutputOfPriorPicsFlag = !!InFromSliceHeader.no_output_of_prior_pics_flag;
					}
					if (bNoOutputOfPriorPicsFlag)
					{
						/*
							If NoOutputOfPriorPicsFlag is equal to 1, all picture storage buffers in the DPB are emptied without
							output of the pictures they contain, and the DPB fullness is set equal to 0.
						*/
						for(int32 i=0; i<DPBEntries.Num(); ++i)
						{
							if (DPBEntries[i].DPBIndex >= 0)
							{
								// We do not need to output missing frames that were not added through `AddDecodedFrame()`.
								if (!DPBEntries[i].bIsMissing)
								{
									FDPBOutputFrame& OutFrame = OutFrames.Emplace_GetRef();
									OutFrame.bDoNotDisplay = true;
									OutFrame.UserFrameInfo = DPBEntries[i].UserFrameInfo;
								}
								DPBEntries[i].Reset();
							}
						}
					}
					else
					{
						/*
							C.5.2.2 - Clause 2
								"..., all picture storage buffers containing a picture that is marked as "not needed for output" and "unused for reference"
								are emptied (without output), and all nonempty picture storage buffers in the DPB are emptied by repeatedly invoking the
								"bumping" process specified in clause C.5.2.4, and the DPB fullness is set equal to 0."
						*/
						EmitNoLongerNeeded(OutFrames);
						EmitAllRemainingAndEmpty(OutFrames);
					}
				}
				else
				{
					/*
						C.5.2.2
							"Otherwise (the current picture is not an IRAP picture with NoRaslOutputFlag equal to 1)"
					*/
					EmitNoLongerNeeded(OutFrames);
					BumpPictures(OutFrames, InFromSliceHeader, false);
				}
			}

			void FDecodedPictureBuffer::EmitNoLongerNeeded(TArray<FDPBOutputFrame>& OutFrames)
			{
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex >= 0 && !DPBEntries[i].bNeededForOutput && !(DPBEntries[i].bIsShortTermReference || DPBEntries[i].bIsLongTermReference))
					{
						if (!DPBEntries[i].bWasAlreadyOutput)
						{
							FDPBOutputFrame& OutFrame = OutFrames.Emplace_GetRef();
							OutFrame.bDoNotDisplay = true;
							OutFrame.UserFrameInfo = DPBEntries[i].UserFrameInfo;
						}
						DPBEntries[i].Reset();
					}
				}
			}

			void FDecodedPictureBuffer::EmitAllRemainingAndEmpty(TArray<FDPBOutputFrame>& OutFrames)
			{
				// Sort the DPB entries by POC value
				DPBEntries.Sort([](const FReferenceFrameListEntry& a, const FReferenceFrameListEntry& b)
				{
					return a.POC < b.POC;
				});

				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex >= 0)
					{
						if (!DPBEntries[i].bWasAlreadyOutput && !DPBEntries[i].bIsMissing)
						{
							FDPBOutputFrame& OutFrame = OutFrames.Emplace_GetRef();
							OutFrame.bDoNotDisplay = !DPBEntries[i].bNeededForOutput;
							OutFrame.UserFrameInfo = DPBEntries[i].UserFrameInfo;
#if LOG_DPB_FRAME_OUTPUT
							UE_LOGF(LogElectraDecoders, Log, "--- OUTPUT: POC %d `flush to empty`", DPBEntries[i].POC);
#endif
						}
					}
					DPBEntries[i].Reset();
				}
			}

			bool FDecodedPictureBuffer::AddDecodedFrame(TArray<FDPBOutputFrame>& OutFrames, const FOutputFrameInfo& InNewDecodedFrame, const FSliceSegmentHeader& InFromSliceHeader)
			{
				// Note: ProcessFirstSliceOfFrame() must have been called before adding this frame

				// Check that there is no entry in the DPB with the current POC already.
				if (DPBEntries.FindByPredicate([poc=GetSlicePOC()](const FReferenceFrameListEntry& e){ return e.DPBIndex >= 0 && e.POC == poc; }))
				{
					return SetLastError(FString::Printf(TEXT("POC %d already exists in DPB"), GetSlicePOC()));
				}
				FReferenceFrameListEntry* NewEntry = nullptr;
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex == -1)
					{
						NewEntry = &DPBEntries[i];
						NewEntry->DPBIndex = i;
						break;
					}
				}
				if (!NewEntry)
				{
					return SetLastError(FString::Printf(TEXT("DPB overflow")));
				}
				NewEntry->UserFrameInfo = InNewDecodedFrame;
				NewEntry->POC = GetSlicePOC();
				NewEntry->bIsShortTermReference = true;
				NewEntry->bIsLongTermReference = false;
				/*
					If the current picture is a RASL picture and NoRaslOutputFlag of the associated IRAP picture is
					equal to 1, PicOutputFlag is set equal to 0.
				*/
				if ((InFromSliceHeader.NalUnitType == 8/*RASL_N*/ || InFromSliceHeader.NalUnitType == 9/*RASL_R*/) && bCRANoRaslOutputFlag)
				{
					NewEntry->bNeededForOutput = false;
				}
				else
				{
					NewEntry->bNeededForOutput = InFromSliceHeader.pic_output_flag != 0;
				}
				NewEntry->bIsMissing = false;
				NewEntry->PicLatencyCount = 0;

				// C.5.2.3 - Additional bumping
				for(int32 i=0; i<DPBEntries.Num(); ++i)
				{
					if (DPBEntries[i].DPBIndex >= 0 && DPBEntries[i].POC > GetSlicePOC())
					{
						++DPBEntries[i].PicLatencyCount;
					}
				}
				BumpPictures(OutFrames, InFromSliceHeader, true);
				return true;
			}

			void FDecodedPictureBuffer::BumpPictures(TArray<FDPBOutputFrame>& OutFrames, const FSliceSegmentHeader& InFromSliceHeader, bool bAdditionalBumping)
			{
				const int32 sps_max_num_reorder_pics = (int32) InFromSliceHeader.sps_max_num_reorder_pics_HighestTid;
				const int32 sps_max_latency_increase_plus1 = (int32) InFromSliceHeader.sps_max_latency_increase_plus1_HighestTid;
				const int32 sps_max_dec_pic_buffering = (int32) InFromSliceHeader.sps_max_dec_pic_buffering_minus1_HighestTid + 1;
				const int32 SpsMaxLatencyPictures = sps_max_latency_increase_plus1 ? sps_max_num_reorder_pics + sps_max_latency_increase_plus1 - 1 : TNumericLimits<int32>::Max(); // See (7-9)
				while(1)
				{
					// Check the bumping conditions.
					int32 NumNeededForOutput = 0;
					int32 NumLatencyCount = 0;
					int32 NumInDPB = 0;
					for(auto& it : DPBEntries)
					{
						NumNeededForOutput += it.DPBIndex >= 0 && it.bNeededForOutput ? 1 : 0;
						NumLatencyCount += it.DPBIndex >= 0 && it.bNeededForOutput && it.PicLatencyCount >= SpsMaxLatencyPictures ? 1 : 0;
						NumInDPB += it.DPBIndex >= 0 ? 1 : 0;
					}
					if (NumNeededForOutput > sps_max_num_reorder_pics || (sps_max_latency_increase_plus1 !=0 && NumLatencyCount > 0) || (!bAdditionalBumping && NumInDPB >= sps_max_dec_pic_buffering))
					{
						int32 SmallestPOC = TNumericLimits<int32>::Max();
						int32 IndexOfSmallestPOC = -1;
						for(int32 i=0; i<DPBEntries.Num(); ++i)
						{
							if (DPBEntries[i].DPBIndex < 0)
							{
								continue;
							}
							if (DPBEntries[i].bNeededForOutput && DPBEntries[i].POC < SmallestPOC)
							{
								IndexOfSmallestPOC = i;
								SmallestPOC = DPBEntries[i].POC;
							}
						}
						check(IndexOfSmallestPOC >= 0);
#if LOG_DPB_FRAME_OUTPUT
						if (NumNeededForOutput > sps_max_num_reorder_pics)
						{
							UE_LOGF(LogElectraDecoders, Log, "--- OUTPUT %d: POC %d `sps_max_num_reorder_pics`", bAdditionalBumping, DPBEntries[IndexOfSmallestPOC].POC);
						}
						if (sps_max_latency_increase_plus1 !=0 && NumLatencyCount > 0)
						{
							UE_LOGF(LogElectraDecoders, Log, "--- OUTPUT %d: POC %d `sps_max_latency_increase_plus1`", bAdditionalBumping, DPBEntries[IndexOfSmallestPOC].POC);
						}
						if (!bAdditionalBumping && NumInDPB >= sps_max_dec_pic_buffering)
						{
							UE_LOGF(LogElectraDecoders, Log, "--- OUTPUT %d: POC %d `sps_max_dec_pic_buffering`", bAdditionalBumping, DPBEntries[IndexOfSmallestPOC].POC);
						}
#endif

						FDPBOutputFrame& OutFrame = OutFrames.Emplace_GetRef();
						OutFrame.bDoNotDisplay = false;
						OutFrame.UserFrameInfo = DPBEntries[IndexOfSmallestPOC].UserFrameInfo;
						DPBEntries[IndexOfSmallestPOC].bNeededForOutput = false;
						DPBEntries[IndexOfSmallestPOC].bWasAlreadyOutput = true;
						// If the DPB entry is not referenced we can clear it as it also holds no delayed output any more.
						if (!DPBEntries[IndexOfSmallestPOC].bIsShortTermReference && !DPBEntries[IndexOfSmallestPOC].bIsLongTermReference)
						{
							DPBEntries[IndexOfSmallestPOC].Reset();
						}
					}
					else
					{
						break;
					}
				}
			}

			void FDecodedPictureBuffer::Flush(TArray<FDPBOutputFrame>& OutRemainingFrames)
			{
				EmitAllRemainingAndEmpty(OutRemainingFrames);
				Reset();
			}

			void FDecodedPictureBuffer::Reset()
			{
				DPBEntries.Empty();
				DPBEntries.SetNum(32);
				DPBIdxStCurrBefore.Empty();
				DPBIdxStCurrAfter.Empty();
				DPBIdxStFoll.Empty();
				DPBIdxLtCurr.Empty();
				DPBIdxLtFoll.Empty();
				LastErrorMsg.Empty();
				FSliceSegmentHeaderPOCVars::Reset();

				bNoRaslOutputFlag = false;
			}

			bool FDecodedPictureBuffer::SetLastError(const FString& InLastErrorMsg)
			{
				LastErrorMsg = InLastErrorMsg;
				UE_LOGF(LogElectraDecoders, Error, "%ls", *InLastErrorMsg);
				return false;
			}










			bool FHEVCDecoderConfigurationRecord::Parse(const TConstArrayView<uint8>& InDCR)
			{
				CSD.Empty();
				RawData.Empty();

				if (InDCR.Num() >= 23)
				{
					RawData = InDCR;

					Electra::FBitstreamReader BitReader(InDCR.GetData(), InDCR.Num());

					ConfigurationVersion = (uint8)BitReader.GetBits(8);
					if (ConfigurationVersion != 1)
					{
						return false;
					}
					GeneralProfileSpace = (uint8)BitReader.GetBits(2);
					GeneralTierFlag = (uint8)BitReader.GetBits(1);
					GeneralProfileIDC = (uint8)BitReader.GetBits(5);
					GeneralProfileCompatibilityFlags  = BitReader.GetBits(32);
					for(int32 i=0; i<48; ++i)
					{
						GeneralConstraintIndicatorFlags = (GeneralConstraintIndicatorFlags << 1U) | uint64(BitReader.GetBits(1));
					}
					GeneralLevelIDC = (uint8)BitReader.GetBits(8);
					if (BitReader.GetBits(4) != 15)
					{
						return false;
					}
					MinSpatialSegmentationIDC = (uint16)BitReader.GetBits(12);
					if (BitReader.GetBits(6) != 63)
					{
						return false;
					}
					ParallelismType = (uint8)BitReader.GetBits(2);
					if (BitReader.GetBits(6) != 63)
					{
						return false;
					}
					ChromaFormat = (uint8)BitReader.GetBits(2);
					if (BitReader.GetBits(5) != 31)
					{
						return false;
					}
					BitDepthLumaMinus8 = (uint8)BitReader.GetBits(3);
					if (BitReader.GetBits(5) != 31)
					{
						return false;
					}
					BitDepthChromaMinus8 = (uint8)BitReader.GetBits(3);
					AverageFrameRate = (uint16)BitReader.GetBits(16);
					ConstantFrameRate = (uint8)BitReader.GetBits(2);
					NumTemporalLayers = (uint8)BitReader.GetBits(3);
					TemporalIdNested = (uint8)BitReader.GetBits(1);
					NALUnitLengthMinus1 = (uint8)BitReader.GetBits(2);
					uint32 numArrays = BitReader.GetBits(8);
					Arrays.Reserve(numArrays);
					for(uint32 i=0; i<numArrays; ++i)
					{
						FArray &a = Arrays.AddDefaulted_GetRef();
						a.Completeness = (uint8)BitReader.GetBits(1);
						if (BitReader.GetBits(1) != 0)
						{
							return false;
						}
						a.NALUnitType = (uint8)BitReader.GetBits(6);
						uint32 numNALUs = BitReader.GetBits(16);
						a.NALUs.Reserve(numNALUs);
						for(uint32 j=0; j<numNALUs; ++j)
						{
							uint16 naluLen = (uint16)BitReader.GetBits(16);
							TArray<uint8> &n = a.NALUs.AddDefaulted_GetRef();
							n.Reserve(naluLen);
							for(uint32 k=0; k<naluLen; ++k)
							{
								n.Emplace((uint8)BitReader.GetBits(8));
							}
						}
					}
					if (BitReader.GetRemainingBits() != 0)
					{
						return false;
					}

					// CSD
					for(int32 i=0; i<Arrays.Num(); ++i)
					{
						const FArray& a = Arrays[i];
						for(int32 j=0; j<a.NALUs.Num(); ++j)
						{
							CSD.Add(0);
							CSD.Add(0);
							CSD.Add(0);
							CSD.Add(1);
							CSD.Append(a.NALUs[j]);
						}

						// VPS nut?
						if (a.NALUnitType == 32)
						{
							for(int32 k=0; k<a.NALUs.Num(); ++k)
							{
								TArray<uint8>& vps(VideoParameterSets.Emplace_GetRef());
								vps.Append(a.NALUs[k]);
								if (!ParseVideoParameterSet(ParsedVideoParameterSets.Emplace_GetRef(), a.NALUs[k].GetData(), a.NALUs[k].Num()))
								{
									ParsedVideoParameterSets.RemoveAt(ParsedVideoParameterSets.Num()-1);
								}
							}
						}
						// SPS nut?
						else if (a.NALUnitType == 33)
						{
							for(int32 k=0; k<a.NALUs.Num(); ++k)
							{
								TArray<uint8>& sps(SequenceParameterSets.Emplace_GetRef());
								sps.Append(a.NALUs[k]);
								if (!ParseSequenceParameterSet(ParsedSequenceParameterSets.Emplace_GetRef(), a.NALUs[k].GetData(), a.NALUs[k].Num()))
								{
									ParsedSequenceParameterSets.RemoveAt(ParsedSequenceParameterSets.Num()-1);
								}
							}
						}
						// Prefix nut?
						else if (a.NALUnitType == 39)
						{
							for(int32 k=0; k<a.NALUs.Num(); ++k)
							{
								TArray<uint8>& Prefix(PrefixNUTs.Emplace_GetRef());
								Prefix.Append(a.NALUs[k]);
							}
						}
						// Suffix nut?
						else if (a.NALUnitType == 40)
						{
							for(int32 k=0; k<a.NALUs.Num(); ++k)
							{
								TArray<uint8>& Suffix(SuffixNUTs.Emplace_GetRef());
								Suffix.Append(a.NALUs[k]);
							}
						}
					}
					return true;
				}
				return false;
			}

			FString FHEVCDecoderConfigurationRecord::GetCodecSpecifierRFC6381(const TCHAR* InPrefix) const
			{
				/*
				// If we have a parsed SPS we use that.
				if (ParsedSequenceParameterSets.Num())
				{
					return ParsedSequenceParameterSets[0].GetRFC6381(InPrefix);
				}
				*/
				// Construct one from the general information as per ISO/IEC 14496-15:2022 Annex E.3
				FString cfs;
				uint64 ConstraintFlags = GeneralConstraintIndicatorFlags;
				bool bNonZero = false;
				for(int32 i=5; i>=0; --i, ConstraintFlags >>= 8)
				{
					if ((ConstraintFlags & 255) == 0 && !bNonZero)
					{
						continue;
					}
					bNonZero = true;
					cfs = FString::Printf(TEXT(".%02X"), (uint8)(ConstraintFlags & 255)) + cfs;
				}
				const uint32 cf = BitReverse32(GeneralProfileCompatibilityFlags);
				FString rfc;
				if (GeneralProfileSpace == 0)
				{
					rfc = FString::Printf(TEXT("%s.%d.%X.%c%d%s"), InPrefix, GeneralProfileIDC, cf, GeneralTierFlag ? TCHAR('H') : TCHAR('L'), GeneralLevelIDC, *cfs);
				}
				else
				{
					rfc = FString::Printf(TEXT("%s.%c%d.%X.%c%d%s"), InPrefix, TCHAR(TCHAR('A')+GeneralProfileSpace-1), GeneralProfileIDC, cf, GeneralTierFlag ? TCHAR('H') : TCHAR('L'), GeneralLevelIDC, *cfs);
				}
				return rfc;
			}

			FString FHEVCDecoderConfigurationRecord::GetFormatInfo() const
			{
				FString fi(TEXT("HEVC, "));
				fi.Append(FSequenceParameterSet::GetFormatInfo(GeneralProfileIDC, GeneralLevelIDC, GeneralConstraintIndicatorFlags));
				return fi;
			}

			bool FHEVCDecoderConfigurationRecord::CreateFromCodecSpecificData(const TConstArrayView<uint8>& InCSD)
			{
				CSD.Empty();
				RawData.Empty();
				if (InCSD.IsEmpty())
				{
					return false;
				}

				CSD = InCSD;
				TArray<FNaluInfo> Nalus;
				if (!ParseBitstreamForNALUs(Nalus, InCSD.GetData(), InCSD.Num()))
				{
					return false;
				}
				FArray aVPS, aSPS, aPPS;
				for(int32 i=0; i<Nalus.Num(); ++i)
				{
					if (Nalus[i].Type == 32)
					{
						aVPS.NALUnitType = 32;
						aVPS.NALUs.Emplace_GetRef() = MakeArrayView<const uint8>(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
					}
					else if (Nalus[i].Type == 33)
					{
						FSequenceParameterSet sps;
						if (ParseSequenceParameterSet(sps, InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size))
						{
							GeneralProfileSpace = sps.profile_tier_level.general_profile_space;
							GeneralTierFlag = sps.profile_tier_level.general_tier_flag;
							GeneralProfileIDC = sps.profile_tier_level.general_profile_idc;
							GeneralProfileCompatibilityFlags = sps.profile_tier_level.general_profile_compatibility_flags;
							GeneralConstraintIndicatorFlags = sps.profile_tier_level.general_constraint_indicator_flags;
							GeneralLevelIDC = sps.profile_tier_level.general_level_idc;
							check(sps.vui_parameters_present_flag);
							MinSpatialSegmentationIDC = sps.vui_parameters.min_spatial_segmentation_idc;
							ParallelismType = 0;	// for simplicity as 0 indicates `unknown`.
							ChromaFormat = sps.chroma_format_idc;
							BitDepthLumaMinus8 = sps.bit_depth_luma_minus8;
							BitDepthChromaMinus8 = sps.bit_depth_chroma_minus8;
							AverageFrameRate = 0;	// unknown
							ConstantFrameRate = 0;	// may or may not
							NumTemporalLayers = 0;	// unknown
							TemporalIdNested = 0;	// may or may not
							NALUnitLengthMinus1 = 3;

							TArray<uint8>& NewSPS(SequenceParameterSets.Emplace_GetRef());
							NewSPS.Append(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
							ParsedSequenceParameterSets.Emplace(MoveTemp(sps));
						}
						else
						{
							return false;
						}
						aSPS.NALUnitType = 33;
						aSPS.NALUs.Emplace_GetRef() = MakeArrayView<const uint8>(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
					}
					else if (Nalus[i].Type == 34)
					{
						aPPS.NALUnitType = 34;
						aPPS.NALUs.Emplace_GetRef() = MakeArrayView<const uint8>(InCSD.GetData() + Nalus[i].Offset + Nalus[i].UnitLength, Nalus[i].Size);
					}
				}
				if (aVPS.NALUs.IsEmpty() || aSPS.NALUs.IsEmpty() || aPPS.NALUs.IsEmpty())
				{
					return false;
				}
				Arrays.Emplace(MoveTemp(aVPS));
				Arrays.Emplace(MoveTemp(aSPS));
				Arrays.Emplace(MoveTemp(aPPS));

				// Create the raw configuration record now.
				ConfigurationVersion = 1;
				Electra::FBitstreamWriter wr;
				wr.PutBits(ConfigurationVersion, 8);
				wr.PutBits(GeneralProfileSpace, 2);
				wr.PutBits(GeneralTierFlag, 1);
				wr.PutBits(GeneralProfileIDC, 5);
				wr.PutBits(GeneralProfileCompatibilityFlags, 32);
				wr.PutBits64(GeneralConstraintIndicatorFlags, 48);
				wr.PutBits(GeneralLevelIDC, 8);
				wr.PutBits(15U, 4);
				wr.PutBits(MinSpatialSegmentationIDC, 12);
				wr.PutBits(63U, 6);
				wr.PutBits(ParallelismType, 2);
				wr.PutBits(63U, 6);
				wr.PutBits(ChromaFormat, 2);
				wr.PutBits(31U, 5);
				wr.PutBits(BitDepthLumaMinus8, 3);
				wr.PutBits(31U, 5);
				wr.PutBits(BitDepthChromaMinus8, 3);
				wr.PutBits(AverageFrameRate, 16);
				wr.PutBits(ConstantFrameRate, 2);
				wr.PutBits(NumTemporalLayers, 3);
				wr.PutBits(TemporalIdNested, 1);
				wr.PutBits(NALUnitLengthMinus1, 2);
				wr.PutBits(static_cast<uint32>(Arrays.Num()), 8);
				for(int32 i=0; i<Arrays.Num(); ++i)
				{
					const FArray& a(Arrays[i]);
					wr.PutBits(a.Completeness, 1);
					wr.PutBits(0U, 1);
					wr.PutBits(a.NALUnitType, 6);
					wr.PutBits(static_cast<uint32>(a.NALUs.Num()), 16);
					for(int32 j=0; j<a.NALUs.Num(); ++j)
					{
						const TArray<uint8>& d(a.NALUs[j]);
						wr.PutBits(static_cast<uint32>(d.Num()), 16);
						for(int32 k=0; k<d.Num(); ++k)
						{
							wr.PutBits(d[k], 8);
						}
					}
				}
				wr.GetArray(RawData);
				return true;
			}




		} // namespace H265
	} // namespace MPEG
} // namespace ElectraDecodersUtil
