// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/OpenAPV/ElectraBitstreamProcessor_APV.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersUtils.h"


class FElectraDecoderBitstreamProcessorAPV::FImpl
{
public:
	class FBitstreamInfo : public IElectraDecoderBitstreamInfo
	{
	public:
		virtual ~FBitstreamInfo() = default;
	};

	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties)
	{
		if (CurrentColorimetry.IsValid() && bColorimetryChanged)
		{
			bColorimetryChanged = false;
			TConstArrayView<uint8> Colorimetry = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentColorimetry.Get()), sizeof(ElectraDecodersUtil::MPEG::FCommonColorimetry));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::CommonColorimetry, TArray<uint8>(Colorimetry));
		}

		if (CurrentMDCV.IsValid() && bMDCVChanged)
		{
			bMDCVChanged = false;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentMDCV.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiMasteringDisplayColorVolume, TArray<uint8>(Params));
		}

		if (CurrentCLLI.IsValid() && bCLLIChanged)
		{
			bCLLIChanged = false;
			TConstArrayView<uint8> Params = MakeConstArrayView(reinterpret_cast<const uint8*>(CurrentCLLI.Get()), sizeof(ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info));
			InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::SeiContentLightLeveInfo, TArray<uint8>(Params));
		}
	}

	void UpdateColorimetry(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
	{
		if (!CurrentColorimetry.IsValid() ||
			CurrentColorimetry->colour_primaries != colour_primaries ||
			CurrentColorimetry->transfer_characteristics != transfer_characteristics ||
			CurrentColorimetry->matrix_coeffs != matrix_coeffs ||
			CurrentColorimetry->video_full_range_flag != video_full_range_flag ||
			CurrentColorimetry->video_format != video_format)
		{
			CurrentColorimetry = MakeShared<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe>();
			CurrentColorimetry->colour_primaries = colour_primaries;
			CurrentColorimetry->transfer_characteristics = transfer_characteristics;
			CurrentColorimetry->matrix_coeffs = matrix_coeffs;
			CurrentColorimetry->video_full_range_flag = video_full_range_flag;
			CurrentColorimetry->video_format = video_format;
			bColorimetryChanged = true;
		}
	}

	void Clear()
	{
		CurrentColorimetry.Reset();
		CurrentMDCV.Reset();
		CurrentCLLI.Reset();
		bColorimetryChanged = false;
		bMDCVChanged = false;
		bCLLIChanged = false;
		LastErrorMessage.Empty();
	}

	static FString DCRName;
	static FString CSDName;

	TSharedPtr<ElectraDecodersUtil::MPEG::FCommonColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume, ESPMode::ThreadSafe> CurrentMDCV;
	TSharedPtr<ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info, ESPMode::ThreadSafe> CurrentCLLI;

	bool bModifiesInputAUInPlace = false;

	bool bColorimetryChanged = false;
	bool bMDCVChanged = false;
	bool bCLLIChanged = false;

	FString LastErrorMessage;
};
FString FElectraDecoderBitstreamProcessorAPV::FImpl::DCRName(TEXT("dcr"));
FString FElectraDecoderBitstreamProcessorAPV::FImpl::CSDName(TEXT("csd"));


FElectraDecoderBitstreamProcessorAPV::~FElectraDecoderBitstreamProcessorAPV()
{
}

FElectraDecoderBitstreamProcessorAPV::FElectraDecoderBitstreamProcessorAPV(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
{
	Impl = MakePimpl<FImpl>();
	Impl->bModifiesInputAUInPlace = !!ElectraDecodersUtil::GetVariantValueSafeI64(InDecoderParams, IElectraDecoderFeature::ModifiesInputAUInPlace, 0);
}

bool FElectraDecoderBitstreamProcessorAPV::WillModifyBitstreamInPlace()
{
	return Impl->bModifiesInputAUInPlace;
}

void FElectraDecoderBitstreamProcessorAPV::Clear()
{
	Impl->Clear();
}

IElectraDecoderBitstreamProcessor::EProcessResult FElectraDecoderBitstreamProcessorAPV::ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData)
{
	// Already processed?
	if ((InOutAccessUnit.Flags & EElectraDecoderFlags::InputIsProcessed) != EElectraDecoderFlags::None)
	{
		return EProcessResult::Ok;
	}

	// Set to processed even if we fail somewhere now.
	InOutAccessUnit.Flags |= EElectraDecoderFlags::InputIsProcessed;
	// Every frame is a keyframe
	InOutAccessUnit.Flags |= EElectraDecoderFlags::IsSyncSample;

#if 0
	// Go over the data and look at the PBUs
	const uint8* CurrentPBU = reinterpret_cast<const uint8*>(InOutAccessUnit.Data);
	const uint8* EndOfPBUs = CurrentPBU + InOutAccessUnit.DataSize;
	// Skip over size and signature if present.
	if (InOutAccessUnit.DataSize >= 8 && Electra::GetFromBigEndian(reinterpret_cast<const uint32*>(InOutAccessUnit.Data)[1]) == 0x61507631)
	{
		CurrentPBU += 8;
	}

	while(CurrentPBU < EndOfPBUs)
	{
		const uint32 PBUsize = Electra::GetFromBigEndian(*reinterpret_cast<const uint32*>(CurrentPBU));
		switch(CurrentPBU[4])
		{
			case 1:		// primary frame
			case 2:		// non-primary frame
			case 25:	// preview frame
			case 26:	// depth frame
			case 27:	// alpha frame
			case 65:	// access unit information
			case 67:	// filler
			default:
			{
				break;
			}
			case 66:	// metadata
			{
				break;
			}
		}
		CurrentPBU += PBUsize + 4;
	}
#endif
	return EProcessResult::Ok;
}

void FElectraDecoderBitstreamProcessorAPV::SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI)
{
	Impl->SetPropertiesOnOutput(InOutProperties);
//	uint8 num_bits = 8;
//	InOutProperties.Emplace(IElectraDecoderBitstreamProcessorInfo::NumBitsLuma, FVariant(num_bits));
}

FString FElectraDecoderBitstreamProcessorAPV::GetLastError()
{
	return Impl->LastErrorMessage;
}
