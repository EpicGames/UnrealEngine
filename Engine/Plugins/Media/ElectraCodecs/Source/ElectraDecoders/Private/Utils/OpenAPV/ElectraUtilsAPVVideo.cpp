// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/OpenAPV/ElectraUtilsAPVVideo.h"
#include "Utilities/ElectraBitstream.h"
#include "MP4Utilities.h"


namespace ElectraDecodersUtil
{
	namespace APVVideo
	{

		bool ParseFrameHeader(FFrameInfo& OutFrameInfo, const TConstArrayView<uint8> InFramePBU)
		{
			Electra::FBitstreamReader bs(InFramePBU.GetData(), InFramePBU.Num(), 4);	// skip over pbu type, group id and reserved
			if (bs.GetRemainingByteLength() < 14)
			{
				return false;
			}
			// frame_info
			OutFrameInfo.ProfileIDC = bs.GetBits(8);
			OutFrameInfo.LevelIDC = bs.GetBits(8);
			OutFrameInfo.BandIDC = bs.GetBits(3);
			bs.SkipBits(5);
			OutFrameInfo.FrameWidth = bs.GetBits(24);
			OutFrameInfo.FrameHeight = bs.GetBits(24);
			OutFrameInfo.ChromaFormatIDC = bs.GetBits(4);
			OutFrameInfo.BitDepthMinus8 = bs.GetBits(4);
			OutFrameInfo.CaptureTimeDistance = bs.GetBits(8);
			bs.SkipBits(8);
			// rest
			bs.SkipBits(8);
			OutFrameInfo.bHaveColorDescription = !!bs.GetBits(1);
			if (OutFrameInfo.bHaveColorDescription)
			{
				if (bs.GetRemainingBits() < 25)
				{
					return false;
				}
				OutFrameInfo.ColorInfo.ColorPrimaries = bs.GetBits(8);
				OutFrameInfo.ColorInfo.TransferCharacteristics = bs.GetBits(8);
				OutFrameInfo.ColorInfo.MatrixCoefficients = bs.GetBits(8);
				OutFrameInfo.ColorInfo.VideoFullRangeFlag = bs.GetBits(1);
			}
			// the remainder is of no interest to us.
			return true;
		}


		FString FAPVDecoderConfigurationRecord::GetFormatInfo(uint32 InProfile, uint32 InLevel)
		{
			FString fi;
			if (InProfile == 33)
			{
				fi = TEXT("422-10");
			}
			else if (InProfile == 44)
			{
				fi = TEXT("422-12");
			}
			else if (InProfile == 55)
			{
				fi = TEXT("444-10");
			}
			else if (InProfile == 66)
			{
				fi = TEXT("444-12");
			}
			else if (InProfile == 77)
			{
				fi = TEXT("4444-10");
			}
			else if (InProfile == 88)
			{
				fi = TEXT("4444-12");
			}
			else if (InProfile == 99)
			{
				fi = TEXT("400-10");
			}
			else
			{
				fi = TEXT("Unknown");
			}
			fi.Append(TEXT(" Profile"));
			if (InLevel >= 30 && InLevel < 240)
			{
				fi.Append(FString::Printf(TEXT(", level %d.%d"), InLevel/30 , InLevel%30));
			}
			else
			{
				fi.Append(TEXT(" at unknown level"));
			}
			return fi;
		}

		FString FAPVDecoderConfigurationRecord::GetFormatInfo()
		{
			return FString::Printf(TEXT("APV, %s"), *GetFormatInfo(Profile, Level));
		}


		bool FAPVDecoderConfigurationRecord::Parse(const TConstArrayView<uint8> InDCR)
		{
			RawData.Empty();
			ColorInfo = FColorInfo();
			Width = 0;
			Height = 0;
			Profile = 0;
			Level = 0;
			Band = 0;
			BitDepth = 0;
			LargestProfile = 0;
			LargestLevel = 0;
			LargestBand = 0;
			NumEncodedFrameVariants = 0;
			bIsNonConformant = false;

			if (InDCR.Num() < 2 || InDCR[0] != 1)
			{
				return false;
			}
			RawData = InDCR;
			const uint8* InapvCBox = InDCR.GetData() + 1;
			const uint8* End = InDCR.GetData() + InDCR.Num();
			const int32 NumCfgs = *InapvCBox++;
			bool bHavePrimary = false;
			for(int32 i=0; i<NumCfgs && InapvCBox+1<End; ++i)
			{
				FConfigurationEntry e;
				e.PBUType = *InapvCBox++;
				const int32 NumFi = *InapvCBox++;
				for(int32 j=0; j<NumFi; ++j)
				{
					// Have enough remaining data?
					if (InapvCBox+13 >= End)
					{
						return false;
					}
					FFrameInfo fi;
					fi.bHaveColorDescription = (*InapvCBox & 2) != 0;
					fi.bCaptureTimeDistanceIgnored = (*InapvCBox & 1) != 0;
					++InapvCBox;
					fi.ProfileIDC = *InapvCBox++;
					fi.LevelIDC = *InapvCBox++;
					fi.BandIDC = *InapvCBox++;
					fi.FrameWidth = MP4Utilities::GetFromBigEndian(*reinterpret_cast<const uint32*>(InapvCBox));
					InapvCBox += 4;
					fi.FrameHeight = MP4Utilities::GetFromBigEndian(*reinterpret_cast<const uint32*>(InapvCBox));
					InapvCBox += 4;
					fi.ChromaFormatIDC = *InapvCBox >> 4;
					fi.BitDepthMinus8 = *InapvCBox++ & 15;
					fi.CaptureTimeDistance = *InapvCBox++;
					if (fi.bHaveColorDescription)
					{
						// Have enough remaining data?
						if (InapvCBox+3 >= End)
						{
							return false;
						}
						fi.ColorInfo.ColorPrimaries = *InapvCBox++;
						fi.ColorInfo.TransferCharacteristics = *InapvCBox++;
						fi.ColorInfo.MatrixCoefficients = *InapvCBox++;
						fi.ColorInfo.VideoFullRangeFlag = *InapvCBox++ >> 7;
					}
					// Keep track of the largest of profile, level and band for the RFC 6381 sub parameters
					LargestProfile = fi.ProfileIDC > LargestProfile ? fi.ProfileIDC : LargestProfile;
					LargestLevel = fi.LevelIDC > LargestLevel ? fi.LevelIDC : LargestLevel;
					LargestBand = fi.BandIDC > LargestBand ? fi.BandIDC : LargestBand;
					e.FrameInfos.Emplace(fi);
					++NumEncodedFrameVariants;
				}
				// Is this the primary frame?
				if (e.PBUType == 1)
				{
					bHavePrimary = true;
					// Set the convenience values from the primary frame
					Width = e.FrameInfos[0].FrameWidth;
					Height = e.FrameInfos[0].FrameHeight;
					ColorInfo = e.FrameInfos[0].ColorInfo;
					Profile = e.FrameInfos[0].ProfileIDC;
					Level = e.FrameInfos[0].LevelIDC;
					Band = e.FrameInfos[0].BandIDC;
					BitDepth = 8 + e.FrameInfos[0].BitDepthMinus8;
					// There can be at most one primary frame.
					bIsNonConformant = bIsNonConformant | (e.FrameInfos.Num() > 1);
				}
				ConfigurationEntries.Emplace(MoveTemp(e));
			}
			if (ConfigurationEntries.IsEmpty())
			{
				return false;
			}
			// A primary frame is required.
			bIsNonConformant |= !bHavePrimary;
			return true;
		}

		bool FAPVDecoderConfigurationRecord::ParseFromAU(const TConstArrayView64<uint8> InAccessUnit)
		{
			struct FPBU
			{
				uint8 Type;
				TConstArrayView<uint8> Data;
			};
			RawData.Empty();
			ColorInfo = FColorInfo();
			Width = 0;
			Height = 0;
			Profile = 0;
			Level = 0;
			Band = 0;
			BitDepth = 0;
			LargestProfile = 0;
			LargestLevel = 0;
			LargestBand = 0;
			NumEncodedFrameVariants = 0;
			bIsNonConformant = false;

			const uint8* CurrentPBU = reinterpret_cast<const uint8*>(InAccessUnit.GetData());
			const uint8* EndOfPBUs = CurrentPBU + InAccessUnit.Num();
			bool bHavePrimary = false;
			// Skip over size and signature if present.
			if (InAccessUnit.Num() >= 8 && MP4Utilities::GetFromBigEndian(reinterpret_cast<const uint32*>(CurrentPBU)[1]) == 0x61507631)
			{
				CurrentPBU += 8;
			}
			TMap<uint8, TArray<FPBU>> PBUs;
			while(CurrentPBU < EndOfPBUs)
			{
				const uint32 PBUsize = MP4Utilities::GetFromBigEndian(*reinterpret_cast<const uint32*>(CurrentPBU));
				FPBU pbu;
				pbu.Type = CurrentPBU[4];
				pbu.Data = MakeConstArrayView(CurrentPBU + 4, PBUsize);
				CurrentPBU += PBUsize + 4;
				TArray<FPBU>& pa = PBUs.FindOrAdd(pbu.Type);
				pa.Emplace(pbu);
			}

			// Parse the values from a frame.
			auto ParseFrameType = [&](uint8 InType) -> bool
			{
				if (PBUs.Contains(InType))
				{
					FConfigurationEntry* cfgE = nullptr;
					const TArray<FPBU>& PBUArr(PBUs[InType]);
					for(int32 i=0; i<PBUArr.Num(); ++i)
					{
						FFrameInfo fi;
						if (ParseFrameHeader(fi, PBUArr[i].Data))
						{
							cfgE = ConfigurationEntries.FindByPredicate([&](const FConfigurationEntry& ce){return ce.PBUType == InType;});
							if (!cfgE)
							{
								ConfigurationEntries.Emplace_GetRef().PBUType = InType;
								cfgE = &ConfigurationEntries.Last();
							}
							// Keep track of the largest of profile, level and band for the RFC 6381 sub parameters
							LargestProfile = fi.ProfileIDC > LargestProfile ? fi.ProfileIDC : LargestProfile;
							LargestLevel = fi.LevelIDC > LargestLevel ? fi.LevelIDC : LargestLevel;
							LargestBand = fi.BandIDC > LargestBand ? fi.BandIDC : LargestBand;
							cfgE->FrameInfos.Emplace(fi);
							++NumEncodedFrameVariants;
						}
						else
						{
							return false;
						}
					}
					// Record convenience values for primary frame only, or any type of frame if there is only one.
					if (cfgE && (InType == 1 || PBUs.Num() == 1))
					{
						bHavePrimary = InType == 1;
						// Set the convenience values from the primary frame
						Width = cfgE->FrameInfos[0].FrameWidth;
						Height = cfgE->FrameInfos[0].FrameHeight;
						ColorInfo = cfgE->FrameInfos[0].ColorInfo;
						Profile = cfgE->FrameInfos[0].ProfileIDC;
						Level = cfgE->FrameInfos[0].LevelIDC;
						Band = cfgE->FrameInfos[0].BandIDC;
						BitDepth = 8 + cfgE->FrameInfos[0].BitDepthMinus8;
						// There can be at most one primary frame.
						bIsNonConformant = bIsNonConformant | (InType == 1 && cfgE->FrameInfos.Num() > 1);
					}
				}
				return true;
			};
			// Check all type of frames
			if (!ParseFrameType(1) || !ParseFrameType(2) || !ParseFrameType(25) || !ParseFrameType(26) || !ParseFrameType(27))
			{
				return false;
			}
			if (ConfigurationEntries.IsEmpty())
			{
				return false;
			}
			// A primary frame is required.
			bIsNonConformant |= !bHavePrimary;
			// Build the raw data
			Electra::FBitstreamWriter wr;
			wr.PutBits(1U, 8);
			wr.PutBits((uint32) ConfigurationEntries.Num(), 8);
			for(int32 i=0; i<ConfigurationEntries.Num(); ++i)
			{
				const FConfigurationEntry& e(ConfigurationEntries[i]);
				wr.PutBits(e.PBUType, 8);
				wr.PutBits((uint32)e.FrameInfos.Num(), 8);
				for(int32 j=0; j<e.FrameInfos.Num(); ++j)
				{
					const FFrameInfo& fi(e.FrameInfos[j]);
					wr.PutBits(0U, 6);
					wr.PutBits(fi.bHaveColorDescription?1U:0U, 1);
					wr.PutBits(fi.bCaptureTimeDistanceIgnored?1U:0U, 1);
					wr.PutBits(fi.ProfileIDC, 8);
					wr.PutBits(fi.LevelIDC, 8);
					wr.PutBits(fi.BandIDC, 8);
					wr.PutBits(fi.FrameWidth, 32);
					wr.PutBits(fi.FrameHeight, 32);
					wr.PutBits(fi.ChromaFormatIDC, 4);
					wr.PutBits(fi.BitDepthMinus8, 4);
					wr.PutBits(fi.CaptureTimeDistance, 8);
					if (fi.bHaveColorDescription)
					{
						wr.PutBits(fi.ColorInfo.ColorPrimaries, 8);
						wr.PutBits(fi.ColorInfo.TransferCharacteristics, 8);
						wr.PutBits(fi.ColorInfo.MatrixCoefficients, 8);
						wr.PutBits(fi.ColorInfo.VideoFullRangeFlag, 1);
						wr.PutBits(0U, 7);
					}
				}
			}
			wr.GetArray(RawData);
			return true;
		}


		void ParseAUIntoFramePBUSubsamples(TArray<FFramePBUInfo>& OutPBUInfos, const TConstArrayView64<uint8> InAccessUnit)
		{
			const uint8* AUBase = reinterpret_cast<const uint8*>(InAccessUnit.GetData());
			const uint8* CurrentPBU = AUBase;
			const uint8* EndOfPBUs = CurrentPBU + InAccessUnit.Num();
			// Skip over size and signature if present.
			if (InAccessUnit.Num() >= 8 && MP4Utilities::GetFromBigEndian(reinterpret_cast<const uint32*>(CurrentPBU)[1]) == 0x61507631)
			{
				CurrentPBU += 8;
			}
			// If there is no size, skip over signature if present.
			else if (InAccessUnit.Num() >= 4 && MP4Utilities::GetFromBigEndian(reinterpret_cast<const uint32*>(CurrentPBU)[0]) == 0x61507631)
			{
				CurrentPBU += 4;
			}
			while(CurrentPBU < EndOfPBUs)
			{
				const uint32 PBUsize = MP4Utilities::GetFromBigEndian(*reinterpret_cast<const uint32*>(CurrentPBU));
				const uint8 Type = CurrentPBU[4];
				// Any frame type PBU?
				if (Type == 1 || Type == 2 || Type == 25 || Type == 26 || Type == 27)
				{
					if (!OutPBUInfos.IsEmpty())
					{
						OutPBUInfos.Last().PBUSize = (CurrentPBU - AUBase) - OutPBUInfos.Last().PBUOffset;
					}
					FFramePBUInfo& pi = OutPBUInfos.Emplace_GetRef();
					pi.bIsPrimaryFrame = Type == 1;
					pi.PBUOffset = CurrentPBU - AUBase;
				}
				CurrentPBU += PBUsize + 4;
			}
			if (!OutPBUInfos.IsEmpty())
			{
				OutPBUInfos.Last().PBUSize = (CurrentPBU - AUBase) - OutPBUInfos.Last().PBUOffset;
			}
		}

	} // namespace APVVideo
} // namespace ElectraDecodersUtil
