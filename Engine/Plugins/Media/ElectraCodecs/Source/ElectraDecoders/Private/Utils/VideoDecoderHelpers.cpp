// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/VideoDecoderHelpers.h"
#include "MP4Utilities.h"
#include "MediaDecoderOutput.h"

namespace Electra
{

namespace MPEG
{

void FColorimetryHelper::Reset()
{
	CurrentColorimetry.Reset();
}

void FColorimetryHelper::Update(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
{
	if (CurrentColorimetry.IsValid() &&
		CurrentColorimetry->Colorimetry.ColourPrimaries == colour_primaries &&
		CurrentColorimetry->Colorimetry.MatrixCoefficients == matrix_coeffs &&
		CurrentColorimetry->Colorimetry.TransferCharacteristics == transfer_characteristics &&
		CurrentColorimetry->Colorimetry.VideoFullRangeFlag == video_full_range_flag &&
		CurrentColorimetry->Colorimetry.VideoFormat == video_format)
	{
		return;
	}
	CurrentColorimetry = MakeShareable(new FVideoDecoderColorimetry(colour_primaries, transfer_characteristics, matrix_coeffs, video_full_range_flag, video_format));
}

void FColorimetryHelper::Update(const TArray<uint8>& InFromCOLRBox)
{
	MP4Utilities::FMP4AtomReader boxReader(InFromCOLRBox);
	uint32 Type;
	if (!boxReader.Read(Type))
	{
		return;
	}
	if (Type != MP4Utilities::MakeBoxAtom('n','c','l','x') && Type != MP4Utilities::MakeBoxAtom('n', 'c', 'l', 'c'))
	{
		return;
	}
	uint16 colour_primaries, transfer_characteristics, matrix_coeffs;
	uint8 video_full_range_flag = 0;
	if (!boxReader.Read(colour_primaries) || !boxReader.Read(transfer_characteristics) || !boxReader.Read(matrix_coeffs))
	{
		return;
	}
	if (Type == MP4Utilities::MakeBoxAtom('n', 'c', 'l', 'x') && !boxReader.Read(video_full_range_flag))
	{
		return;
	}
	Update((uint8) colour_primaries, (uint8)transfer_characteristics, (uint8)matrix_coeffs, (uint8)video_full_range_flag, 5 /*Unspecified video format*/);
}

void FColorimetryHelper::UpdateParamDict(FParamDict& InOutDictionary)
{
	InOutDictionary.Set(IDecoderOutputOptionNames::Colorimetry, FVariantValue(CurrentColorimetry));
}

bool FColorimetryHelper::GetCurrentValues(uint8& colour_primaries, uint8& transfer_characteristics, uint8& matrix_coeffs) const
{
	if (CurrentColorimetry.IsValid() && CurrentColorimetry->GetMPEGDefinition())
	{
		colour_primaries = CurrentColorimetry->GetMPEGDefinition()->ColourPrimaries;
		transfer_characteristics = CurrentColorimetry->GetMPEGDefinition()->TransferCharacteristics;
		matrix_coeffs = CurrentColorimetry->GetMPEGDefinition()->MatrixCoefficients;
		return true;
	}
	else
	{
		colour_primaries = transfer_characteristics = matrix_coeffs = 2;
		return false;
	}
}





void FHDRHelper::Reset()
{
	CurrentHDRInfo.Reset();
	ActiveMasteringDisplayColourVolume.Reset();
	ActiveContentLightLevelInfo.Reset();
	ActiveAlternativeTransferCharacteristics.Reset();
	CurrentAlternativeTransferCharacteristics = -1;
	bIsFirst = true;
}

void FHDRHelper::SetHDRType(int32 BitDepth, const FColorimetryHelper& InColorimetry)
{
	// Set HDR type from colorimetry!
	IVideoDecoderHDRInformation::EType hdrType = IVideoDecoderHDRInformation::EType::Unknown;
	uint8 colour_primaries, transfer_characteristics, matrix_coeffs;
	InColorimetry.GetCurrentValues(colour_primaries, transfer_characteristics, matrix_coeffs);
	if (colour_primaries == 9 && matrix_coeffs == 9)
	{
		if (BitDepth == 10 && transfer_characteristics == 16)
		{
			hdrType = IVideoDecoderHDRInformation::EType::PQ10;
			if (CurrentHDRInfo.IsValid() && CurrentHDRInfo->GetMasteringDisplayColourVolume() && CurrentHDRInfo->GetContentLightLevelInfo())
			{
				hdrType = IVideoDecoderHDRInformation::EType::HDR10;
			}
		}
		else if (BitDepth >= 10 && (transfer_characteristics == 18 || (transfer_characteristics == 14 && CurrentAlternativeTransferCharacteristics == 18)))
		{
			hdrType = IVideoDecoderHDRInformation::EType::HLG10;
		}
	}
	if (CurrentHDRInfo.IsValid())
	{
		CurrentHDRInfo->SetHDRType(hdrType);
	}
}


void FHDRHelper::Update(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& InGlobalPrefixSEIs, const TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& InLocalPrefixSEIs, bool bIsNewCLVS)
{
	auto UseSEI = [](TOptional<ElectraDecodersUtil::MPEG::FSEIMessage>& InOutWhere, const TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& InSeis, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType InWhich) -> bool
	{
		bool bUpdated = false;
		for(auto &sei : InSeis)
		{
			if (sei.PayloadType == InWhich)
			{
				bUpdated = !InOutWhere.IsSet() || InOutWhere.GetValue().Message != sei.Message;
				if (bUpdated)
				{
					InOutWhere = sei;
				}
				break;
			}
		}
		return bUpdated;
	};

	bool bChanged = false;
	if (bIsFirst)
	{
		bChanged |= UseSEI(ActiveMasteringDisplayColourVolume, InGlobalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_mastering_display_colour_volume);
		bChanged |= UseSEI(ActiveContentLightLevelInfo, InGlobalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_content_light_level_info);
		bChanged |= UseSEI(ActiveAlternativeTransferCharacteristics, InGlobalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_alternative_transfer_characteristics);
	}
	bChanged |= UseSEI(ActiveMasteringDisplayColourVolume, InLocalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_mastering_display_colour_volume);
	bChanged |= UseSEI(ActiveContentLightLevelInfo, InLocalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_content_light_level_info);
	bChanged |= UseSEI(ActiveAlternativeTransferCharacteristics, InLocalPrefixSEIs, ElectraDecodersUtil::MPEG::FSEIMessage::EPayloadType::PT_alternative_transfer_characteristics);

	if (bChanged)
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);

		// Mastering Display Colour Volume
		if (ActiveMasteringDisplayColourVolume.IsSet())
		{
			ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume seim;
			if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(seim, ActiveMasteringDisplayColourVolume.GetValue()))
			{
				const float ST2086_Chroma_Scale = 50000.0f;
				const float ST2086_Luma_Scale = 10000.0f;
				FVideoDecoderHDRMetadata_mastering_display_colour_volume mdcv;
				// The order in the HEVC SEI messages is G,B,R
				mdcv.display_primaries_x[0] = seim.display_primaries_x[2] / ST2086_Chroma_Scale;
				mdcv.display_primaries_y[0] = seim.display_primaries_y[2] / ST2086_Chroma_Scale;
				mdcv.display_primaries_x[1] = seim.display_primaries_x[0] / ST2086_Chroma_Scale;
				mdcv.display_primaries_y[1] = seim.display_primaries_y[0] / ST2086_Chroma_Scale;
				mdcv.display_primaries_x[2] = seim.display_primaries_x[1] / ST2086_Chroma_Scale;
				mdcv.display_primaries_y[2] = seim.display_primaries_y[1] / ST2086_Chroma_Scale;
				mdcv.white_point_x = seim.white_point_x / ST2086_Chroma_Scale;
				mdcv.white_point_y = seim.white_point_y / ST2086_Chroma_Scale;
				mdcv.max_display_mastering_luminance = seim.max_display_mastering_luminance / ST2086_Luma_Scale;
				mdcv.min_display_mastering_luminance = seim.min_display_mastering_luminance / ST2086_Luma_Scale;
				CurrentHDRInfo->SetMasteringDisplayColourVolume(mdcv);
			}
		}

		// Content Light Level Info
		if (ActiveContentLightLevelInfo.IsSet())
		{
			ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info seim;
			if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(seim, ActiveContentLightLevelInfo.GetValue()))
			{
				FVideoDecoderHDRMetadata_content_light_level_info ll;
				ll.max_content_light_level = seim.max_content_light_level;
				ll.max_pic_average_light_level = seim.max_pic_average_light_level;
				CurrentHDRInfo->SetContentLightLevelInfo(ll);
			}
		}

		// Alternative Transfer Characteristics
		if (ActiveAlternativeTransferCharacteristics.IsSet())
		{
			ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics seim;
			if (ElectraDecodersUtil::MPEG::ParseFromSEIMessage(seim, ActiveAlternativeTransferCharacteristics.GetValue()))
			{
				CurrentAlternativeTransferCharacteristics = seim.preferred_transfer_characteristics;
			}
		}
	}

	SetHDRType(BitDepth, InColorimetry);
	bIsFirst = false;
}

void FHDRHelper::UpdateWith(const ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume& InSEI)
{
	if (!CurrentHDRInfo.IsValid())
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);
	}
	const float ST2086_Chroma_Scale = 50000.0f;
	const float ST2086_Luma_Scale = 10000.0f;
	FVideoDecoderHDRMetadata_mastering_display_colour_volume mdcv;
	// The order in the HEVC SEI messages is G,B,R
	mdcv.display_primaries_x[0] = InSEI.display_primaries_x[2] / ST2086_Chroma_Scale;
	mdcv.display_primaries_y[0] = InSEI.display_primaries_y[2] / ST2086_Chroma_Scale;
	mdcv.display_primaries_x[1] = InSEI.display_primaries_x[0] / ST2086_Chroma_Scale;
	mdcv.display_primaries_y[1] = InSEI.display_primaries_y[0] / ST2086_Chroma_Scale;
	mdcv.display_primaries_x[2] = InSEI.display_primaries_x[1] / ST2086_Chroma_Scale;
	mdcv.display_primaries_y[2] = InSEI.display_primaries_y[1] / ST2086_Chroma_Scale;
	mdcv.white_point_x = InSEI.white_point_x / ST2086_Chroma_Scale;
	mdcv.white_point_y = InSEI.white_point_y / ST2086_Chroma_Scale;
	mdcv.max_display_mastering_luminance = InSEI.max_display_mastering_luminance / ST2086_Luma_Scale;
	mdcv.min_display_mastering_luminance = InSEI.min_display_mastering_luminance / ST2086_Luma_Scale;
	CurrentHDRInfo->SetMasteringDisplayColourVolume(mdcv);
}

void FHDRHelper::UpdateWith(const ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info& InSEI)
{
	if (!CurrentHDRInfo.IsValid())
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);
	}
	FVideoDecoderHDRMetadata_content_light_level_info ll;
	ll.max_content_light_level = InSEI.max_content_light_level;
	ll.max_pic_average_light_level = InSEI.max_pic_average_light_level;
	CurrentHDRInfo->SetContentLightLevelInfo(ll);
}

void FHDRHelper::UpdateWith(const ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics& InSEI)
{
	if (!CurrentHDRInfo.IsValid())
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);
	}
	CurrentAlternativeTransferCharacteristics = InSEI.preferred_transfer_characteristics;
}


void FHDRHelper::UpdateFromMPEGBoxes(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TArray<uint8>& InMDCVBox, const TArray<uint8>& InCLLIBox)
{
	if (!CurrentHDRInfo.IsValid() && (InMDCVBox.Num() || InCLLIBox.Num()))
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);
	}

	if (InMDCVBox.Num())
	{
		MP4Utilities::FMP4AtomReader boxReader(InMDCVBox);

		struct mastering_display_colour_volume
		{
			uint16 display_primaries_x[3]{ 0 };
			uint16 display_primaries_y[3]{ 0 };
			uint16 white_point_x = 0;
			uint16 white_point_y = 0;
			uint32 max_display_mastering_luminance = 0;
			uint32 min_display_mastering_luminance = 0;
		};
		mastering_display_colour_volume boxData;
		for(int32 i=0; i<3; ++i)
		{
			if (!boxReader.Read(boxData.display_primaries_x[i]) || !boxReader.Read(boxData.display_primaries_y[i]))
			{
				return;
			}
		}
		if (!boxReader.Read(boxData.white_point_x) || !boxReader.Read(boxData.white_point_y))
		{
			return;
		}
		if (!boxReader.Read(boxData.max_display_mastering_luminance) || !boxReader.Read(boxData.min_display_mastering_luminance))
		{
			return;
		}
		const float ST2086_Chroma_Scale = 50000.0f;
		const float ST2086_Luma_Scale = 10000.0f;
		FVideoDecoderHDRMetadata_mastering_display_colour_volume mdcv;
		// The order in the MDCV box is G,B,R
		mdcv.display_primaries_x[0] = boxData.display_primaries_x[2] / ST2086_Chroma_Scale;
		mdcv.display_primaries_y[0] = boxData.display_primaries_y[2] / ST2086_Chroma_Scale;
		mdcv.display_primaries_x[1] = boxData.display_primaries_x[0] / ST2086_Chroma_Scale;
		mdcv.display_primaries_y[1] = boxData.display_primaries_y[0] / ST2086_Chroma_Scale;
		mdcv.display_primaries_x[2] = boxData.display_primaries_x[1] / ST2086_Chroma_Scale;
		mdcv.display_primaries_y[2] = boxData.display_primaries_y[1] / ST2086_Chroma_Scale;
		mdcv.white_point_x = boxData.white_point_x / ST2086_Chroma_Scale;
		mdcv.white_point_y = boxData.white_point_y / ST2086_Chroma_Scale;
		mdcv.max_display_mastering_luminance = boxData.max_display_mastering_luminance / ST2086_Luma_Scale;
		mdcv.min_display_mastering_luminance = boxData.min_display_mastering_luminance / ST2086_Luma_Scale;
		CurrentHDRInfo->SetMasteringDisplayColourVolume(mdcv);
	}

	if (InCLLIBox.Num())
	{
		MP4Utilities::FMP4AtomReader boxReader(InCLLIBox);

		uint16 max_content_light_level = 0;			// MaxCLL
		uint16 max_pic_average_light_level = 0;		// MaxFALL
		if (!boxReader.Read(max_content_light_level) || !boxReader.Read(max_pic_average_light_level))
		{
			return;
		}
		FVideoDecoderHDRMetadata_content_light_level_info ll;
		ll.max_content_light_level = max_content_light_level;
		ll.max_pic_average_light_level = max_pic_average_light_level;
		CurrentHDRInfo->SetContentLightLevelInfo(ll);
	}

	SetHDRType(BitDepth, InColorimetry);
}

void FHDRHelper::Update(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TOptional<FVideoDecoderHDRMetadata_mastering_display_colour_volume>& InMDCV, const TOptional<FVideoDecoderHDRMetadata_content_light_level_info>& InCLLI)
{
	if (!InMDCV.IsSet() && !InCLLI.IsSet())
	{
		return;
	}
	if (!CurrentHDRInfo.IsValid())
	{
		CurrentHDRInfo = MakeShareable(new FVideoDecoderHDRInformation);
	}
	if (InMDCV.IsSet())
	{
		CurrentHDRInfo->SetMasteringDisplayColourVolume(InMDCV.GetValue());
	}
	if (InCLLI.IsSet())
	{
		CurrentHDRInfo->SetContentLightLevelInfo(InCLLI.GetValue());
	}
	SetHDRType(BitDepth, InColorimetry);
}




void FHDRHelper::UpdateParamDict(FParamDict& InOutDictionary)
{
	InOutDictionary.Set(IDecoderOutputOptionNames::HDRInfo, FVariantValue(CurrentHDRInfo));
}


} // namespace MPEG
} // namespace Electra

