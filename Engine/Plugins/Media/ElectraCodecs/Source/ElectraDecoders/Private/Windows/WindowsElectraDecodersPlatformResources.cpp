// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsElectraDecodersPlatformResources.h"
#include "ElectraDecodersModule.h"
#include "WindowsPlatformHeaders_Video_DX.h"
#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"
#include "Async/Async.h"

#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MediaDecoderOutput.h"
#include "MediaVideoDecoderOutput.h"
#include "ElectraTextureSample.h"

#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)


DECLARE_CYCLE_STAT(TEXT("Electra decoder AsyncJob"), STAT_ElectraDecoderAsyncJob, STATGROUP_Media);


namespace PlatformElectraDecodersWindows
{
	static FCriticalSection AsyncJobAccessCS;
	static FGraphEventRef RunCodeAsyncEvent;
}


struct FElectraDecodersPlatformResourcesWindows::IDecoderPlatformResource
{
	void* D3DDevice = nullptr;
	int32 D3DVersionTimes1000 = 0;
	uint32 MaxWidth = 0;
	uint32 MaxHeight = 0;
	uint32 MaxOutputBuffers = 0;

	TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
};

FElectraDecodersPlatformResourcesWindows::IDecoderPlatformResource* FElectraDecodersPlatformResourcesWindows::CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions)
{
	IDecoderPlatformResource* pr = new IDecoderPlatformResource;
	GetD3DDeviceAndVersion(&pr->D3DDevice, &pr->D3DVersionTimes1000);
	pr->MaxWidth  = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_width"), 1920), 16);
	pr->MaxHeight = Align((uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_height"), 1080), 16);
	pr->MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InOptions, TEXT("max_output_buffers"), 5);
	return pr;
}

void FElectraDecodersPlatformResourcesWindows::ReleasePlatformVideoResource(void* InOwnerHandle, FElectraDecodersPlatformResourcesWindows::IDecoderPlatformResource* InHandleToRelease)
{
	if (InHandleToRelease)
	{
		delete InHandleToRelease;
	}
}



bool FElectraDecodersPlatformResourcesWindows::GetD3DDeviceAndVersion(void** OutD3DDevice, int32* OutD3DVersion)
{
	if (!GDynamicRHI || !OutD3DDevice || !OutD3DVersion)
	{
		return false;
	}

	switch (RHIGetInterfaceType())
	{
	case ERHIInterfaceType::D3D11:
		*OutD3DDevice = GetID3D11DynamicRHI()->RHIGetDevice();
		*OutD3DVersion = 11000;
		return true;

	case ERHIInterfaceType::D3D12:
		*OutD3DDevice = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
		*OutD3DVersion = 12000;
		return true;

	default:
		return false;
	}
}

static void TriggerDataCopy(TRefCountPtr<ID3D12Resource> InDstResource, TRefCountPtr<ID3D12Resource> InSrcResource, TRefCountPtr<ID3D12GraphicsCommandList> D3DCmdList, TRefCountPtr<ID3D12CommandAllocator> D3DCmdAllocator, TRefCountPtr<ID3D12Fence> D3DFence, uint64 FenceValue, const TSharedPtr<std::atomic<bool>>& FenceSignalled, const FElectraDecoderOutputSync& OutputSync)
{
	// Trigger copy (this will eventually execute on the submission thread of RHI if running in UE)
	// (note: we pass in all of FElectraDecoderOutputSync to guarantee any references needed to make the decoder output sync work are passed along, too!)
	GetID3D12PlatformDynamicRHI()->RHIRunOnQueue(ED3D12RHIRunOnQueueType::Copy,
		[DstResource = InDstResource, SrcResource = InSrcResource, CmdList = D3DCmdList, CmdAllocator = D3DCmdAllocator, DestFence = D3DFence, DestFenceValue = FenceValue, DestFenceSignalled = FenceSignalled, OutputSync](ID3D12CommandQueue* D3DCmdQueue)
		{
			TRefCountPtr<ID3D12Fence> ResourceFence;
	#if ELECTRA_DECODERS_ENABLE_DX && ALLOW_MFSAMPLE_WITH_DX12
			TRefCountPtr<IMFD3D12SynchronizationObjectCommands> DecoderSync;
	#endif

			if (OutputSync.Sync.IsValid())
			{
	#if ELECTRA_DECODERS_ENABLE_DX && ALLOW_MFSAMPLE_WITH_DX12
				if (OutputSync.Sync->QueryInterface(__uuidof(IMFD3D12SynchronizationObjectCommands), (void**)DecoderSync.GetInitReference()) != S_OK)
	#endif
				{
					OutputSync.Sync->QueryInterface(__uuidof(ID3D12Fence), (void**)ResourceFence.GetInitReference());
				}
			}

	#if ELECTRA_DECODERS_ENABLE_DX && ALLOW_MFSAMPLE_WITH_DX12
			if (DecoderSync)
			{
				// Sync queue to make sure decoder output is ready for us (WMF case)
				DecoderSync->EnqueueResourceReadyWait(D3DCmdQueue);
			}
	#endif
			if (ResourceFence)
			{
				// Sync queue to make sure decoder output is ready for us
				D3DCmdQueue->Wait(ResourceFence, OutputSync.SyncValue);
			}

			// Execute copy
			ID3D12CommandList* CmdLists[1] = { CmdList.GetReference() };
			D3DCmdQueue->ExecuteCommandLists(1, CmdLists);

	#if ELECTRA_DECODERS_ENABLE_DX && ALLOW_MFSAMPLE_WITH_DX12
			if (DecoderSync)
			{
				// Sync to end of copy operation and release MFSample once reached
				DecoderSync->EnqueueResourceRelease(D3DCmdQueue);
			}
	#endif
			// Trigger optional notification back to the decoder, so it could reuse its buffer after the copy
			if (OutputSync.CopyDoneSync)
			{
				D3DCmdQueue->Signal(OutputSync.CopyDoneSync, OutputSync.CopyDoneSyncValue);
			}

			// Notify user of the output data we just copied of its arrival
			D3DCmdQueue->Signal(DestFence, DestFenceValue);
			checkf(!*DestFenceSignalled, TEXT("Electra texture sample copy fence should not have been signalled yet at this point"));

			*DestFenceSignalled = true;

		}, false);
}


static bool SetupOutputTextureSampleFromDX12Resource(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup,
													 TRefCountPtr<ID3D12Device>& InD3D12Device, TRefCountPtr<ID3D12Resource> InResource, uint32 ResourcePitch,
													 const FElectraDecoderOutputSync& OutputSync, const FIntPoint& InSampleDim, const Electra::FParamDict& InParamDict,
													 TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>& InOutD3D12ResourcePool,
													 uint32 MaxWidth, uint32 MaxHeight, uint32 MaxOutputBuffers)
{
	HRESULT Result;
	FString ResultMsg;

	// Check that the sample is a fresh or a cleanly released one.
	check(!InOutTextureSampleToSetup->D3DFence.IsValid());

	// Source is a DX12 resource
	InOutTextureSampleToSetup->SourceType = FElectraTextureSample::ESourceType::ResourceDX12;
	InOutTextureSampleToSetup->ClearDX11Vars();
	InOutTextureSampleToSetup->ClearBufferVars();

	// General initialization.
	InOutTextureSampleToSetup->InitializeCommon(InParamDict);

	// Hold a reference to the resource passed in as we need to ensure it is alive until we have copied from it.
	InOutTextureSampleToSetup->DecoderOutputResource = InResource;
	// Remember the dimension of the source sample, which may be different from the display size depending on pixel format.
	InOutTextureSampleToSetup->SampleDim = InSampleDim;

	if (!InOutTextureSampleToSetup->D3D12ResourcePool.IsValid() && InOutD3D12ResourcePool.IsValid())
	{
		InOutTextureSampleToSetup->D3D12ResourcePool = InOutD3D12ResourcePool;
	}

	// Create D3D12 resources as needed (we will reuse if possible as this instance is reused)
	if (!InOutTextureSampleToSetup->D3DCmdAllocator.IsValid())
	{
		Result = InD3D12Device->CreateCommandAllocator(D3D12_COMMAND_LIST_TYPE_COPY, IID_PPV_ARGS(InOutTextureSampleToSetup->D3DCmdAllocator.GetInitReference()));
		if (FAILED(Result))
		{
			OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): CreateCommandAllocator() failed with 0x%08x"), Result);
			return false;
		}
		#if !UE_BUILD_SHIPPING
			InOutTextureSampleToSetup->D3DCmdAllocator->SetName(TEXT("ElectraTextureSampleCmdAllocator"));
		#endif
	}
	if (!InOutTextureSampleToSetup->D3DCmdList.IsValid())
	{
		Result = InD3D12Device->CreateCommandList(0, D3D12_COMMAND_LIST_TYPE_COPY, InOutTextureSampleToSetup->D3DCmdAllocator.GetReference(), nullptr, __uuidof(ID3D12CommandList), reinterpret_cast<void**>(InOutTextureSampleToSetup->D3DCmdList.GetInitReference()));
		if (FAILED(Result))
		{
			OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): CreateCommandList() failed with 0x%08x"), Result);
			return false;
		}
		InOutTextureSampleToSetup->D3DCmdList->Close();
		#if !UE_BUILD_SHIPPING
			InOutTextureSampleToSetup->D3DCmdList->SetName(TEXT("ElectraTextureSampleCmdList"));
		#endif
	}

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
	// Create texture to copy decoder output buffer into
	D3D12_RESOURCE_DESC ResourceDesc;
	ResourceDesc = InResource->GetDesc();

	// Find out if the image data is sRGB encoded (we also assume this by default)
	bool bOutputIsSRGB = true;
	if (auto Colorimetry = InOutTextureSampleToSetup->GetColorimetry())
	{
		if (auto MPEGDef = Colorimetry->GetMPEGDefinition())
		{
			bOutputIsSRGB = (MPEGDef->TransferCharacteristics == 2);
		}
	}
	const bool bIsTexture = ResourceDesc.Dimension == D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	const FIntPoint TextureDim = bIsTexture ? FIntPoint{ static_cast<int>(MaxWidth), static_cast<int>(MaxHeight) } : InSampleDim;
	const EElectraTextureSamplePixelEncoding FmtEnc = InOutTextureSampleToSetup->GetPixelFormatEncoding();
	// Check if we can use HW sRGB decode (we must avoid doing this if the data represents something else as true RGB(A) values)
	const bool bIsSRGB = bOutputIsSRGB && (FmtEnc == EElectraTextureSamplePixelEncoding::Native || FmtEnc == EElectraTextureSamplePixelEncoding::RGB || FmtEnc == EElectraTextureSamplePixelEncoding::RGBA);

	uint32 BlockSizeX = 1;
	uint32 BlockSizeY = 1;
	DXGI_FORMAT DXGIFmt = DXGI_FORMAT_UNKNOWN;
	const EPixelFormat PixelFormat = InOutTextureSampleToSetup->GetPixelFormat();
	switch(PixelFormat)
	{
		case PF_NV12:
		{
			DXGIFmt = DXGI_FORMAT_NV12;
			break;
		}
		case PF_P010:
		{
			DXGIFmt = DXGI_FORMAT_P010;
			break;
		}
		case PF_A16B16G16R16:
		{
			DXGIFmt = DXGI_FORMAT_R16G16B16A16_UNORM;
			break;
		}
		case PF_R16G16B16A16_UNORM:
		{
			DXGIFmt = DXGI_FORMAT_R16G16B16A16_UNORM;
			break;
		}
		case PF_A32B32G32R32F:
		{
			DXGIFmt = DXGI_FORMAT_R32G32B32A32_FLOAT;
			break;
		}
		case PF_B8G8R8A8:
		{
			DXGIFmt = bIsSRGB ? DXGI_FORMAT_B8G8R8A8_UNORM_SRGB : DXGI_FORMAT_B8G8R8A8_TYPELESS;
			break;
		}
		case PF_R8G8B8A8:
		{
			DXGIFmt = bIsSRGB ? DXGI_FORMAT_R8G8B8A8_UNORM_SRGB : DXGI_FORMAT_R8G8B8A8_TYPELESS;
			break;
		}
		case PF_A2B10G10R10:
		{
			DXGIFmt = DXGI_FORMAT_R10G10B10A2_UNORM;
			break;
		}
		case PF_DXT1:
		{
			DXGIFmt = bIsSRGB ? DXGI_FORMAT_BC1_UNORM_SRGB : DXGI_FORMAT_BC1_UNORM;
			BlockSizeX = BlockSizeY = 4;
			break;
		}
		case PF_DXT5:
		{
			DXGIFmt = bIsSRGB ? DXGI_FORMAT_BC3_UNORM_SRGB : DXGI_FORMAT_BC3_UNORM;
			BlockSizeX = BlockSizeY = 4;
			break;
		}
		case PF_BC4:
		{
			DXGIFmt = DXGI_FORMAT_BC4_UNORM;
			BlockSizeX = BlockSizeY = 4;
			break;
		}
		default:
		{
			OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): Unexpected pixel format"));
			return false;
		}
	}

	/*
		- We need to copy this from the output buffers ASAP to not throttle buffer availability (and hence performance) for the decoder
		- As RHI limits resource creation to the renderthread, which would delay creation, we need to use a platform API level texture we can create right here and now
	*/
	bool bIsCompatibleAsTexture = InOutTextureSampleToSetup->D3D12ResourcePool.IsValid() && InOutTextureSampleToSetup->D3D12ResourcePool->IsCompatibleAsTexture(MaxOutputBuffers + kElectraDecoderPipelineExtraFrames, MaxWidth, MaxHeight, DXGIFmt);
	if (!InOutTextureSampleToSetup->TextureDX12.IsValid() || TextureDim != InOutTextureSampleToSetup->TextureDX12Dim || !bIsCompatibleAsTexture)
	{
		// We get here only after the instance came back from the pool! Hence we can be sure any old texture is no longer actively used.
		InOutTextureSampleToSetup->TextureDX12 = nullptr;

		bool bPoolCompatible = InOutD3D12ResourcePool.IsValid() && InOutD3D12ResourcePool->IsCompatibleAsTexture(MaxOutputBuffers + kElectraDecoderPipelineExtraFrames, MaxWidth, MaxHeight, DXGIFmt);
		if (!InOutTextureSampleToSetup->D3D12ResourcePool.IsValid() || !bPoolCompatible)
		{
			InOutD3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(InD3D12Device, MaxOutputBuffers + kElectraDecoderPipelineExtraFrames, MaxWidth, MaxHeight, DXGIFmt, D3D12_HEAP_TYPE_DEFAULT);
			if (!InOutD3D12ResourcePool.IsValid())
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): Could not allocate texture resource heap"));
				return false;
			}
		}
		InOutTextureSampleToSetup->D3D12ResourcePool = InOutD3D12ResourcePool;

		FElectraMediaDecoderOutputBufferPool_DX12::FOutputData OutputData;
		if (!InOutTextureSampleToSetup->D3D12ResourcePool->AllocateOutputDataAsTexture(Result, ResultMsg, OutputData, TextureDim.X, TextureDim.Y, DXGIFmt))
		{
			OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): Could not allocate texture resource from heap (%x, %s)"), Result, *ResultMsg);
			return false;
		}

		InOutTextureSampleToSetup->TextureDX12 = OutputData.Resource;
		#if !UE_BUILD_SHIPPING
			InOutTextureSampleToSetup->TextureDX12->SetName(TEXT("ElectraTextureSampleOutputFrame"));
		#endif
		InOutTextureSampleToSetup->D3DFence = OutputData.Fence;
		InOutTextureSampleToSetup->FenceValue = OutputData.FenceValue;

		InOutTextureSampleToSetup->TextureDX12Dim = TextureDim;
	}
	else
	{
		// We reuse the texture, but we need to update the fence info...
		InOutTextureSampleToSetup->D3DFence = InOutTextureSampleToSetup->D3D12ResourcePool->GetUpdatedBufferFence(InOutTextureSampleToSetup->FenceValue);
	}

	InOutTextureSampleToSetup->FenceSignalled = MakeShared<std::atomic<bool>>(false);

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
	// Build command list to copy data from decoder buffer

	InOutTextureSampleToSetup->D3DCmdAllocator->Reset();
	InOutTextureSampleToSetup->D3DCmdList->Reset(InOutTextureSampleToSetup->D3DCmdAllocator, nullptr);

	if (bIsTexture)
	{
		// If we got a texture, we simply can copy the resource as a whole...
		InOutTextureSampleToSetup->D3DCmdList->CopyResource(InOutTextureSampleToSetup->TextureDX12, InResource);
	}
	else
	{
		// Source is a buffer. Build a properly configured footprint and copy from there...
		if (PixelFormat == PF_NV12 || PixelFormat == PF_P010)
		{
			//
			// Formats with 2 planes
			//

			D3D12_BOX SrcBox;
			SrcBox.left = 0;
			SrcBox.top = 0;
			SrcBox.right = InSampleDim.X;
			SrcBox.bottom = InSampleDim.Y;
			SrcBox.back = 1;
			SrcBox.front = 0;

			// Copy Y plane
			D3D12_PLACED_SUBRESOURCE_FOOTPRINT SrcFootPrint{};
			SrcFootPrint.Offset = 0;
			SrcFootPrint.Footprint.Format = (PixelFormat == PF_NV12) ? DXGI_FORMAT_R8_UNORM : DXGI_FORMAT_R16_UNORM;
			SrcFootPrint.Footprint.Width = InSampleDim.X;
			SrcFootPrint.Footprint.Height = InSampleDim.Y;
			SrcFootPrint.Footprint.Depth = 1;
			SrcFootPrint.Footprint.RowPitch = ResourcePitch;
			CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(InResource, SrcFootPrint);
			CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(InOutTextureSampleToSetup->TextureDX12, 0);
			InOutTextureSampleToSetup->D3DCmdList->CopyTextureRegion(&DestCopyLocation, 0, 0, 0, &SourceCopyLocation, &SrcBox);

			// Copy CbCr plane
			SrcBox.right = InSampleDim.X >> 1;
			SrcBox.bottom = InSampleDim.Y >> 1;
			SourceCopyLocation.PlacedFootprint.Offset = ResourcePitch * InSampleDim.Y;
			SourceCopyLocation.PlacedFootprint.Footprint.Format = (PixelFormat == PF_NV12) ? DXGI_FORMAT_R8G8_UNORM : DXGI_FORMAT_R16G16_UNORM;
			SourceCopyLocation.PlacedFootprint.Footprint.Width = InSampleDim.X >> 1;
			SourceCopyLocation.PlacedFootprint.Footprint.Height = InSampleDim.Y >> 1;
			DestCopyLocation.SubresourceIndex = 1;
			InOutTextureSampleToSetup->D3DCmdList->CopyTextureRegion(&DestCopyLocation, 0, 0, 0, &SourceCopyLocation, &SrcBox);
		}
		else
		{
			//
			// Single plane formats
			//

			D3D12_BOX SrcBox;
			SrcBox.left = 0;
			SrcBox.top = 0;
			SrcBox.right = InSampleDim.X;
			SrcBox.bottom = InSampleDim.Y;
			SrcBox.back = 1;
			SrcBox.front = 0;

			D3D12_PLACED_SUBRESOURCE_FOOTPRINT SrcFootPrint{};
			SrcFootPrint.Offset = 0;
			SrcFootPrint.Footprint.Format = DXGIFmt;
			SrcFootPrint.Footprint.Width = Align(InSampleDim.X, BlockSizeX);
			SrcFootPrint.Footprint.Height = Align(InSampleDim.Y, BlockSizeY);
			SrcFootPrint.Footprint.Depth = 1;
			SrcFootPrint.Footprint.RowPitch = ResourcePitch;
			CD3DX12_TEXTURE_COPY_LOCATION SourceCopyLocation(InResource, SrcFootPrint);
			CD3DX12_TEXTURE_COPY_LOCATION DestCopyLocation(InOutTextureSampleToSetup->TextureDX12, 0);
			InOutTextureSampleToSetup->D3DCmdList->CopyTextureRegion(&DestCopyLocation, 0, 0, 0, &SourceCopyLocation, &SrcBox);
		}
	}

	InOutTextureSampleToSetup->D3DCmdList->Close();

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -
	// Schedule the copy right away. Any needed synchronization will be done in the copy-queue by the GPU
	TriggerDataCopy(InOutTextureSampleToSetup->TextureDX12, InResource, InOutTextureSampleToSetup->D3DCmdList, InOutTextureSampleToSetup->D3DCmdAllocator, InOutTextureSampleToSetup->D3DFence, InOutTextureSampleToSetup->FenceValue, InOutTextureSampleToSetup->FenceSignalled, OutputSync);

	// Finish initialization.
	if (!InOutTextureSampleToSetup->FinishInitialization())
	{
		OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromDX12Resource(): Unsupported pixel format encoding"));
		return false;
	}
	return true;
}





static bool SetupOutputTextureSampleFromSharedTexture(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup,
													  TRefCountPtr<ID3D11Device> InD3D11Device, TRefCountPtr<ID3D11DeviceContext> InDeviceContext,
													  TRefCountPtr<ID3D11Texture2D> InDecoderTexture, uint32 InViewIndex,
													  const FIntPoint& InOutputDim, const Electra::FParamDict& InParamDict)
{
	// Source is a shared DX11 texture
	InOutTextureSampleToSetup->SourceType = FElectraTextureSample::ESourceType::SharedTextureDX11;
	InOutTextureSampleToSetup->ClearDX12Vars();
	InOutTextureSampleToSetup->ClearBufferVars();

	// General initialization.
	InOutTextureSampleToSetup->InitializeCommon(InParamDict);

	const bool bNeedsNew = !InOutTextureSampleToSetup->TextureDX11.IsValid() || (InOutTextureSampleToSetup->SampleDim.X != InOutputDim.X || InOutTextureSampleToSetup->SampleDim.Y != InOutputDim.Y);
	if (bNeedsNew)
	{
		InOutTextureSampleToSetup->SampleDim = InOutputDim;

		D3D11_TEXTURE2D_DESC DecoderTextureDesc;
		InDecoderTexture->GetDesc(&DecoderTextureDesc);

		// Make a texture we pass on as output
		D3D11_TEXTURE2D_DESC TextureDesc;
		TextureDesc.Width = InOutputDim.X;
		TextureDesc.Height = InOutputDim.Y;
		TextureDesc.MipLevels = 1;
		TextureDesc.ArraySize = 1;
		TextureDesc.Format = DecoderTextureDesc.Format;
		TextureDesc.SampleDesc.Count = 1;
		TextureDesc.SampleDesc.Quality = 0;
		TextureDesc.Usage = D3D11_USAGE_DEFAULT;
		TextureDesc.BindFlags = 0;
		TextureDesc.CPUAccessFlags = 0;
		TextureDesc.MiscFlags = D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX;
		if (FAILED(InD3D11Device->CreateTexture2D(&TextureDesc, nullptr, InOutTextureSampleToSetup->TextureDX11.GetInitReference())))
		{
			OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromSharedTexture(): CreateTexture2D() failed"));
			return false;
		}
		InOutTextureSampleToSetup->D3D11Device = InD3D11Device;
	}

	// If we got a texture, copy the data from the decoder into it...
	if (InOutTextureSampleToSetup->TextureDX11)
	{
		// Source data may be larger than desired output, but crop area will be aligned such that we can always copy from 0,0
		D3D11_BOX SrcBox;
		SrcBox.left = 0;
		SrcBox.top = 0;
		SrcBox.front = 0;
		SrcBox.right = InOutputDim.X;
		SrcBox.bottom = InOutputDim.Y;
		SrcBox.back = 1;

		TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
		HRESULT Result = InOutTextureSampleToSetup->TextureDX11->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
		if (KeyedMutex)
		{
			// No wait on acquire since sample is new and key is 0.
			Result = KeyedMutex->AcquireSync(0, 0);
			if (Result == S_OK)
			{
				// Copy texture using the decoder DX11 device... (and apply any cropping - see above note)
				InDeviceContext->CopySubresourceRegion(InOutTextureSampleToSetup->TextureDX11, 0, 0, 0, 0, InDecoderTexture, InViewIndex, &SrcBox);
				// Mark texture as updated with key of 1
				// Sample will be read in Convert() method of texture sample
				KeyedMutex->ReleaseSync(1);
			}
		}

		// Make sure texture is updated before giving access to the sample in the rendering thread.
		InDeviceContext->Flush();
	}

	// Finish initialization.
	if (!InOutTextureSampleToSetup->FinishInitialization())
	{
		OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromSharedTexture(): Unsupported pixel format encoding"));
		return false;
	}

	return true;
}




static bool SetupOutputTextureSampleFromBuffer(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup,
												TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> InBuffer, uint32 InStride,
												const FIntPoint& InOutputDim, const Electra::FParamDict& InParamDict)
{
	// Source is a buffer
	InOutTextureSampleToSetup->SourceType = FElectraTextureSample::ESourceType::Buffer;
	InOutTextureSampleToSetup->ClearDX11Vars();
	InOutTextureSampleToSetup->ClearDX12Vars();

	// General initialization.
	InOutTextureSampleToSetup->InitializeCommon(InParamDict);

	InOutTextureSampleToSetup->Buffer = MoveTemp(InBuffer);
	InOutTextureSampleToSetup->SetDim(InOutputDim);
	InOutTextureSampleToSetup->SampleDim = InOutputDim;
	InOutTextureSampleToSetup->Stride = InStride;

	// Finish initialization.
	if (!InOutTextureSampleToSetup->FinishInitialization())
	{
		OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSampleFromBuffer(): Unsupported pixel format encoding"));
		return false;
	}
	return true;
}




bool FElectraDecodersPlatformResourcesWindows::SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, FElectraDecodersPlatformResourcesWindows::IDecoderPlatformResource* InPlatformResource)
{
	if (!InOutTextureSampleToSetup.IsValid() || !InDecoderOutput.IsValid() || !InOutBufferPropertes.IsValid() || !InPlatformResource)
	{
		OutErrorMessage = TEXT("Bad parameter");
		return false;
	}

	TMap<FString, FVariant> ExtraValues;
	InDecoderOutput->GetExtraValues(ExtraValues);
	const int32 Width = InDecoderOutput->GetDecodedWidth();
	const int32 Height = InDecoderOutput->GetDecodedHeight();

	FElectraVideoDecoderOutputCropValues Crop = InDecoderOutput->GetCropValues();
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Width, Electra::FVariantValue((int64)InDecoderOutput->GetWidth()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::Height, Electra::FVariantValue((int64)InDecoderOutput->GetHeight()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropLeft, Electra::FVariantValue((int64)Crop.Left));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropRight, Electra::FVariantValue((int64)Crop.Right));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropTop, Electra::FVariantValue((int64)Crop.Top));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::CropBottom, Electra::FVariantValue((int64)Crop.Bottom));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectRatio, Electra::FVariantValue((double)InDecoderOutput->GetAspectRatioW() / (double)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectW, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioW()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::AspectH, Electra::FVariantValue((int64)InDecoderOutput->GetAspectRatioH()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSNumerator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateNumerator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::FPSDenominator, Electra::FVariantValue((int64)InDecoderOutput->GetFrameRateDenominator()));
	InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64)InDecoderOutput->GetNumberOfBits()));

	//
	// Did we get image buffers from the decoder?
	//
	IElectraDecoderVideoOutputImageBuffers* ImageBuffers = reinterpret_cast<IElectraDecoderVideoOutputImageBuffers*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::ImageBuffers));
	if (ImageBuffers != nullptr)
	{

		int32 NumImageBuffers = ImageBuffers->GetNumberOfBuffers();
		check(NumImageBuffers == 1 || NumImageBuffers == 2);

		// Color buffer
		EPixelFormat RHIPixFmt = ImageBuffers->GetBufferFormatByIndex(0);
		check(RHIPixFmt != EPixelFormat::PF_Unknown);
		EElectraTextureSamplePixelEncoding DecPixEnc = ImageBuffers->GetBufferEncodingByIndex(0);

		int32 Pitch = ImageBuffers->GetBufferPitchByIndex(0);
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)RHIPixFmt));
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelEncoding, Electra::FVariantValue((int64)DecPixEnc));
		InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));

		InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelDataScale, Electra::FVariantValue((double)ElectraDecodersUtil::GetVariantValueSafeDouble(ExtraValues, TEXT("pix_datascale"), 1.0)));

		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> ColorBuffer = ImageBuffers->GetBufferDataByIndex(0);
		if (ColorBuffer.IsValid() && ColorBuffer->Num())
		{
			//
			// CPU side buffer
			//
			return SetupOutputTextureSampleFromBuffer(OutErrorMessage, InOutTextureSampleToSetup, ColorBuffer,
						Pitch,							// Buffer stride
						FIntPoint(Width, Height),		// Buffer dimensions
						*InOutBufferPropertes);
		}
		else
		{
			TRefCountPtr<IUnknown> TextureCommon(static_cast<IUnknown*>(ImageBuffers->GetBufferTextureByIndex(0)));
			if (TextureCommon.IsValid())
			{
				HRESULT Result;
				if (InPlatformResource->D3DVersionTimes1000 >= 12000)
				{
					TRefCountPtr<ID3D12Device> D3D12Device(static_cast<ID3D12Device*>(InPlatformResource->D3DDevice));

					// Can we get a DX12 texture? (this will only work if the transform is associated with a DX12 device earlier on; hence no need for a SDK version guard here)
					TRefCountPtr<ID3D12Resource> Resource;
					Result = TextureCommon->QueryInterface(__uuidof(ID3D12Resource), (void**)Resource.GetInitReference());
					if (SUCCEEDED(Result))
					{
						//
						// DX12 texture / buffer
						//

						// We might also have a fence / sync we need to use before accessing the data on the texture...
						FElectraDecoderOutputSync OutputSync;
						ImageBuffers->GetBufferTextureSyncByIndex(0, OutputSync);

						return SetupOutputTextureSampleFromDX12Resource(OutErrorMessage, InOutTextureSampleToSetup,
									D3D12Device, Resource, Pitch, OutputSync, FIntPoint(Width, Height), *InOutBufferPropertes,
									InPlatformResource->D3D12ResourcePool,
									InPlatformResource->MaxWidth, InPlatformResource->MaxHeight, InPlatformResource->MaxOutputBuffers);
					}
				}

				TRefCountPtr<ID3D11Texture2D> TextureD3D11;
				Result = TextureCommon->QueryInterface(__uuidof(ID3D11Texture2D), (void**)TextureD3D11.GetInitReference());
				if (SUCCEEDED(Result))
				{
					//
					// DX11 texture
					//
					ID3D11Device* Device = reinterpret_cast<ID3D11Device*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::DXDevice));
					ID3D11DeviceContext* DeviceContext = reinterpret_cast<ID3D11DeviceContext*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::DXDeviceContext));
					if (Device && DeviceContext)
					{
						const uint32 ViewIndex = 0;
						return SetupOutputTextureSampleFromSharedTexture(OutErrorMessage, InOutTextureSampleToSetup,
									Device, DeviceContext,
									TextureD3D11, ViewIndex,
									FIntPoint(InDecoderOutput->GetWidth(), InDecoderOutput->GetHeight()), *InOutBufferPropertes);
					}
				}
			}
		}
	}
	//
	// IMF sample?
	//
	else if (InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::MFSample))
	{
		// What type of decoder output do we have here?
		int32 DXVersion = (int32)ElectraDecodersUtil::GetVariantValueSafeI64(ExtraValues, TEXT("dxversion"), 0);
		bool bIsSW = !!ElectraDecodersUtil::GetVariantValueSafeI64(ExtraValues, TEXT("sw"), 0);

		HRESULT Result;
		// DX11, DX12 or non-DX & HW accelerated?
		if ((DXVersion == 0 || DXVersion >= 11000) && !bIsSW)
		{
			TRefCountPtr<IMFSample> DecodedOutputSample = reinterpret_cast<IMFSample*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::MFSample));

			DWORD BuffersNum = 0;
			if ((Result = DecodedOutputSample->GetBufferCount(&BuffersNum)) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferCount() failed with %08x"), Result);
				return false;
			}
			if (BuffersNum != 1)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferCount() returned %u buffers instead of 1"), BuffersNum);
				return false;
			}
			TRefCountPtr<IMFMediaBuffer> Buffer;
			if ((Result = DecodedOutputSample->GetBufferByIndex(0, Buffer.GetInitReference())) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferByIndex() failed with %08x"), Result);
				return false;
			}
			TRefCountPtr<IMFDXGIBuffer> DXGIBuffer;
			if ((Result = Buffer->QueryInterface(__uuidof(IMFDXGIBuffer), (void**)DXGIBuffer.GetInitReference())) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): QueryInterface(IMFDXGIBuffer) failed with %08x"), Result);
				return false;
			}
			TRefCountPtr<ID3D11Texture2D> Texture2D;
			if ((Result = DXGIBuffer->GetResource(IID_PPV_ARGS(Texture2D.GetInitReference()))) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetResource(ID3D11Texture2D) failed with %08x"), Result);
				return false;
			}
			D3D11_TEXTURE2D_DESC TextureDesc;
			Texture2D->GetDesc(&TextureDesc);
			if (TextureDesc.Format != DXGI_FORMAT_NV12 && TextureDesc.Format != DXGI_FORMAT_P010)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetDesc() did not return NV12 or P010 format as expected"));
				return false;
			}

			InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)(TextureDesc.Format == DXGI_FORMAT_NV12 ? EPixelFormat::PF_NV12 : EPixelFormat::PF_P010)));
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64)(TextureDesc.Format == DXGI_FORMAT_NV12 ? 8 : 10)));

			if (DXVersion == 0 || DXVersion >= 12000)
			{
				//
				// DX12 (with DX11 decode device) & non-DX
				//
				// (access buffer for CPU use)
				//
				TRefCountPtr<IMF2DBuffer> Buffer2D;
				if ((Result = Buffer->QueryInterface(__uuidof(IMF2DBuffer), (void**)Buffer2D.GetInitReference())) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): QueryInterface(IMF2DBuffer) failed with %08x"), Result);
					return false;
				}

				uint8* Data = nullptr;
				LONG Pitch = 0;
				if ((Result = Buffer2D->Lock2D(&Data, &Pitch)) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Lock2D(IMF2DBuffer) failed with %08x"), Result);
					return false;
				}

				InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));
				//const int32 HeightYUV = TextureDesc.Format == DXGI_FORMAT_NV12 ? Height * 3 / 2 : Height;
				const int32 HeightYUV = Height;
				TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> DataCopy = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>(MakeArrayView<uint8>(Data, Pitch * HeightYUV));
				if ((Result = Buffer2D->Unlock2D()) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Unlock2D(IMF2DBuffer) failed with %08x"), Result);
					return false;
				}
				return SetupOutputTextureSampleFromBuffer(OutErrorMessage, InOutTextureSampleToSetup,
												DataCopy,
												Pitch,							// Buffer stride
												FIntPoint(Width, HeightYUV),	// Buffer dimensions
												*InOutBufferPropertes);
			}
			else
			{
				//
				// DX11
				//
				check(DXVersion >= 11000);

				// Notes:
				// - No need to apply a *1.5 factor to the height in this case. DX11 can access sub-resources in both NV12 & P010 to get to the CbCr data
				// - No need to specify a pitch as this is all directly handled on the GPU side of things
				//
				ID3D11Device* Device = reinterpret_cast<ID3D11Device*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::DXDevice));
				ID3D11DeviceContext* DeviceContext = reinterpret_cast<ID3D11DeviceContext*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::DXDeviceContext));
				if (Device && DeviceContext)
				{
					uint32 ViewIndex = 0;
					if ((Result = DXGIBuffer->GetSubresourceIndex(&ViewIndex)) != S_OK)
					{
						OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetSubresourceIndex failed with %08x"), Result);
						return false;
					}

					return SetupOutputTextureSampleFromSharedTexture(OutErrorMessage, InOutTextureSampleToSetup,
								Device, DeviceContext,
								Texture2D, ViewIndex,
								FIntPoint(InDecoderOutput->GetWidth(), InDecoderOutput->GetHeight()), *InOutBufferPropertes);
				}
			}
		}
		else if (bIsSW)
		{
			//
			// "Software" case
			//
			TRefCountPtr<IMFSample> DecodedOutputSample = reinterpret_cast<IMFSample*>(InDecoderOutput->GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType::MFSample));

			DWORD BuffersNum = 0;
			if ((Result = DecodedOutputSample->GetBufferCount(&BuffersNum)) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferCount() failed with %08x"), Result);
				return false;
			}
			if (BuffersNum != 1)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferCount() returned %u buffers instead of 1"), BuffersNum);
				return false;
			}
			TRefCountPtr<IMFMediaBuffer> Buffer;
			if ((Result = DecodedOutputSample->GetBufferByIndex(0, Buffer.GetInitReference())) != S_OK)
			{
				OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetBufferByIndex() failed with %08x"), Result);
				return false;
			}

			// With software decode we cannot query any DXGI/DirectX data types, so we query the decoder extra data...
			EPixelFormat PixFmt = static_cast<EPixelFormat>(ElectraDecodersUtil::GetVariantValueSafeI64(ExtraValues, TEXT("pixfmt"), (int64)EPixelFormat::PF_NV12));
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::PixelFormat, Electra::FVariantValue((int64)PixFmt));
			InOutBufferPropertes->Set(IDecoderOutputOptionNames::BitsPerComponent, Electra::FVariantValue((int64)((PixFmt == EPixelFormat::PF_NV12) ? 8 : 10)));

			TRefCountPtr<IMF2DBuffer> Buffer2D;
			if ((Result = Buffer->QueryInterface(__uuidof(IMF2DBuffer), (void**)Buffer2D.GetInitReference())) == S_OK)
			{
				uint8* Data = nullptr;
				LONG Pitch = 0;
				if ((Result = Buffer2D->Lock2D(&Data, &Pitch)) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Lock2D(IMF2DBuffer) failed with %08x"), Result);
					return false;
				}

				InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));

				TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> DataCopy = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>(MakeArrayView<uint8>(Data, Pitch * Height));
				if ((Result = Buffer2D->Unlock2D()) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Unlock2D(IMF2DBuffer) failed with %08x"), Result);
					return false;
				}

				return SetupOutputTextureSampleFromBuffer(OutErrorMessage, InOutTextureSampleToSetup,
												DataCopy,
												Pitch,						// Buffer stride
												FIntPoint(Width, Height),	// Buffer dimensions
												*InOutBufferPropertes);
			}
			else
			{
				DWORD BufferSize = 0;
				uint8* Data = nullptr;
				if ((Result = Buffer->GetCurrentLength(&BufferSize)) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): GetCurrentLength() failed with %08x"), Result);
					return false;
				}
				if ((Result = Buffer->Lock(&Data, NULL, NULL)) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Lock(IMFMediaBuffer) failed with %08x"), Result);
					return false;
				}
				TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> DataCopy = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>(MakeArrayView<uint8>(Data, BufferSize));
				if ((Result = Buffer->Unlock()) != S_OK)
				{
					OutErrorMessage = FString::Printf(TEXT("SetupOutputTextureSample(): Unlock(IMFMediaBuffer) failed with %08x"), Result);
					return false;
				}
				int32 Pitch = Width * ((PixFmt == EPixelFormat::PF_NV12) ? 1 : 2);
				InOutBufferPropertes->Set(IDecoderOutputOptionNames::Pitch, Electra::FVariantValue((int64)Pitch));
				return SetupOutputTextureSampleFromBuffer(OutErrorMessage, InOutTextureSampleToSetup,
												DataCopy,
												Pitch,						// Buffer stride
												FIntPoint(Width, Height),	// Buffer dimensions
												*InOutBufferPropertes);
			}
		}
	}

	return false;
}

