// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformElectraDecoders.h"
#include "IElectraCodecRegistry.h"
#include "ElectraDecodersModule.h"

#include "h264/H264_VideoDecoder_Windows.h"
#include "h265/H265_VideoDecoder_Windows.h"
#include "aac/AAC_AudioDecoder_Windows.h"

#include "HAL/PlatformProcess.h"
#include "RHI.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "mfapi.h"
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace PlatformElectraDecodersWindows
{
	static TArray<TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe>> Factories;
	static bool bMFInitialized = false;
}


void FPlatformElectraDecodersWindows::Startup()
{
	// Minimum of Windows 8 required.
	if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
	{
		UE_LOGF(LogElectraDecoders, Error, "Need at least Windows 8.0.");
		return;
	}
	// We need RHI to perform conversion from the decoder output.
	if (GDynamicRHI == nullptr || RHIGetInterfaceType() == ERHIInterfaceType::Null)
	{
		UE_LOGF(LogElectraDecoders, Log, "No or only dummy dynamic RHI detected, cannot use Electra decoders without RHI.");
		return;
	}

	// Prepare Microsoft's Media Foundation
	if (!(FPlatformProcess::GetDllHandle(TEXT("mf.dll"))
		&& FPlatformProcess::GetDllHandle(TEXT("mfplat.dll"))
		&& FPlatformProcess::GetDllHandle(TEXT("msmpeg2vdec.dll"))
		&& FPlatformProcess::GetDllHandle(TEXT("MSAudDecMFT.dll"))))
	{
		UE_LOGF(LogElectraDecoders, Warning, "Could not load Media Foundation. One or more of these dlls not found - mf.dll, mfplat.dll, msmpeg2vdec.dll, MSAudDecMFT.dll. Usually this indicators running on a Windows Server OS.");
		return;
	}
	HRESULT Result = MFStartup(MF_VERSION);
	if (FAILED(Result))
	{
		UE_LOGF(LogElectraDecoders, Error, "MFStartup failed with 0x%08x", Result);
		return;
	}
	PlatformElectraDecodersWindows::bMFInitialized = true;
}

void FPlatformElectraDecodersWindows::Shutdown()
{
	PlatformElectraDecodersWindows::Factories.Empty();

	if (PlatformElectraDecodersWindows::bMFInitialized)
	{
		PlatformElectraDecodersWindows::bMFInitialized = false;
		MFShutdown();
	}
}

void FPlatformElectraDecodersWindows::RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith)
{
	check(InCodecFactoryToRegisterWith);
	if (InCodecFactoryToRegisterWith && PlatformElectraDecodersWindows::bMFInitialized)
	{
#ifdef ELECTRA_DECODERS_ENABLE_DX
		auto RegisterFactory = [&](TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> InFactory) -> void
		{
			if (InFactory.IsValid())
			{
				PlatformElectraDecodersWindows::Factories.Emplace(InFactory);
				InCodecFactoryToRegisterWith->AddCodecFactory(InFactory);
			}
		};

		// H.264 video decoder
		RegisterFactory(FH264VideoDecoderWindows::CreateFactory());
		// H.265 video decoder
		RegisterFactory(FH265VideoDecoderWindows::CreateFactory());
		// AAC audio decoder
		RegisterFactory(FAACAudioDecoderWindows::CreateFactory());
#endif
	}
}
