// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class IElectraCodecRegistry;

class FPlatformElectraDecodersWindows
{
public:
	static void Startup();
	static void Shutdown();
	static void RegisterWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith);
};

using FPlatformElectraDecoders = FPlatformElectraDecodersWindows;
