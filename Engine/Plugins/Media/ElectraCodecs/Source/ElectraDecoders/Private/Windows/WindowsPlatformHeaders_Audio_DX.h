// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include "mftransform.h"
#include "mfapi.h"
#include "mferror.h"
#include "mfidl.h"
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#endif
