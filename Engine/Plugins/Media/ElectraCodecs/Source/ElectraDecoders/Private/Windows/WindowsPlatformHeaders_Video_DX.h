// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"

#include <d3d11.h>
#include <d3d11_1.h>
#include <d3d12.h>
#include <dxgi1_4.h>

#include "mftransform.h"
#include "mfapi.h"
#include "mferror.h"
#include "mfidl.h"

#if 0 // defined(NTDDI_WIN10_NI) // Currently disabled as this causes trouble on AMD GPU
#include "mfd3d12.h"
#define ALLOW_MFSAMPLE_WITH_DX12	1	// Windows SDK 22621 and up do feature APIs to support DX12 texture resources with WMF transforms
#else
#define ALLOW_MFSAMPLE_WITH_DX12	0
#endif

#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

struct FElectraVideoDecoderDXDeviceContext
{
	void Release()
	{
		DxDevice.SafeRelease();
		DxDeviceContext.SafeRelease();
	}
	void SetDeviceAndContext(void* InDxDevice, void* InDxDeviceContext)
	{
		DxDevice = reinterpret_cast<ID3D11Device*>(InDxDevice);
		DxDeviceContext = reinterpret_cast<ID3D11DeviceContext*>(InDxDeviceContext);
	}
	TRefCountPtr<ID3D11Device> DxDevice;
	TRefCountPtr<ID3D11DeviceContext> DxDeviceContext;
};

#endif
