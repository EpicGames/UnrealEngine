// Copyright Epic Games, Inc. All Rights Reserved.

#include "h264/H264_VideoDecoder_Windows.h"

#ifdef ELECTRA_DECODERS_ENABLE_DX

#include "DX/VideoDecoderH264_DX.h"
#include "DX/DecoderErrors_DX.h"
#include "ElectraDecodersPlatformResources.h"
#include "ElectraDecodersUtils.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "ElectraDecodersModule.h"

/*********************************************************************************************************************/
#include COMPILED_PLATFORM_HEADER(PlatformHeaders_Video_DX.h)

#ifdef ELECTRA_DECODERS_HAVE_DX11
#pragma comment(lib, "D3D11.lib")
#endif
#pragma comment(lib, "D3D12.lib")
#pragma comment(lib, "DXGI.lib")

/*********************************************************************************************************************/
#include "HAL/IConsoleManager.h"

#if defined(ELECTRACODECS_ENABLE_MF_SWDECODE_H264) && ELECTRACODECS_ENABLE_MF_SWDECODE_H264 != 0
static TAutoConsoleVariable<int32> CVarElectraWindowsUseSoftwareDecoding(
	TEXT("Electra.Win.UseSoftwareDecodingH264"),
	0,
	TEXT("Use software decoding on Windows for H.264 even if hardware decoding is supported.\n")
	TEXT(" 0: use hardware decoding if supported (default); 1: use software decoding."),
	ECVF_Default);
#endif

/*********************************************************************************************************************/


namespace ElectraGuidsH264
{
DEFINE_GUID(MF_SA_D3D11_AWARE, 0x206b4fc8, 0xfcf9, 0x4c51, 0xaf, 0xe3, 0x97, 0x64, 0x36, 0x9e, 0x33, 0xa0);
}


class FH264VideoDecoderFactoryWindows : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation
{
public:
	virtual ~FH264VideoDecoderFactoryWindows()
	{}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		IElectraVideoDecoderH264_DX::PlatformGetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		if (InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','1') && InCodecFormat.FourCC != ElectraDecodersUtil::Make4CC('a','v','c','3'))
		{
			return 0;
		}

		const Electra::FCodecTypeFormat::FVideo& vi = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		int32 Width = (int32)vi.Width;
		int32 Height = (int32)vi.Height;
		double fps = vi.FrameRate.IsValid() ? vi.FrameRate.AsDecimal() : 0.0;

		// Check if supported.
		TArray<IElectraVideoDecoderH264_DX::FSupportedConfiguration> Configs;
		IElectraVideoDecoderH264_DX::PlatformGetSupportedConfigurations(Configs);
		bool bSupported = false;
		for(int32 i=0; i<Configs.Num(); ++i)
		{
			const IElectraVideoDecoderH264_DX::FSupportedConfiguration& Cfg = Configs[i];
			if (Cfg.Profile == vi.Profile.Profile)
			{
				if (vi.Profile.Level > (uint32)Cfg.Level)
				{
					continue;
				}
				if (ElectraDecodersUtil::ExceedsMaxDimensions(Width, Height, Cfg.Width, Cfg.Height))
				{
					continue;
				}
				if (fps > 0.0 && Cfg.FramesPerSecond && (int32)fps > Cfg.FramesPerSecond)
				{
					continue;
				}
				if (Cfg.Num16x16Macroblocks && ((Align(Width, 16) * Align(Height, 16)) / 256) > Cfg.Num16x16Macroblocks)
				{
					continue;
				}
				bSupported = true;
				break;
			}
		}

		return bSupported ? 1 : 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? IElectraVideoDecoderH264_DX::Create(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("Media Foundation")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString(TEXT("Windows"))); }
	FString GetVendor() const override
	{ return FString(TEXT("Microsoft")); }
};


TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FH264VideoDecoderWindows::CreateFactory()
{
	if (IElectraVideoDecoderH264_DX::PlatformStaticInitialize())
	{
		return MakeShared<FH264VideoDecoderFactoryWindows, ESPMode::ThreadSafe>();
	}
	return nullptr;
}


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/



namespace IElectraVideoDecoderH264_DX_Platform
{
	static TArray<IElectraVideoDecoderH264_DX::FSupportedConfiguration> DecoderConfigurations;
	static bool bDecoderConfigurationsDirty = true;

	class FPlatformHandle : public IElectraVideoDecoderH264_DX::IPlatformHandle
	{
	public:
		virtual ~FPlatformHandle()
		{
			Close();
		}
		virtual int32 GetDXVersionTimes1000() const override
		{ return DXVersion; }
		virtual void* GetMFTransform() const override
		{ return MFT.GetReference(); }
		virtual void* GetDXDevice() const override
		{ return Ctx.DxDevice.GetReference(); }
		virtual void* GetDXDeviceContext() const override
		{ return Ctx.DxDeviceContext.GetReference(); }
		virtual bool IsSoftware() const override
		{ return bIsSW; }

		void Close()
		{
			MFT.SafeRelease();
			Ctx.Release();
			DxDeviceManager.SafeRelease();
		}

		TRefCountPtr<IMFTransform> MFT;
		int32 DXVersion = 0;
		bool bIsSW = false;


		FElectraVideoDecoderDXDeviceContext Ctx;
		TRefCountPtr<IMFDXGIDeviceManager> DxDeviceManager;
	};
}

bool IElectraVideoDecoderH264_DX::PlatformStaticInitialize()
{
	return true;
}

void IElectraVideoDecoderH264_DX::PlatformGetSupportedConfigurations(TArray<FSupportedConfiguration>& OutSupportedConfigurations)
{
	if (IElectraVideoDecoderH264_DX_Platform::bDecoderConfigurationsDirty)
	{
		IElectraVideoDecoderH264_DX_Platform::DecoderConfigurations.Empty();
		// Baseline
		IElectraVideoDecoderH264_DX_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_DX::FSupportedConfiguration(66, 52, 120, 4096, 2304, 0 /*1280*720 / 256*/));
		// Main
		IElectraVideoDecoderH264_DX_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_DX::FSupportedConfiguration(77, 52, 120, 4096, 2304, 0 /*1280*720 / 256*/));
		// High
		IElectraVideoDecoderH264_DX_Platform::DecoderConfigurations.Emplace(IElectraVideoDecoderH264_DX::FSupportedConfiguration(100, 52, 120, 4096, 2304, 0 /*1280*720 / 256*/));

		IElectraVideoDecoderH264_DX_Platform::bDecoderConfigurationsDirty = false;
	}
	OutSupportedConfigurations = IElectraVideoDecoderH264_DX_Platform::DecoderConfigurations;
}

void IElectraVideoDecoderH264_DX::PlatformGetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
{
	OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)8));
	OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	OutOptions.Emplace(IElectraDecoderFeature::SupportsDroppingOutput, FVariant(true));
}


IElectraDecoder::FError IElectraVideoDecoderH264_DX::PlatformCreateMFDecoderTransform(IPlatformHandle** OutPlatformHandle, const TMap<FString, FVariant>& InOptions)
{
	IElectraDecoder::FError Error;
	check(OutPlatformHandle);
	if (OutPlatformHandle)
	{
		*OutPlatformHandle = nullptr;
	}
	if (OutPlatformHandle)
	{
		// Must be Windows 8 or better
		if (FPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			TUniquePtr<IElectraVideoDecoderH264_DX_Platform::FPlatformHandle> pfh = MakeUnique<IElectraVideoDecoderH264_DX_Platform::FPlatformHandle>();

			// Create the decoder transform.
			HRESULT hr;
			MFT_REGISTER_TYPE_INFO MediaInputInfo { MFMediaType_Video, MFVideoFormat_H264 };
			TRefCountPtr<IMFAttributes>	Attributes;
			IMFActivate** ActivateObjects = nullptr;
			UINT32 NumActivateObjects = 0;
			if ((hr = MFTEnumEx(MFT_CATEGORY_VIDEO_DECODER, MFT_ENUM_FLAG_SYNCMFT, &MediaInputInfo, nullptr, &ActivateObjects, &NumActivateObjects)) != S_OK)
			{
				return Error.SetMessage(TEXT("MFTEnumEx() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
			}
			if (NumActivateObjects == 0)
			{
				return Error.SetMessage(TEXT("MFTEnumEx() returned zero activation objects")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
			}
			IMFTransform* NewDecoder = nullptr;
			hr = ActivateObjects[0]->ActivateObject(IID_PPV_ARGS(&NewDecoder));
			if (hr == S_OK)
			{
				*(pfh->MFT.GetInitReference()) = NewDecoder;
			}
			for(UINT32 i=0; i<NumActivateObjects; ++i)
			{
				ActivateObjects[i]->Release();
			}
			CoTaskMemFree(ActivateObjects);
			ActivateObjects = nullptr;
			if (hr != S_OK)
			{
				return Error.SetMessage(TEXT("H.264 decoder transform activation failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
			}
			if ((hr = NewDecoder->GetAttributes(Attributes.GetInitReference())) != S_OK)
			{
				return Error.SetMessage(TEXT("Failed to get H.264 decoder transform attributes")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
			}


#if defined(ELECTRACODECS_ENABLE_MF_SWDECODE_H264) && ELECTRACODECS_ENABLE_MF_SWDECODE_H264 != 0
			// Forced software decoding? If so there is nothing further required.
			if (CVarElectraWindowsUseSoftwareDecoding.GetValueOnAnyThread() > 0)
			{
				pfh->bIsSW = true;
				*OutPlatformHandle = pfh.Release();
				return Error;
			}
#endif

			// Check if the decoder is DX11 aware, in which case we need to create a device manager and a decoding DX device
			// we can use to enable hardware acceleration.
			uint32 IsDX11Aware = 0;
			/*hr =*/ Attributes->GetUINT32(ElectraGuidsH264::MF_SA_D3D11_AWARE, &IsDX11Aware);
			if (IsDX11Aware)
			{
				uint32 DxDeviceCreationFlags = 0;
				TRefCountPtr<IDXGIAdapter> DXGIAdapter;
				void* ApplicationD3DDevice = nullptr;
				int32 ApplicationD3DDeviceVersion = 0;
				if (FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&ApplicationD3DDevice, &ApplicationD3DDeviceVersion))
				{
					if (ApplicationD3DDeviceVersion >= 12000)
					{
						TRefCountPtr<IDXGIFactory4> DXGIFactory;
						if ((hr = CreateDXGIFactory(__uuidof(IDXGIFactory4), (void**)DXGIFactory.GetInitReference())) != S_OK)
						{
							return Error.SetMessage(TEXT("CreateDXGIFactory() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
						}

						ID3D12Device* DxDevice = static_cast<ID3D12Device*>(ApplicationD3DDevice);
						LUID Luid = DxDevice->GetAdapterLuid();
						if ((hr = DXGIFactory->EnumAdapterByLuid(Luid, __uuidof(IDXGIAdapter), (void**)DXGIAdapter.GetInitReference())) != S_OK)
						{
							return Error.SetMessage(TEXT("EnumAdapterByLuid() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
						}

						pfh->DXVersion = 12000;
					}
					else if (ApplicationD3DDeviceVersion >= 11000)
					{
						ID3D11Device* DxDevice = static_cast<ID3D11Device*>(ApplicationD3DDevice);
						TRefCountPtr<IDXGIDevice> DXGIDevice;
						DxDevice->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
						if (!DXGIDevice.IsValid())
						{
							return Error.SetMessage(TEXT("ID3D11Device::QueryInterface(IDXGIDevice) failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
						}
						if ((hr = DXGIDevice->GetAdapter((IDXGIAdapter**)DXGIAdapter.GetInitReference())) != S_OK)
						{
							return Error.SetMessage(TEXT("IDXGIDevice::GetAdapter() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);;
						}
						DxDeviceCreationFlags = DxDevice->GetCreationFlags();
						pfh->DXVersion = 11000;
					}
					else
					{
						return Error.SetMessage(TEXT("Must be either DirectX 11 or 12")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
					}
				}
				else
				{
					// Not using D3D for rendering. Create decoding device on the default adapter then.
				}

				// Create device manager
				UINT ResetToken = 0;
				if ((hr = MFCreateDXGIDeviceManager(&ResetToken, pfh->DxDeviceManager.GetInitReference())) != S_OK)
				{
					return Error.SetMessage(TEXT("MFCreateDXGIDeviceManager() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
				}

				bool bAssociatedDeviceManager = false;
#if ALLOW_MFSAMPLE_WITH_DX12
				// Try setting the D3D12 device with the device manager.
				if (pfh->DXVersion == 12000)
				{
					bAssociatedDeviceManager = SUCCEEDED(pfh->DxDeviceManager->ResetDevice(static_cast<ID3D12Device*>(ApplicationD3DDevice), ResetToken));
					// note: pfh->Ctx.DxDevice & pfh->Ctx.DxDeviceContext will be nullptr (everything uses the DX12 render device)
				}
#endif

#ifdef ELECTRA_DECODERS_HAVE_DX11
				if (!bAssociatedDeviceManager)
				{
					D3D_FEATURE_LEVEL FeatureLevel;

					uint32 DeviceCreationFlags = 0;
					if ((DxDeviceCreationFlags & D3D11_CREATE_DEVICE_DEBUG) != 0)
					{
						DeviceCreationFlags |= D3D11_CREATE_DEVICE_DEBUG;
					}

					hr = D3D11CreateDevice(DXGIAdapter, DXGIAdapter ? D3D_DRIVER_TYPE_UNKNOWN : D3D_DRIVER_TYPE_HARDWARE, nullptr,
							DeviceCreationFlags, nullptr, 0, D3D11_SDK_VERSION, pfh->Ctx.DxDevice.GetInitReference(), &FeatureLevel, pfh->Ctx.DxDeviceContext.GetInitReference());
					if (hr != S_OK || !pfh->Ctx.DxDevice.IsValid())
					{
						return Error.SetMessage(TEXT("D3D11CreateDevice() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
					}
					if (FeatureLevel < D3D_FEATURE_LEVEL_9_3)
					{
						return Error.SetMessage(TEXT("Failed to create D3D11 Device with feature level 9.3 or above")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
					}

					hr = pfh->DxDeviceManager->ResetDevice(pfh->Ctx.DxDevice, ResetToken);
					if (hr != S_OK)
					{
						return Error.SetMessage(TEXT("DXGIDeviceManager::ResetDevice() failed")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
					}
					// Multithread protect the newly created device as we're going to use it from decoding thread and from render thread for texture
					// sharing between decoding and rendering DX devices
					TRefCountPtr<ID3D10Multithread> DxMultithread;
					if ((hr = pfh->Ctx.DxDevice->QueryInterface(__uuidof(ID3D10Multithread), (void**)DxMultithread.GetInitReference())) == S_OK)
					{
						DxMultithread->SetMultithreadProtected(1);
					}
					bAssociatedDeviceManager = true;
				}
#endif

				// No need to set a device manager when we didn't create a device for it to manage.
				if (!bAssociatedDeviceManager || (hr = NewDecoder->ProcessMessage(MFT_MESSAGE_SET_D3D_MANAGER, reinterpret_cast<ULONG_PTR>(pfh->DxDeviceManager.GetReference()))) != S_OK)
				{
					// Fallback to software decoding.
					pfh->bIsSW = true;
					pfh->Ctx.Release();
					pfh->DxDeviceManager.SafeRelease();

					//return Error.SetMessage(TEXT("Failed to set the D3D manager on the decoder transform")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER).SetSdkCode(hr);
				}

				*OutPlatformHandle = pfh.Release();
			}
			else
			{
				// Not D3D aware. Fallback to software decoding.
				pfh->bIsSW = true;
				*OutPlatformHandle = pfh.Release();
			}
		}
		else
		{
			return Error.SetMessage(TEXT("Must be at least Windows 8")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
		}
	}
	else
	{
		return Error.SetMessage(TEXT("Bad parameters")).SetCode(ERRCODE_INTERNAL_BAD_PARAMETER);
	}
	return Error;
}

void IElectraVideoDecoderH264_DX::PlatformReleaseMFDecoderTransform(IPlatformHandle** InOutPlatformHandle)
{
	if (InOutPlatformHandle && *InOutPlatformHandle)
	{
		TUniquePtr<IElectraVideoDecoderH264_DX_Platform::FPlatformHandle> pfh(static_cast<IElectraVideoDecoderH264_DX_Platform::FPlatformHandle*>(*InOutPlatformHandle));
		*InOutPlatformHandle = nullptr;
		pfh->Close();
		pfh.Reset();
	}
}

#endif
