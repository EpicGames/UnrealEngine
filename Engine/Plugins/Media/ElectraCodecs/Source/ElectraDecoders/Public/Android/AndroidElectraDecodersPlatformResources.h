// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#include "ParameterDictionary.h"

class IElectraDecoderVideoOutput;
class FElectraTextureSample;

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"


class FElectraTextureSamplePool;

class FElectraDecodersPlatformResourcesAndroid
{
public:

	class IDecoderResourceInstance
	{
	public:
		virtual ~IDecoderResourceInstance() = default;

		class ISurfaceRequestCallback : public TSharedFromThis<ISurfaceRequestCallback, ESPMode::ThreadSafe>
		{
		public:
			virtual ~ISurfaceRequestCallback() = default;
			enum class ESurfaceType
			{
				NoSurface,		// No decoding surface. Must use CPU side buffer.
				Surface,
				Error
			};
			virtual void OnNewSurface(ESurfaceType InSurfaceType, jobject InSurface) = 0;
		};
		// The decoder calls this to request a surface onto which to decode.
		virtual void RequestSurface(TWeakPtr<ISurfaceRequestCallback, ESPMode::ThreadSafe> InRequestCallback) = 0;
	};

	static const FName OptionKeyOutputTexturePoolID()
	{ return TEXT("output_texturepool_id"); }
	static ELECTRADECODERS_API int64 RegisterOutputTexturePool(TSharedPtr<FElectraTextureSamplePool, ESPMode::ThreadSafe> InTexturePool);
	static ELECTRADECODERS_API void UnregisterOutputTexturePool(int64 InTexturePoolID);

	static ELECTRADECODERS_API IDecoderResourceInstance* CreateResourceInstance(int64 InResourceID);
	static ELECTRADECODERS_API void ReleaseResourceInstance(IDecoderResourceInstance* InInstanceToRelease);


	struct IDecoderPlatformResource;
	static ELECTRADECODERS_API IDecoderPlatformResource* CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions);
	static ELECTRADECODERS_API void ReleasePlatformVideoResource(void* InOwnerHandle, IDecoderPlatformResource* InHandleToRelease);

	static ELECTRADECODERS_API bool SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, IDecoderPlatformResource* InPlatformResource);
};

using FElectraDecodersPlatformResources = FElectraDecodersPlatformResourcesAndroid;
