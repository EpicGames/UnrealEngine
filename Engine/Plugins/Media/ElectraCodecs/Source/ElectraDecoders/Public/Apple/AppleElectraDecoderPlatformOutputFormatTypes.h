// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include <CoreVideo/CoreVideo.h>

namespace ElectraVideoDecoderFormatTypesApple
{
	static inline int32 GetNumComponentBitsForPixelFormat(int32 InPixelFormat)
	{
		switch(InPixelFormat)
		{
			case kCVPixelFormatType_24RGB:
			case kCVPixelFormatType_24BGR:
			case kCVPixelFormatType_32ARGB:
			case kCVPixelFormatType_32BGRA:
			case kCVPixelFormatType_32ABGR:
			case kCVPixelFormatType_32RGBA:
			case kCVPixelFormatType_422YpCbCr8:
			case kCVPixelFormatType_4444YpCbCrA8:
			case kCVPixelFormatType_4444YpCbCrA8R:
			case kCVPixelFormatType_4444AYpCbCr8:
			case kCVPixelFormatType_444YpCbCr8:
			case kCVPixelFormatType_420YpCbCr8Planar:
			case kCVPixelFormatType_420YpCbCr8PlanarFullRange:
			case kCVPixelFormatType_422YpCbCr_4A_8BiPlanar:
			case kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange:
			case kCVPixelFormatType_420YpCbCr8BiPlanarFullRange:
			case kCVPixelFormatType_422YpCbCr8BiPlanarVideoRange:
			case kCVPixelFormatType_422YpCbCr8BiPlanarFullRange:
			case kCVPixelFormatType_444YpCbCr8BiPlanarVideoRange:
			case kCVPixelFormatType_444YpCbCr8BiPlanarFullRange:
			case kCVPixelFormatType_422YpCbCr8_yuvs:
			case kCVPixelFormatType_422YpCbCr8FullRange:
			case kCVPixelFormatType_OneComponent8:
			case kCVPixelFormatType_TwoComponent8:
			case kCVPixelFormatType_420YpCbCr8VideoRange_8A_TriPlanar:
			case kCVPixelFormatType_8Indexed:
			case kCVPixelFormatType_8IndexedGray_WhiteIsZero:
				return 8;
			case kCVPixelFormatType_64ARGB:
			case kCVPixelFormatType_64RGBALE:
			case kCVPixelFormatType_48RGB:
			case kCVPixelFormatType_32AlphaGray:
			case kCVPixelFormatType_16Gray:
			case kCVPixelFormatType_4444AYpCbCr16:
			case kCVPixelFormatType_OneComponent16:
			case kCVPixelFormatType_TwoComponent16:
			case kCVPixelFormatType_OneComponent16Half:
			case kCVPixelFormatType_TwoComponent16Half:
			case kCVPixelFormatType_DisparityFloat16:
			case kCVPixelFormatType_DepthFloat16:
			case kCVPixelFormatType_16VersatileBayer:
			case kCVPixelFormatType_64RGBA_DownscaledProResRAW:
			case kCVPixelFormatType_422YpCbCr16BiPlanarVideoRange:
			case kCVPixelFormatType_444YpCbCr16BiPlanarVideoRange:
			case kCVPixelFormatType_444YpCbCr16VideoRange_16A_TriPlanar:
			case kCVPixelFormatType_16BE555:
			case kCVPixelFormatType_16LE555:
			case kCVPixelFormatType_16LE5551:
			case kCVPixelFormatType_16BE565:
			case kCVPixelFormatType_16LE565:
				return 16;
			case kCVPixelFormatType_30RGB:
			case kCVPixelFormatType_422YpCbCr16:
			case kCVPixelFormatType_422YpCbCr10:
			case kCVPixelFormatType_444YpCbCr10:
			case kCVPixelFormatType_30RGBLEPackedWideGamut:
			case kCVPixelFormatType_ARGB2101010LEPacked:
			case kCVPixelFormatType_40ARGBLEWideGamut:
			case kCVPixelFormatType_40ARGBLEWideGamutPremultiplied:
			case kCVPixelFormatType_OneComponent10:
			case kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange:
			case kCVPixelFormatType_422YpCbCr10BiPlanarVideoRange:
			case kCVPixelFormatType_444YpCbCr10BiPlanarVideoRange:
			case kCVPixelFormatType_420YpCbCr10BiPlanarFullRange:
			case kCVPixelFormatType_422YpCbCr10BiPlanarFullRange:
			case kCVPixelFormatType_444YpCbCr10BiPlanarFullRange:
				return 10;
			case kCVPixelFormatType_OneComponent12:
				return 12;
			case kCVPixelFormatType_OneComponent32Float:
			case kCVPixelFormatType_TwoComponent32Float:
			case kCVPixelFormatType_DisparityFloat32:
			case kCVPixelFormatType_DepthFloat32:
				return 32;
			case kCVPixelFormatType_64RGBAHalf:
				return 64;
			case kCVPixelFormatType_128RGBAFloat:
				return 128;
			case kCVPixelFormatType_14Bayer_GRBG:
			case kCVPixelFormatType_14Bayer_RGGB:
			case kCVPixelFormatType_14Bayer_BGGR:
			case kCVPixelFormatType_14Bayer_GBRG:
				return 14;
			case kCVPixelFormatType_1Monochrome:
			case kCVPixelFormatType_1IndexedGray_WhiteIsZero:
				return 1;
			case kCVPixelFormatType_2Indexed:
			case kCVPixelFormatType_2IndexedGray_WhiteIsZero:
				return 2;
			case kCVPixelFormatType_4Indexed:
			case kCVPixelFormatType_4IndexedGray_WhiteIsZero:
				return 4;
			default:
				return -1;
		}
	}

}
