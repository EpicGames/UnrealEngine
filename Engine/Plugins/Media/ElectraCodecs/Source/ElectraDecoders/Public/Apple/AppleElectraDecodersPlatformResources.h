// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MAC || PLATFORM_IOS || PLATFORM_TVOS

#include "ParameterDictionary.h"

class IElectraDecoderVideoOutput;
class FElectraTextureSample;

class FElectraDecodersPlatformResourcesApple
{
public:
    static ELECTRADECODERS_API bool GetMetalDevice(void** OutMetalDevice);

    struct IDecoderPlatformResource;
	static ELECTRADECODERS_API IDecoderPlatformResource* CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions);
	static ELECTRADECODERS_API void ReleasePlatformVideoResource(void* InOwnerHandle, IDecoderPlatformResource* InHandleToRelease);

	static ELECTRADECODERS_API bool SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, IDecoderPlatformResource* InPlatformResource);
};

using FElectraDecodersPlatformResources = FElectraDecodersPlatformResourcesApple;

#endif
