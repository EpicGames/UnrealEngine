// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if ELECTRA_DECODERS_HAVE_PLATFORM_DEFAULTS
#include COMPILED_PLATFORM_HEADER(ElectraDecodersPlatformResources.h)
#else

class FElectraDecodersPlatformResourcesNull
{
public:
};

using FElectraDecodersPlatformResources = FElectraDecodersPlatformResourcesNull;

#endif
