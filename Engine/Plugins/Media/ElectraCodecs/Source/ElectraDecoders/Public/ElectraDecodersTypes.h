// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace ElectraDecodersUtil
{
	struct FFractionalValue
	{
		FFractionalValue(int64 InNumerator=0, uint32 InDenominator=0)
			: Num(InNumerator), Denom(InDenominator)
		{ }
		int64 Num = 0;
		uint32 Denom = 0;
	};
}
