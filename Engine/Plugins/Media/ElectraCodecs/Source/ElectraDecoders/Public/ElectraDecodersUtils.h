// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Misc/Variant.h"
#include "CodecTypeFormat.h"


namespace ElectraDecodersUtil
{
	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	// Advance a pointer by a number of bytes.
	template <typename T, typename C>
	inline T AdvancePointer(T pPointer, C numBytes)
	{
		return(T(UPTRINT(pPointer) + UPTRINT(numBytes)));
	}

	template <typename T>
	inline T AbsoluteValue(T Value)
	{
		return Value >= T(0) ? Value : -Value;
	}

	template <typename T>
	inline T Min(T a, T b)
	{
		return a < b ? a : b;
	}

	template <typename T>
	inline T Max(T a, T b)
	{
		return a > b ? a : b;
	}

	inline uint32 BitReverse32(uint32 InValue)
	{
		uint32 rev = 0;
		for (int32 i = 0; i < 32; ++i)
		{
			rev = (rev << 1) | (InValue & 1);
			InValue >>= 1;
		}
		return rev;
	}

	// Returns true if the given dimensions exceed the configured maximum dimensions.
	// The comparison is orientation-independent: the two dimensions are matched by
	// magnitude (larger-against-larger, smaller-against-smaller) rather than positionally,
	// so a portrait source is accepted against landscape-oriented limits and vice versa.
	// A maximum of 0 means "no limit" for that dimension.
	inline bool ExceedsMaxDimensions(const int64 InDimensionA, const int64 InDimensionB, const int64 InMaxDimensionA, const int64 InMaxDimensionB)
	{
		const int64 LongDimension = Max(InDimensionA, InDimensionB);
		const int64 ShortDimension = Min(InDimensionA, InDimensionB);
		const int64 MaxLongDimension = Max(InMaxDimensionA, InMaxDimensionB);
		const int64 MaxShortDimension = Min(InMaxDimensionA, InMaxDimensionB);
		return (LongDimension > MaxLongDimension && MaxLongDimension) || (ShortDimension > MaxShortDimension && MaxShortDimension);
	}

	ELECTRADECODERS_API	bool PrepareCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat);

	ELECTRADECODERS_API int64  GetVariantValueSafeI64(const TMap<FString, FVariant>& InFromMap, const FString& InName, int64 InDefaultValue=0);
	ELECTRADECODERS_API uint64 GetVariantValueSafeU64(const TMap<FString, FVariant>& InFromMap, const FString& InName, uint64 InDefaultValue=0);
	ELECTRADECODERS_API double GetVariantValueSafeDouble(const TMap<FString, FVariant>& InFromMap, const FString& InName, double InDefaultValue=0.0);
	ELECTRADECODERS_API TArray<uint8> GetVariantValueUInt8Array(const TMap<FString, FVariant>& InFromMap, const FString& InName);
	ELECTRADECODERS_API FString GetVariantValueFString(const TMap<FString, FVariant>& InFromMap, const FString& InName);
}
