// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Misc/Optional.h"

#include "CodecTypeFormat.h"


class IElectraCodecFormatUtilsModularFeature : public IModularFeature
{
public:
	virtual ~IElectraCodecFormatUtilsModularFeature() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ElectraCodecFormatUtils"));
		return FeatureName;
	}

	struct FCodecTypeFormatParams
	{
		// No members right now.
	};

	/**
	 * Tries to prepare the given codec type format format with as much information as possible.
	 * On input:
	 *   - set at least the .FourCC member OR the mime type with a codecs parameter
	 *   - set the .RFC6381 member if possible and not already provided with the mime type
	 *   - provide any additional boxes in .ExtraBoxes in ISO/IEC 14496-12 binary box format
	 *      (these should be all boxes from the `stsd` or similar container box)
	 * On output:
	 *   - true, with as many filled in values as could be derived from the given input.
	 *   - false if no information could be derived from the given input.
	 */
	virtual bool PrepareCodecTypeFormat(Electra::FCodecTypeFormat& InOutCodecTypeFormat, const FCodecTypeFormatParams& InCodecTypeFormatParams) = 0;
};
