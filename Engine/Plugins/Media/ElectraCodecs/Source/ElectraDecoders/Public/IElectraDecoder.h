// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/PimplPtr.h"
#include "Containers/Map.h"
#include "Misc/Timespan.h"
#include "Misc/Variant.h"
#include "CodecTypeFormat.h"


class IElectraDecoderOutput
{
public:
	virtual ~IElectraDecoderOutput() = default;
	enum class EType
	{
		None,
		Video,
		Audio,
		Subtitle
	};
	virtual EType GetType() const = 0;

	virtual FTimespan GetPTS() const = 0;
	virtual uint64 GetUserValue() const = 0;
};


enum class EElectraDecoderFlags : uint32
{
	None = 0,
	IsSyncSample = 0x01,
	IsDiscardable = 0x02,
	IsReplaySample = 0x04,
	IsLastReplaySample = 0x08,
	InitCSDOnly = 0x10,
	DoNotOutput = 0x20,
	InputIsProcessed = 0x100
};
ENUM_CLASS_FLAGS(EElectraDecoderFlags);

struct FElectraDecoderInputAccessUnit
{
	const void* Data = nullptr;
	int64 DataSize = 0;
	FTimespan DTS;
	FTimespan PTS;
	FTimespan Duration;
	uint64 UserValue = 0;
	EElectraDecoderFlags Flags = EElectraDecoderFlags::None;
};



class IElectraDecoderBitstreamInfo
{
public:
	virtual ~IElectraDecoderBitstreamInfo() = default;
};



class IElectraDecoderBitstreamProcessor : public TSharedFromThis<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe>
{
public:
	virtual ~IElectraDecoderBitstreamProcessor() = default;

	enum class EProcessResult
	{
		Ok,
		CSDChanged,
		Error
	};

	/**
	 * Returns whether or not calling ProcessInputForDecoding() will modify the provided bitstream in place.
	 * If true you may need to create a temporary copy if you want to retain the bitstream in a cache.
	 */
	virtual bool WillModifyBitstreamInPlace() = 0;

	/**
	 * Clears the processor. Must be called when seeking or flushing.
	 */
	virtual void Clear() = 0;

	/**
	 * Processes the bitstream and places sideband information in the provided IElectraDecoderBitstreamInfo for use
	 * with a subsequent call to SetPropertiesOnOutput().
	 * If necessary the bitstream is modified in place.
	 * See `WillModifyBitstreamInPlace()`
	 * Returns `Ok` or `Error`. If the codec specific data changed `CSDChanged` is returned which is only
	 * informational for you and is functionally the same as `Ok`.
	 */
	virtual EProcessResult ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData) = 0;

	/**
	 * Sets previously extracted sideband data in the provided output dictionary.
	 */
	virtual void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI) = 0;

	/**
	 * Returns an error message if either method returned `Error`.
	 */
	virtual FString GetLastError() = 0;
};



class IElectraDecoder : public TSharedFromThis<IElectraDecoder, ESPMode::ThreadSafe>
{
public:

	virtual ~IElectraDecoder() = default;

	using FInputAccessUnit = FElectraDecoderInputAccessUnit;

	enum class EDecoderError
	{
		// No error, all is well.
		None,
		// End of data processed
		EndOfData,
		// No buffer available when sending access unit to decode or no output available yet (more input required)
		NoBuffer,
		// The decoder was lost due to resource sharing conflicts or when the application was suspended.
		LostDecoder,
		// An internal decoder error occurred. Call GetError() to get the error code.
		Error
	};

	enum class EOutputStatus
	{
		// Output is available.
		Available,
		// Output is not available. Provide more input.
		NeedInput,
		// Output is not available right now. New input is not required, call HaveOutput() again.
		TryAgainLater,
		// All output has been provided.
		EndOfData,
		// An internal decoder error occurred. Call GetError() to get the error code.
		Error
	};

	enum class ECSDCompatibility
	{
		// Current decoder configuration is fully capable to continue decoding with the new configuration.
		Compatible,
		// The decoder must be drained before it can continue with the new configuration.
		Drain,
		// The decoder must be drained and then ResetToCleanStart() before it can continue with the new configuration.
		DrainAndReset
	};

	struct FError
	{
		bool IsSet() const
		{ return Code || SdkCode || Message.Len(); }
		const FString& GetMessage() const
		{ return Message; }
		int32 GetCode() const
		{ return Code; }
		uint32 GetSdkCode() const
		{ return SdkCode; }

		FError& SetMessage(const FString& InMessage)
		{ Message = InMessage; return *this; }
		FError& SetCode(int32 InCode)
		{ Code = InCode; return *this; }
		FError& SetSdkCode(uint32 InSdkCode)
		{ SdkCode = InSdkCode; return *this; }

		FString Message;
		int32 Code = 0;
		uint32 SdkCode = 0;
	};

	enum class EType
	{
		None,
		Video,
		Audio,
		Subtitle
	};

	/**
	 * Returns the type of this decoder instance.
	 * This is useful to ensure that the correct type was indeed created and you did not get a different
	 * type of decoder from the factory if the format was ambiguous.
	 */
	virtual EType GetType() const = 0;

	/**
	 * Populates the provided map with decoder features or required options.
	 * See IElectraDecoderFeature and IElectraDecoderOption.
	 * Not all decoders will provide the same features.
	 */
	virtual void GetFeatures(TMap<FString, FVariant>& OutFeatures) const = 0;

	/**
	 * Returns the most recent error that you should retrieve when either method returns failure.
	 */
	virtual FError GetError() const = 0;

	/**
	 * Closes the decoder instance. This must be called prior to dropping the last reference of the decoder
	 * resulting in the call to its destructor.
	 * This may already close internally used resources.
	 * The decoder instance can no longer be used and all decoding and output related methods will return an error.
	 */
	virtual void Close() = 0;


	/**
	 * When a change in decoder specific data is detected, call this method to determine if the decoder
	 * can just continue decoding with the new CSD or if it needs to be drained or drained first and then reset.
	 */
	virtual ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) = 0;

	/**
	 * Tries to reset the decoder to a clean start state.
	 * If this fails and 'false' is returned the decoder must be destroyed and a new instance created.
	 */
	virtual bool ResetToCleanStart() = 0;

	/**
	 * Decodes one access unit.
	 * The data sent for decoding must form one complete decodable unit. Partial data is not permitted.
	 * You are not required to retain the access unit. The decoder will copy the data internally if required.
	 *
	 * Not every call will result in a new decoded output. It is possible that several input access units are
	 * required before an output will be made available, even with complete decodable units being provided.
	 *
	 * The return value indicates the following:
	 *  - None : No error, the access unit was accepted successfully.
	 *  - EndOfData : SendEndOfData() was called to drain the decoder to produce output for every access unit
	 *                that was delivered. All pending output must be retrieved before sending a new access unit.
	 *  - NoBuffer : The decoder can not accept new input at the moment unless pending output has been retrieved.
	 *               If there is no pending output this indicates some abnormal condition and the decoder should
	 *               be destroyed.
	 *  - Error : An error has occurred. Get the details by calling GetError(). The decoder will need to be
	 *            destroyed and recreated.
	 */
	virtual EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) = 0;

	/**
	 * Sends an end-of-data notification to have the decoder process all pending input and generate as
	 * much output as possible.
	 * All output must be retrieved by calling HaveOutput() and GetOutput() in sequence.
	 * Once all output has been retrieved new data can be provided to the decoder.
	 *
	 * The return value indicates the following:
	 *  - None : No error, the call was successful.
	 *  - EndOfData : This is returned when SendEndOfData() is called more than once while the decoder is
	 *                already processing the remaining input.
	 *  - NoBuffer : This value is not returned by this method.
	 *  - Error : An error has occurred. Get the details by calling GetError(). The decoder will need to be
	 *            destroyed and recreated.
	 */
	virtual EDecoderError SendEndOfData() = 0;

	/**
	 * Flushes the decoder to discard any pending input. No output will be generated.
	 * New data can be provided to the decoder immediately.
	 *
	 * The return value indicates the following:
	 *  - None : No error, the call was successful.
	 *  - EndOfData : This value is not returned by this method.
	 *  - NoBuffer : This value is not returned by this method.
	 *  - Error : An error has occurred. Get the details by calling GetError(). The decoder will need to be
	 *            destroyed and recreated.
	 */
	virtual EDecoderError Flush() = 0;

	/**
	 * Checks for available decoded output.
	 * Calling DecodeAccessUnit() may not produce any output at that moment.
	 * You need to check if output is available by calling this method.
	 * If new output is available it can be retrieved calling GetOutput().
	 * After a call to SendEndOfData() you need to call this method to
	 * perform decoding of any pending input.
	 *
	 * The return value indicates the following:
	 *  - Available : New output is available. It must be retrieved by calling GetOutput()
	 *  - NeedInput : There is no output available. More data must be provided to the
	 *                decoder through DecodeAccessUnit() or pending input be completed
	 *                by calling SendEndOfData().
	 *  - EndOfData : Indicates the last output had been returned after a call to
	 *                SendEndOfData(). This return value is returned only once. The next
	 *                call will return NeedInput.
	 *  - Error : An error has occurred. Get the details by calling GetError(). The decoder will need to be
	 *            destroyed and recreated.
	 */
	virtual EOutputStatus HaveOutput() = 0;

	/**
	 * Returns the next pending output.
	 * This call should be preceded by a call to HaveOutput() to check if output is
	 * available.
	 * Calling without checking first will return a nullptr when no output is available.
	 *
	 * NOTE: Ownership of the decoded output stays with the decoder.
	 *       You may hold on to the returned pointer but it is recommended the output
	 *       is processed immediately. If the output is not returned to the decoder
	 *       (by resetting the pointer) it may be possible for the decoder to be
	 *       unable to produce additional output.
	 */
	virtual TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() = 0;


	/**
	 * Creates a processor to pre-parse the decoder input bitstream.
	 */
	virtual TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() = 0;

	virtual void Suspend() = 0;
	virtual void Resume() = 0;
};

