// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * This namespace contains constant names used to describe a decoder feature or a requirement.
 * Features are a pair of FString giving the name and an FVariant containing the
 * typed value.
 */
namespace IElectraDecoderFeature
{
	/**
	 * FVariant(int32)
	 *
	 * Indicates the minimum number of decoded frames the application must be able to take from the decoder.
	 */
	const TCHAR* const MinimumNumberOfOutputFrames = TEXT("minimum_number_of_output_frames");

	/**
	 * FVariant(bool) : true - adaptive decoding supported, false - not supported
	 *
	 * Indicates whether or not the decoder is adaptive on input format changes, like
	 * different resolutions for video decoders.
	 * If the decoder is not adaptive then you need to perform end-of-stream handling
	 * like so:
	 *
	 *   SendEndOfData();		// indicate no further data will be sent
	 *   while(HaveOutput() == Available)
	 *   {
	 *       GetOutput();
	 *   }
	 *   Reset();
	 */
	const TCHAR* const IsAdaptive = TEXT("is_adaptive");

	/*
	 * FVariant(bool) : true - adaptive decoding supported, false - not supported
	 *
	 * Indicates if the decoder supports dropping output frames that we do not want to display.
	 * Such frames still need to be decoded to update the internal decoder state but the output
	 * is not needed and can already be discarded on the decoder level instead of being converted
	 * and passed up the pipeline where it will then be discarded.
	 */
	const TCHAR* const SupportsDroppingOutput = TEXT("supports_drop_output");

	/**
	 * FVariant(int32)
	 *    Value to add to the length of the decoding unit.
	 *    Usually 4 to include the 32 bit length value replacing
	 *    the start code in the length.
	 *    If the start code is NOT to be replaced with the length
	 *    then this element must NOT be returned as feature.
	 *    A value of 0 still indicates replacement!
	 *
	 * Indicates that the decoder wants format specific start codes in the input to
	 * be replaced with the length of the decoding unit.
	 * This is primarily for video decoders of the H.264 and H.265 family that use
	 * a 32 bit value of 0x00000001 to separate individual decoding units in a single
	 * input access unit. Some decoders want this value to be the length of the
	 * following decoding unit.
	 * The byte order is the same as the startcode, which is almost always big endian.
	 */
	const TCHAR* const StartcodeToLength = TEXT("startcode_to_length");

	/**
	 * FVariant(bool)
	 *    true - input data gets modified in place
	 *    false - input data remains unchanged
	 *
	 * If the input access unit is modified in place this should be set to true
	 * to have the higher up decoder layer create a copy of the data if necessary.
	 */
	const TCHAR* const ModifiesInputAUInPlace = TEXT("modifies_input_au_in_place");

	/**
	 * FVariant(FTimespan)
	 *    Time offset with which to feed decoder input earlier
	 *    than indicated by the decode timestamp.
	 *    This if typically used only by subtitle decoders that
	 *    want an access unit to be fed only just in time.
	 *    Such access units are sparse and should potentially not
	 *    be fed ahead of time.
	 *    The presence of this feature indicates the decoder wants
	 *    to be provided with a new access unit only just in time
	 *    with this value indicating how much earlier it wants it.
	 *    Absence of this feature indicates no preference.
	 */
	const TCHAR* const DecodeTimeOffset = TEXT("decode_time_offset");

	/**
	 * FVariant(bool) : true - media local time needed in decode options, false - not needed
	 *
	 * The decoder may need to media local time in addition to the absolute decoding time.
	 * This is primarily used for subtitle decoders in multiperiod content in which the
	 * subtitle timing starts at zero for each period.
	 * If local time is required it must be passed as an additional option to DecodeAccessUnit()
	 * (see IElectraDecoderOption::MediaLocalTime)
	 */
	const TCHAR* const NeedsMediaLocalTime = TEXT("needs_media_local_time");


	/**
	 * FVariant(bool) : true - the decoder must not be used during application background, false - not needed
	 *
	 * This tends to be more of a per-platform setting than a per-decoder-instance one, but it is
	 * more flexible to do it this way.
	 */
	const TCHAR* const MustBeSuspendedInBackground = TEXT("must_be_suspended_in_background");

	/**
	 * FVariant(bool) : true - the decoder may get lost and if it does it needs to be fed previous samples
	 *
	 * If the decoder is lost and a new internal instance replaces it, the new instance needs to be
	 * provided with previous samples starting with the most recent keyframe. This flag indicates that you
	 * need to hold on to those samples for this eventuality.
	 * When you then provide those samples again you need to indicate this by setting the
	 * `IsReplaySample` flag on all replay samples and additionally `IsLastReplaySample` on the last
	 * replay sample before returning to normal decoding.
	 * This allows the decoder to perform decoding optimizations if possible.
	 *
	 * NOTE: The decoder must not provide decoded output for these samples!
	 */
	const TCHAR* const NeedReplayDataOnDecoderLoss = TEXT("need_replay_data_on_decoder_loss");
};



/**
 * This namespace contains constant names used to describe an option to pass to a decoder
 * as an additional option to DecodeAccessUnit().
 * Options are a pair of FString giving the name and an FVariant containing the
 * typed value.
 */
namespace IElectraDecoderOption
{
	/**
	 * FVariant(FTimespan)
	 *
	 * Local time of the media corresponding to the absolute decoding time.
	 * See IElectraDecoderFeature::NeedsMediaLocalTime.
	 */
	const TCHAR* const MediaLocalTime = TEXT("media_local_time");

	/**
	 * FVariant(FTimespan)
	 *
	 *
	 */
	const TCHAR* const PresentationTimeOffset = TEXT("presentation_time_offset");

	/**
	 * FVariant(String)
	 *
	 *
	 */
	const TCHAR* const SideloadedID = TEXT("sideloaded_id");

	/**
	 * FVariant(ByteArray)
	 */
	const TCHAR* const CodecSpecificData = TEXT("csd");


	/**
	 * FVariant(ByteArray)
	 */
	const TCHAR* const DecoderConfigurationRecord = TEXT("dcr");
}



/**
 * This namespace contains constant names that describe an decoder format.
 */
namespace IElectraDecoderFormatInfo
{
	/**
	 * FVariant(bool) : true - The format uses only keyframes.
	 *
	 * Indicates that this format encodes every frame as a keyframe and no delta frames.
	 * This is used with video decoders only. Audio is assumed to be keyframe-only.
	 * The absence of this on a video decoder or a value of 'false' implies that delta frames are used.
	 */
	const TCHAR* const IsEveryFrameKeyframe = TEXT("is_only_keyframes");

	/**
	 * FVariant(String)
	 *
	 * A human readable string of the format to be decoded.
	 */
	const TCHAR* const HumanReadableFormatName = TEXT("human_readable_format_name");
}



/**
 * This namespace contains constant names that describe bitstream processor values.
 */
namespace IElectraDecoderBitstreamProcessorInfo
{
	/**
	 * FVariant(uint8)
	 *
	 * Number of luminance bits.
	 */
	const TCHAR* const NumBitsLuma = TEXT("num_bits_luma");

	/**
	 * FVariant(FCommonColorimetry)
	 *
	 * Colorimetry information in standard MPEG definition.
	 */
	const TCHAR* const CommonColorimetry = TEXT("common_colorimetry");

	/**
	 * FVariant(FCommonPictureTiming)
	 *
	 * Timing from a bitstream SEI message.
	 */
	const TCHAR* const CommonPictureTiming = TEXT("common_pic_timing");

	/**
	 * FVariant(FSEImastering_display_colour_volume)
	 *
	 * Mastering display colour volume from bitstream SEI message
	 */
	const TCHAR* const SeiMasteringDisplayColorVolume = TEXT("sei_mdcv");

	/**
	 * FVariant(FSEIcontent_light_level_info)
	 *
	 * Content light level info from bitstream SEI message
	 */
	const TCHAR* const SeiContentLightLeveInfo = TEXT("sei_clli");

	/**
	 * FVariant(FSEIalternative_transfer_characteristics)
	 *
	 * Alternate transfer characterisctics from bitstream SEI message
	 */
	const TCHAR* const SeiAlternateTransferCharacteristics = TEXT("sei_altc");

}
