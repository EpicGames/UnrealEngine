// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"

class IElectraDecoderAudioOutput : public IElectraDecoderOutput
{
public:
	virtual ~IElectraDecoderAudioOutput() = default;
	EType GetType() const override
	{
		return EType::Audio;
	}

	// Channel position as per ISO/IEC 23001-8 Table 7.
	enum class EChannelPosition
	{
		Invalid = -1,
		L=0, R, C, LFE, Ls, Rs, Lc, Rc,
		Lsr, Rsr, Cs, Lsd, Rsd, Lss, Rss, Lw,
		Rw, Lv, Rv, Cv, Lvr, Rvr, Cvr, Lvss,
		Rvss, Ts, LFE2, Lb, Rb, Cb, Lvs, Rvs,
		Undefined = 128,
		Disabled = 255,
		// No specified position.
		Unspec0 = 256, Unspec1, Unspec2, Unspec3, Unspec4, Unspec5, Unspec6, Unspec7, Unspec8, Unspec9, Unspec10, Unspec11, Unspec12, Unspec13, Unspec14, Unspec15,
		Unspec16, Unspec17, Unspec18, Unspec19, Unspec20, Unspec21, Unspec22, Unspec23, Unspec24, Unspec25, Unspec26, Unspec27, Unspec28, Unspec29, Unspec30, Unspec31
	};

	enum class ESampleFormat
	{
		Int16,
		Float
	};

	/**
	 * Returns the number of decoded channels.
	 */
	virtual int32 GetNumChannels() const = 0;

	/**
	 * Returns the decoded sampling rate.
	 * This may be different from what was known about the input.
	 */
	virtual int32 GetSampleRate() const = 0;

	/**
	 * Returns the number of decoded samples per channel.
	 */
	virtual int32 GetNumFrames() const = 0;

	/**
	 * If samples are interleaved they appear in memory alternatingly per channel.
	 * Eg. L,R,L,R,L,R,...
	 * If they are not interleaved there will be separate buffers containing samples
	 * of a single channel each.
	 */
	virtual bool IsInterleaved() const = 0;
	
	/**
	 * Returns the position of the decoded channel.
	 * If a position is not inferred or described by the format it is set to `Undefined`.
	 * If a channel failed to decode it may be included as `Disabled` with the sample
	 * data set to silence.
	 */
	virtual EChannelPosition GetChannelPosition(int32 InChannelNumber) const = 0;

	/**
	 * Returns the format of the decoded samples.
	 * The data pointers shall be cast to the appropriate type.
	 */
	virtual ESampleFormat GetSampleFormat() const = 0;

	/**
	 * Returns the number of bytes per decoded sample.
	 * This typically corresponds to the size of the decoded format,
	 * ie 2 for Int16 and 4 for Float.
	 */
	virtual int32 GetBytesPerSample() const = 0;

	/**
	 * Returns the number of bytes per decoded frame.
	 * For channel interleaved data this is typically
	 * GetNumChannels() * GetBytesPerSample() while per-channel data
	 * typically returns the same as GetBytesPerSample()
	 */
	virtual int32 GetBytesPerFrame() const = 0;

	/**
	 * Returns a pointer to the start of the decoded data for the specified channel.
	 * This can be used for interleaved data as well although you
	 * would normally ask for the data pointer of channel 0 to get
	 * the base address of the interleaved data.
	 */
	virtual const void* GetData(int32 InChannelNumber) const = 0;
};
