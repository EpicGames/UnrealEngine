// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"


#if ELECTRA_DECODERS_HAVE_PLATFORM_DEFAULTS
#include COMPILED_PLATFORM_HEADER(ElectraDecoderPlatformOutputHandleTypes.h)

#else

enum class EElectraDecoderPlatformOutputHandleType
{
};

#endif


struct FElectraVideoDecoderOutputCropValues
{
	// Number of rows at the top that must not be displayed.
	int32 Top = 0;

	// Number of columns at the left that must not be displayed.
	int32 Left = 0;

	// Number of rows at the bottom that must not be displayed.
	int32 Bottom = 0;

	// Number of columns at the right that must not be displayed.
	int32 Right = 0;
};

class IElectraDecoderVideoOutputCopyResources;


class IElectraDecoderVideoOutput : public IElectraDecoderOutput
{
public:
	virtual ~IElectraDecoderVideoOutput() = default;
	EType GetType() const override
	{
		return EType::Video;
	}

	enum class EOutputType
	{
		// Output this output.
		Output,

		// Do not output this. Either the decoder did not produce an actual output for the given
		// input, or the output has decoding errors that makes it unusable.
		DoNotOutput
	};

	/**
	 * Returns the state of this output.
	 * Depending on the input bitstream the decoder might not have been able to produce
	 * a valid output or it may be partially corrupted.
	 * If the output should not be used for display `DoNotOutput` is returned.
	 */
	virtual EOutputType GetOutputType() const = 0;

	/**
	 * Returns the active number of horizontal pixels.
	 * This has potential cropping values applied and does not include any
	 * pitch requirements.
	 */
	virtual int32 GetWidth() const = 0;

	/**
	 * Returns the active number of vertical pixels.
	 * This has potential cropping values applied and does not include any
	 * pitch requirements.
	 */
	virtual int32 GetHeight() const = 0;

	/**
	 * Returns the number of decoded horizontal pixels.
	 * This value may be different from GetWidth() since it does not take any
	 * cropping values into account.
	 * Note that this represents the actual dimension of the data returned, which
	 * may differ from GetWidth() + cropping offsets!
	 * (e.g. if the data is returned as RGBA pixels, but represents a YUYV 4:2:2 format)
	 */
	virtual int32 GetDecodedWidth() const = 0;

	/**
	 * Returns the number of decoded vertical pixels.
	 * This value may be greater than GetHeight() since it does not take any
	 * cropping values into account.
	 * Note that this represents the actual dimension of the data returned, which
	 * may differ from GetWidth() + cropping offsets!
	 * (e.g. if the data is returned as RGBA pixels, but represents a YUYV 4:2:2 format)
	 */
	virtual int32 GetDecodedHeight() const = 0;

	/**
	 * Returns the cropping values.
	 * This is useful in combination with GetDecodedWidth() and GetDecodedHeight()
	 * that give you the number of pixels created by the decoder.
	 */
	virtual FElectraVideoDecoderOutputCropValues GetCropValues() const = 0;

	/**
	 * Returns the aspect ratio w value.
	 */
	virtual int32 GetAspectRatioW() const = 0;

	/**
	 * Returns the aspect ratio h value.
	 */
	virtual int32 GetAspectRatioH() const = 0;

	/**
	 * Returns the numerator of the frame rate, if known.
	 * If unknown this returns 0.
	 */
	virtual int32 GetFrameRateNumerator() const = 0;

	/**
	 * Returns the denominator of the frame rate, if known.
	 * If unknown this returns 0.
	 */
	virtual int32 GetFrameRateDenominator() const = 0;

	/**
	 * Returns the number of bits. Usually 8, 10, or 12.
	 */
	virtual int32 GetNumberOfBits() const = 0;

	/**
	 * Returns additional values specific to the decoder and format that are used
	 * in handling this output in a platform specific way.
	 */
	virtual void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const = 0;

	/**
	 * Returns the platform specific decoder output handle, if any.
	 * This value is specific to the decoder being used.
	 */
	virtual void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const = 0;

	enum class EImageCopyResult
	{
		Succeeded,
		Failed,
		NotSupported
	};

	/**
	 * Asks to create a copy of the image into a platform specific structure.
	 * Parameters for the operation are provided by a platform specific implementation of
	 * IElectraDecoderVideoOutputCopyResources provided to this method through which
	 * relevant parameters (eg. a device or texture handle, etc.) can be obtained.
	 */
	virtual EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const = 0;
};
