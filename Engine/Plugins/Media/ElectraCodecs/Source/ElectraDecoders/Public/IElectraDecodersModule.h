// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "IElectraCodecRegistry.h"

/**
 * Interface for the `ElectraDecodersModule` module.
 */
class IElectraDecodersModule : public IModuleInterface
{
public:
	virtual ~IElectraDecodersModule() = default;

	virtual void RegisterDecodersWithCodecFactory(IElectraCodecRegistry* InCodecFactoryToRegisterWith) = 0;
};
