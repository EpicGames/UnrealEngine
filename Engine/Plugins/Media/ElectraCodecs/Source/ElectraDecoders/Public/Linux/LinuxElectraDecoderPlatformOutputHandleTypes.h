// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "IElectraTextureSampleEncoding.h"

enum class EElectraDecoderPlatformOutputHandleType
{
	LibavDecoderDecodedImage,			// Get POD pointer to a ILibavDecoderDecodedImage
	ImageBuffers						// IElectraDecoderVideoOutputImageBuffers interface
};


class FElectraDecoderOutputSync final
{
public:
	FElectraDecoderOutputSync() = default;
	~FElectraDecoderOutputSync() = default;
};

class IElectraDecoderVideoOutputImageBuffers
{
public:
	// Return the 4cc of the codec.
	virtual uint32 GetCodec4CC() const = 0;

	// Returns the number of separate image buffers making up the frame.
	virtual int32 GetNumberOfBuffers() const = 0;

	// Returns the n'th image buffer as CPU data buffer.
	virtual TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer as GPU texture resource.
	virtual void* GetBufferTextureByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer's GPU sync object to signal texture data as being ready to be read
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const = 0;

	// Returns the n'th image buffer format
	virtual EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer encoding
	virtual EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer pitch
	virtual int32 GetBufferPitchByIndex(int32 InBufferIndex) const = 0;
};
