// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#include "ParameterDictionary.h"

class IElectraDecoderVideoOutput;
class FElectraTextureSample;

class FElectraDecodersPlatformResourcesLinux
{
public:
	struct IDecoderPlatformResource;
	static ELECTRADECODERS_API IDecoderPlatformResource* CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions);
	static ELECTRADECODERS_API void ReleasePlatformVideoResource(void* InOwnerHandle, IDecoderPlatformResource* InHandleToRelease);

	static ELECTRADECODERS_API bool SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, IDecoderPlatformResource* InPlatformResource);
};

using FElectraDecodersPlatformResources = FElectraDecodersPlatformResourcesLinux;
