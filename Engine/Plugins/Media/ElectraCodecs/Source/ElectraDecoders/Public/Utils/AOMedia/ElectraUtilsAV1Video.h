// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>
#include "Utilities/ElectraBitstream.h"

namespace ElectraDecodersUtil
{
	namespace AV1Video
	{

		/**
		 * AV1 Codec ISO Media File Format Binding
		 *
		 * see: https://aomediacodec.github.io/av1-isobmff/#av1codecconfigurationbox-syntax
		 */
		class FAV1CodecConfigurationRecord
		{
		public:
			struct FColorInfo
			{
				uint8 monochrome = 0;
				uint8 chromaSubsamplingX = 1;
				uint8 chromaSubsamplingY = 1;
				uint8 chromaSamplingPosition = 0;
				uint8 colorPrimaries = 1;
				uint8 transferCharacteristics = 1;
				uint8 matrixCoefficients = 1;
				uint8 videoFullRangeFlag = 0;
			};

			ELECTRADECODERS_API bool Parse(const TConstArrayView<uint8>& InDCR);

			inline const TArray<uint8>& GetRawData() const
			{ return RawData; }
			inline const TArray<uint8>& GetCodecSpecificData() const
			{ return CSD; }

			inline uint8 GetProfile() const
			{ return Profile; }
			inline uint8 GetLevel() const
			{ return Level; }
			inline uint8 GetTier() const
			{ return Tier; }
			inline uint8 GetBitDepth() const
			{ return BitDepth; }

			inline const FColorInfo& GetColorInfo() const
			{ return ColorInfo; }

			inline FString GetCodecSpecifierRFC6381() const
			{
				FString rfc(FString::Printf(TEXT("av01.%d.%02d%c.%02d.%d.%d%d%d.%02d.%02d.%02d.%d"), Profile, Level, Tier?TCHAR('H'):TCHAR('M'), BitDepth,
					ColorInfo.monochrome, ColorInfo.chromaSubsamplingX, ColorInfo.chromaSubsamplingY, ColorInfo.chromaSamplingPosition,	ColorInfo.colorPrimaries, ColorInfo.transferCharacteristics, ColorInfo.matrixCoefficients, ColorInfo.videoFullRangeFlag));
				if (rfc.EndsWith(TEXT(".0.110.01.01.01.0")))
				{
					rfc.LeftChopInline(17);
				}
				return rfc;
			}

			ELECTRADECODERS_API FString GetFormatInfo();

			ELECTRADECODERS_API static FString GetFormatInfo(uint32 InProfile, uint32 InLevel);
		private:
			TArray<uint8> RawData;
			TArray<uint8> CSD;
			FColorInfo ColorInfo;
			uint8 Profile = 0;
			uint8 Level = 0;
			uint8 Tier = 0;
			uint8 BitDepth = 0;
			uint8 initial_presentation_delay_present = 0;
			uint8 initial_presentation_delay_minus_one = 0;
		};



		class FBitstreamReaderAV1 : public Electra::FBitstreamReader
		{
		public:
			FBitstreamReaderAV1() = default;
			FBitstreamReaderAV1(const uint8* InData, uint64 InDataSize)
				: Electra::FBitstreamReader(InData, InDataSize)
			{ }

			uint32 uvlc()
			{
				uint32 leadingZeros = 0;
				while(!GetBits(1))
				{
					if (++leadingZeros == 32)
					{
						return 0xffffffffU;
					}
				}
				uint32 v = GetBits(leadingZeros);
				return v + ((1U << leadingZeros) - 1U);
			}

			uint8 GetByte()
			{
				check(IsByteAligned());
				return (DataSize - BytePosition) ? Data[BytePosition++] : 0U;
			}

			uint32 leb128(uint32& OutNumBytesRead)
			{
				uint64 v = 0;
				uint32 Shift = 0;
				bool bMore = false;
				OutNumBytesRead = 0;
				do
				{
					uint8 lb = GetByte();
					bMore = !!(lb & 0x80);
					v |= (uint64)(lb & 0x7fU) << Shift;
					Shift += 7;
					++OutNumBytesRead;
				} while(bMore && Shift < 56);
				if (v > 0xffffffffU || bMore)
				{
					// Not good.
					return 0;
				}
				return (uint32)(v & 0xffffffffU);
			}
			uint32 leb128()
			{
				uint32 nr;
				return leb128(nr);
			}
		};


		enum class EOBUType
		{
			Reserved0 = 0,
			OBU_SEQUENCE_HEADER,
			OBU_TEMPORAL_DELIMITER,
			OBU_FRAME_HEADER,
			OBU_TILE_GROUP,
			OBU_METADATA,
			OBU_FRAME,
			OBU_REDUNDANT_FRAME_HEADER,
			OBU_TILE_LIST,
			Reserved9 = 9,
			OBU_PADDING = 15
		};

		enum class EMetadataType
		{
			Reserved0 = 0,
			METADATA_TYPE_HDR_CLL,
			METADATA_TYPE_HDR_MDCV,
			METADATA_TYPE_SCALABILITY,
			METADATA_TYPE_ITUT_T35,
			METADATA_TYPE_TIMECODE,
			UnregisteredUserPrivate6 = 6,
			Reserved32 = 32
		};

		enum class EColorPrimaries : uint8
		{
			Undefined = 0,
			CP_BT_709 = 1,
			CP_UNSPECIFIED = 2,
			CP_BT_470_M = 4,
			CP_BT_470_B_G,
			CP_BT_601,
			CP_SMPTE_240,
			CP_GENERIC_FILM,
			CP_BT_2020,
			CP_XYZ,
			CP_SMPTE_431,
			CP_SMPTE_432,
			CP_EBU_3213 = 22
		};

		enum class ETransferCharacteristics : uint8
		{
			TC_RESERVED_0 = 0,
			TC_BT_709,
			TC_UNSPECIFIED,
			TC_RESERVED_3,
			TC_BT_470_M,
			TC_BT_470_B_G,
			TC_BT_601,
			TC_SMPTE_240,
			TC_LINEAR,
			TC_LOG_100,
			TC_LOG_100_SQRT10,
			TC_IEC_61966,
			TC_BT_1361,
			TC_SRGB,
			TC_BT_2020_10_BIT,
			TC_BT_2020_12_BIT,
			TC_SMPTE_2084,
			TC_SMPTE_428,
			TC_HLG
		};

		enum class EMatrixCoefficients : uint8
		{
			MC_IDENTITY = 0,
			MC_BT_709,
			MC_UNSPECIFIED,
			MC_RESERVED_3,
			MC_FCC,
			MC_BT_470_B_G,
			MC_BT_601,
			MC_SMPTE_240,
			MC_SMPTE_YCGCO,
			MC_BT_2020_NCL,
			MC_BT_2020_CL,
			MC_SMPTE_2085,
			MC_CHROMAT_NCL,
			MC_CHROMAT_CL,
			MC_ICTCP
		};

		enum class EChromaSamplePosition : uint8
		{
			CSP_UNKNOWN = 0,
			CSP_VERTICAL,
			CSP_COLOCATED,
			CSP_RESERVED
		};

		struct FMetadata_hdr_cll
		{
			uint16 max_cll = 0;
			uint16 max_fall = 0;
			bool ELECTRADECODERS_API ParseFrom(FBitstreamReaderAV1& InReader);
		};

		struct FMetadata_hdr_mdvc
		{
			uint16 primary_chromaticity_x[3] { 0 };	// CIE 1931; 0=red, 1=green, 2=blue
			uint16 primary_chromaticity_y[3] { 0 };
			uint16 white_point_chromaticity_x = 0;
			uint16 white_point_chromaticity_y = 0;
			uint32 luminance_max = 0;				// a 24.8 fixed-point maximum luminance, represented in candelas per square meter
			uint32 luminance_min = 0;				// a 18.14 fixed-point minimum luminance, represented in candelas per square meter
			bool ELECTRADECODERS_API ParseFrom(FBitstreamReaderAV1& InReader);
		};


		struct FSequenceHeader
		{
			uint8 seq_profile = 0;
			uint8 still_picture = 0;
			uint8 reduced_still_picture_header = 0;
			uint8 timing_info_present_flag = 0;
			struct FTimingInfo
			{
				uint32 num_units_in_display_tick = 0;
				uint32 time_scale = 0;
				uint8 equal_picture_interval = 0;
				uint32 num_ticks_per_picture_minus_1 = 0;
			};
			FTimingInfo timing_info;

			uint8 decoder_model_info_present_flag = 0;
			struct FDecoderModelInfo
			{
				uint8 buffer_delay_length_minus_1 = 0;
				uint32 num_units_in_decoding_tick = 0;
				uint8 buffer_removal_time_length_minus_1 = 0;
				uint8 frame_presentation_time_length_minus_1 = 0;
			};
			FDecoderModelInfo decoder_model_info;

			uint8 initial_display_delay_present_flag = 0;
			uint8 operating_points_cnt_minus_1 = 0;

			struct FOperatingPoint
			{
				uint16 operating_point_idc = 0;
				uint8 seq_level_idx = 0;
				uint8 seq_tier = 0;
				uint8 decoder_model_present_for_this_op = 0;
				uint8 initial_display_delay_present_for_this_op = 0;
				uint8 initial_display_delay_minus_1 = 0;
			};
			FOperatingPoint operating_points[32];
			struct FOperatingParametersInfo
			{
				uint32 decoder_buffer_delay = 0;
				uint32 encoder_buffer_delay = 0;
				uint8 low_delay_mode_flag = 0;
			};
			FOperatingParametersInfo operating_parameters_info[32];

			uint8 frame_width_bits_minus_1 = 0;
			uint8 frame_height_bits_minus_1 = 0;
			uint32 max_frame_width_minus_1 = 0;
			uint32 max_frame_height_minus_1 = 0;
			uint8 frame_id_numbers_present_flag = 0;
			uint8 delta_frame_id_length_minus_2 = 0;
			uint8 additional_frame_id_length_minus_1 = 0;
			uint8 use_128x128_superblock = 0;
			uint8 enable_filter_intra = 0;
			uint8 enable_intra_edge_filter = 0;
			uint8 enable_interintra_compound = 0;
			uint8 enable_masked_compound = 0;
			uint8 enable_warped_motion = 0;
			uint8 enable_dual_filter = 0;
			uint8 enable_order_hint = 0;
			uint8 enable_jnt_comp = 0;
			uint8 enable_ref_frame_mvs = 0;
			uint8 seq_choose_screen_content_tools = 0;
			uint8 seq_force_screen_content_tools = 0;
			uint8 seq_force_integer_mv = 0;
			uint8 seq_choose_integer_mv = 0;
			uint8 order_hint_bits_minus_1 = 0;
			uint8 OrderHintBits = 0;
			uint8 enable_superres = 0;
			uint8 enable_cdef = 0;
			uint8 enable_restoration = 0;

			struct FColorConfig
			{
				uint8 high_bitdepth = 0;
				uint8 twelve_bit = 0;
				uint8 mono_chrome = 0;
				uint8 color_description_present_flag = 0;
				EColorPrimaries color_primaries = EColorPrimaries::CP_UNSPECIFIED;
				ETransferCharacteristics transfer_characteristics = ETransferCharacteristics::TC_UNSPECIFIED;
				EMatrixCoefficients matrix_coefficients = EMatrixCoefficients::MC_UNSPECIFIED;
				uint8 color_range = 0;
				uint8 subsampling_x = 0;
				uint8 subsampling_y = 0;
				EChromaSamplePosition chroma_sample_position = EChromaSamplePosition::CSP_UNKNOWN;
				uint8 separate_uv_delta_q = 0;
			};
			FColorConfig color_config;

			uint8 film_grain_params_present = 0;


			bool ELECTRADECODERS_API ParseFrom(FBitstreamReaderAV1& InReader);
		};

	} // namespace AV1Video
} // namespace ElectraDecodersUtil
