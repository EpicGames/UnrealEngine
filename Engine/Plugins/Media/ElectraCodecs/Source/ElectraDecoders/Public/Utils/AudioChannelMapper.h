// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Templates/SharedPointer.h"

#include "IElectraDecoderOutputAudio.h"

#define UE_API ELECTRADECODERS_API

namespace Electra
{
	/**
	 * Class to map audio channels from some decoder-specific output order into the
	 * order expected by FMediaAudioResampler.
	 *
	 * Channels that are not supported by UE will be ignored and are silent.
	 * No effort to mix them into other channels is made.
	 * As fas as media playback is concerned we handle the most common channel layouts
	 * only, like mono, stereo, 5.1 and 7.1.
	 */
	class FAudioChannelMapper
	{
	public:
/*
		// Channel position as per ISO/IEC 23001-8 Table 7.
		enum class IElectraDecoderAudioOutput::EChannelPosition
		{
			L, R, C, LFE, Ls, Rs, Lc, Rc,
			Lsr, Rsr, Cs, Lsd, Rsd, Lss, Rss, Lw,
			Rw, Lv, Rv, Cv, Lvr, Rvr, Cvr, Lvss,
			Rvss, Ts, LFE2, Lb, Rb, Cb, Lvs, Rvs,
			Disabled = 255
		};
*/
		struct FSourceLayout
		{
			IElectraDecoderAudioOutput::EChannelPosition ChannelPosition = IElectraDecoderAudioOutput::EChannelPosition::Disabled;
		};

		UE_API FAudioChannelMapper();
		UE_API ~FAudioChannelMapper();

		// Initializes with an ISO/IEC 23001-8 channel configuration value from Table 7 (compatible with ISO/IEC 14496-3:2009/Amd 4)
		UE_API bool Initialize(int32 NumBytesPerSample, uint32 ChannelConfiguration);

		// Initializes with ISO/IEC 23001-8 channel positions.
		UE_API bool Initialize(int32 NumBytesPerSample, TArrayView<const FSourceLayout> ChannelPositions);

		UE_API bool IsInitialized() const;

		UE_API void Reset();

		// Gets the number of output channels mapped to the UE internal layout.
		UE_API int32 GetNumTargetChannels() const;

		// Maps the supported input channels to the output channels in UE internal layout.
		UE_API void MapChannels(void* OutputBuffer, uint32 OutputBufferSize, const void* InputBuffer, uint32 InputBufferSize, int32 NumSamplesPerChannel) const;

		UE_API bool Initialize(TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> InSampleBlock);

	private:
		struct FTargetSource
		{
			IElectraDecoderAudioOutput::EChannelPosition ChannelPosition = IElectraDecoderAudioOutput::EChannelPosition::Disabled;
			int32 FirstOffset = 0;
			int32 Stride = 0;
		};

		static bool CanBeCopiedDirectly(TArrayView<const FSourceLayout> ChannelPositions);
		bool CanBeCopiedDirectly(TSharedPtr<IElectraDecoderAudioOutput, ESPMode::ThreadSafe> InSampleBlock);
		int32 MatchesResamplerLayout(TArray<FTargetSource>& InputSources);
		void CreateResamplerChannelMapping(int32 ResamplerChannelIndex, const TArray<FTargetSource>& InSources);

		TArray<FTargetSource> TargetSources;
		int32 BytesPerSample = 0;
		bool bCanCopyDirectly = false;
	};
} // namespace Electra

#undef UE_API
