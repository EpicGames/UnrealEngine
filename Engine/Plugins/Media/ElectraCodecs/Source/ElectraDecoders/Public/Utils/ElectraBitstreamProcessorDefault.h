// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"


class FElectraDecoderBitstreamProcessorDefault : public IElectraDecoderBitstreamProcessor
{
public:
	static TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> Create()
	{ return MakeShareable<>(new FElectraDecoderBitstreamProcessorDefault); }

	virtual ~FElectraDecoderBitstreamProcessorDefault();
	bool WillModifyBitstreamInPlace() override;
	void Clear() override;
	EProcessResult ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData) override;
	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI) override;
	FString GetLastError() override;

private:
	ELECTRADECODERS_API FElectraDecoderBitstreamProcessorDefault();
	class FImpl;
	TPimplPtr<FImpl> Impl;
};
