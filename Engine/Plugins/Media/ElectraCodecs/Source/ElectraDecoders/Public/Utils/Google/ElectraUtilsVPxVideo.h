// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>

namespace ElectraDecodersUtil
{
	namespace VPxVideo
	{

		/**
		 * VP Codec ISO Media File Format Binding
		 *
		 * see: https://www.webmproject.org/vp9/mp4/#vp-codec-configuration-box
		 *
		 * This is the same configuration for both VP8 and VP9.
		 */
		class FVPCodecConfigurationRecord
		{
		public:
			struct FColorInfo
			{
				uint8 chromaSubsampling = 1;
				uint8 colourPrimaries = 1;
				uint8 transferCharacteristics = 1;
				uint8 matrixCoefficients = 1;
				uint8 videoFullRangeFlag = 0;
			};

			ELECTRADECODERS_API bool Parse(int32 InVP8Or9, const TConstArrayView<uint8>& InDCR);

			inline const TArray<uint8>& GetRawData() const
			{ return RawData; }
			inline const TArray<uint8>& GetCodecSpecificData() const
			{ return CSD; }

			inline int32 GetVPxVersion() const
			{ return VPx; };

			inline uint8 GetProfile() const
			{ return Profile; }
			inline uint8 GetLevel() const
			{ return Level; }
			inline uint8 GetBitDepth() const
			{ return BitDepth; }

			inline const FColorInfo& GetColorInfo() const
			{ return ColorInfo; }

			inline FString GetCodecSpecifierRFC6381() const
			{
				FString Common = FString::Printf(TEXT("vp0%d.%02d.%02d.%02d"), VPx, Profile, Level, BitDepth);
				if (ColorInfo.chromaSubsampling != 1 || ColorInfo.colourPrimaries != 1 || ColorInfo.transferCharacteristics != 1 || ColorInfo.matrixCoefficients != 1 || ColorInfo.videoFullRangeFlag != 0)
				{
					Common += FString::Printf(TEXT(".%02d.%02d.%02d.%02d.%02d"), ColorInfo.chromaSubsampling, ColorInfo.colourPrimaries, ColorInfo.transferCharacteristics, ColorInfo.matrixCoefficients, ColorInfo.videoFullRangeFlag);
				}
				return Common;
			}

			ELECTRADECODERS_API FString GetFormatInfo();

			ELECTRADECODERS_API static FString GetFormatInfo(uint32 InProfile, uint32 InLevel);

		private:
			TArray<uint8> RawData;
			TArray<uint8> CSD;
			FColorInfo ColorInfo;
			int32 VPx = 0;
			uint8 Profile = 0;
			uint8 Level = 0;
			uint8 BitDepth = 0;
		};


		struct FVP9UncompressedHeader
		{
			uint8 frame_marker = 0;
			uint8 profile_low_bit = 0;
			uint8 profile_high_bit = 0;
			uint8 show_existing_frame = 0;
			uint8 frame_to_show_map_idx = 0;
			uint8 frame_type = 0;
			uint8 show_frame = 0;
			uint8 error_resilient_mode = 0;

			uint8 intra_only = 0;
			uint8 reset_frame_context = 0;

			uint8 ten_or_twelve_bit = 0;
			uint8 color_space = 0;
			uint8 color_range = 0;
			uint8 subsampling_x = 0;
			uint8 subsampling_y = 0;

			uint32 frame_width = 0;
			uint32 frame_height = 0;

			uint32 render_width = 0;
			uint32 render_height = 0;

			enum EColorSpace
			{
				CS_Unknown = 0,
				CS_BT_601 = 1,		// Rec. ITU-R BT.601-7
				CS_BT_709 = 2,		// Rec. ITU-R BT.709-6
				CS_SMPTE_170 = 3,	// SMPTE-170
				CS_SMPTE_240 = 4,	// SMPTE-240
				CS_BT_2020 = 5,		// Rec. ITU-R BT.2020-2
				CS_Reserved = 6,
				CS_RGB = 7			// sRGB (IEC 61966-2-1)
			};

			enum EColorRange
			{
				/*
					For BitDepth equals 8:
						Y is between 16 and 235 inclusive.
						U and V are between 16 and 240 inclusive.
					For BitDepth equals 10:
						Y is between 64 and 940 inclusive.
						U and V are between 64 and 960 inclusive.
					For BitDepth equals 12:
						Y is between 256 and 3760.
						U and V are between 256 and 3840 inclusive.
				*/
				CR_StudioSwing = 0,

				// No restriction on Y, U, V values.
				CR_FullSwing = 1
			};

			enum ESubSampling
			{
				SS_YUV_444 = 0,
				SS_YUV_440 = 1,
				SS_YUV_422 = 2,
				SS_YUV_420 = 3
			};

			bool IsKeyframe() const
			{ return frame_type == 0; }
			int32 GetProfile() const
			{ return (profile_high_bit << 1) + profile_low_bit; }
			int32 GetBitDepth() const
			{ return GetProfile() >= 2 ? (ten_or_twelve_bit ? 12 : 10) : 8; }
			EColorSpace GetColorSpace() const
			{ return (EColorSpace) color_space; }
			EColorRange GetColorRange() const
			{ return(EColorRange) color_range; }
			ESubSampling GetSubSampling() const
			{ return (ESubSampling)((subsampling_y << 1) + subsampling_x); }
		};

		bool ELECTRADECODERS_API GetVP9SuperframeSizes(TArray<uint32>& OutSizes, const void* Data, int64 Size);
		bool ELECTRADECODERS_API ParseVP9UncompressedHeader(FVP9UncompressedHeader& OutHeader, const void* Data, int64 Size);






		struct FVP8UncompressedHeader
		{
			uint8 key_frame = 0;
			uint8 version = 0;
			uint8 is_experimental = 0;
			uint8 show_frame = 0;
			uint16 horizontal_size_code = 0;
			uint16 vertical_size_code = 0;

			bool IsKeyframe() const
			{ return key_frame == 0; }
		};

		bool ELECTRADECODERS_API ParseVP8UncompressedHeader(FVP8UncompressedHeader& OutHeader, const void* Data, int64 Size);

	} // namespace VPxVideo

} // namespace ElectraDecodersUtil
