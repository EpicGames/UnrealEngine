// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "CodecTypeFormat.h"
#include "MP4Boxes.h"

namespace ElectraDecodersUtil
{
	namespace MP4
	{
		ELECTRADECODERS_API bool GetTrackFormatInfo(FString& OutMessage, Electra::FCodecTypeFormat& OutCodecInfo, Electra::FDRMTypeFormat& OutDRMInfo, Electra::FDecoderInformation& OutDecoderInfo, const TSharedPtr<MP4Boxes::FMP4BoxTRAK, ESPMode::ThreadSafe>& InTrack, uint32 InTrackID, bool bInCheckIfDecodable);

	} // namespace MP4

} // namespace ElectraDecodersUtil
