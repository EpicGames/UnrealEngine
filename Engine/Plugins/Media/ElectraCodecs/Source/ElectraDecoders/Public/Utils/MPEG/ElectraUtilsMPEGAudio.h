// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>

#define UE_API ELECTRADECODERS_API

namespace ElectraDecodersUtil
{
	namespace MPEG
	{

		class FESDescriptor
		{
		public:
			ELECTRADECODERS_API bool Parse(const TConstArrayView<uint8>& InESDS);

			inline const TArray<uint8>& GetCodecSpecificData() const
			{ return CSD; }

			inline uint32 GetBufferSize() const
			{ return BufferSize; }

			inline uint32 GetMaxBitrate() const
			{ return MaxBitrate; }

			inline uint32 GetAvgBitrate() const
			{ return AvgBitrate; }

			// See http://mp4ra.org/#/object_types
			enum class FObjectTypeID
			{
				Unknown = 0,
				Text_Stream = 8,
				MPEG4_Video = 0x20,
				H264 = 0x21,
				H264_ParameterSets = 0x22,
				H265 = 0x23,
				MPEG4_Audio = 0x40,		// ISO/IEC 14496-3
				MPEG2_AAC_LC = 0x67,	// ISO/IEC 13818-7
				MPEG2_Audio = 0x69,		// ISO/IEC 13818-3
				MPEG1_Audio = 0x6b		// ISO/IEC 11172-3
			};
			enum class FStreamType
			{
				Unknown = 0,
				VisualStream = 4,
				AudioStream = 5
			};

			inline FObjectTypeID GetObjectTypeID() const
			{ return ObjectTypeID; }

			inline FStreamType GetStreamType() const
			{ return StreamTypeID; }

		private:
			TArray<uint8> RawData;
			TArray<uint8> CSD;
			FObjectTypeID ObjectTypeID = FObjectTypeID::Unknown;
			FStreamType StreamTypeID = FStreamType::Unknown;
			uint32 BufferSize = 0;
			uint32 MaxBitrate = 0;
			uint32 AvgBitrate = 0;
			uint16 ESID = 0;
			uint16 DependsOnStreamESID = 0;
			uint8 StreamPriority = 0;
			bool bDependsOnStream = false;
		};


		class FAACDecoderConfigurationRecord
		{
		public:
			UE_API FAACDecoderConfigurationRecord();

			UE_API bool ParseFrom(const void* Data, int64 Size);
			UE_API void Reset();
			UE_API const TArray<uint8>& GetCodecSpecificData() const;

			inline bool Parse(const TConstArrayView<uint8>& InDCR)
			{ return ParseFrom(InDCR.GetData(), InDCR.Num()); }

			inline void SetRawData(const TArray<uint8>& InRawData)
			{ RawData = InRawData; }

			inline const TArray<uint8>& GetRawData() const
			{ return RawData; }

			inline FString GetCodecSpecifierRFC6381() const
			{ return FString::Printf(TEXT("mp4a.40.%u"), ExtAOT ? ExtAOT : AOT); }

			UE_API FString GetFormatInfo() const;

			int32 SBRSignal;
			int32 PSSignal;
			uint32 ChannelConfiguration;
			uint32 SamplingFrequencyIndex;
			uint32 SamplingRate;
			uint32 ExtSamplingFrequencyIndex;
			uint32 ExtSamplingFrequency;
			uint32 AOT;
			uint32 ExtAOT;
		private:
			TArray<uint8> CodecSpecificData;
			TArray<uint8> RawData;
		};


		namespace AACUtils
		{
			struct ADTSheader
			{
				uint16 SyncWord = 0;
				uint8 Version = 0;
				uint8 Layer = 0;
				uint8 ProtectionAbsent = 0;
				uint8 Profile = 0;
				uint8 SamplingFrequencyIndex = 0;
				uint8 PrivateStream = 0;
				uint8 ChannelConfiguration = 0;
				uint8 Originality = 0;
				uint8 Home = 0;
				uint8 Copyrighted = 0;
				uint8 CopyrightStart = 0;
				uint16 FrameLength = 0;
				uint16 BufferFullness = 0;
				uint8 NumFrames = 0;
				uint16 CRCifPresent = 0;
				//
				uint16 HeaderSize = 0;
			};

			int32 ELECTRADECODERS_API GetNumberOfChannelsFromChannelConfiguration(uint32 InChannelConfiguration);
			int32 ELECTRADECODERS_API GetSampleRateFromFrequenceIndex(uint32 InSamplingFrequencyIndex);

			bool ELECTRADECODERS_API ParseADTSHeader(ADTSheader& OutHeader, const TConstArrayView<uint8>& InHeaderData);
		}


		namespace UtilsMPEG123
		{
			// Raw bit methods
			bool ELECTRADECODERS_API HasValidSync(uint32 InFrameHeader);
			int32 ELECTRADECODERS_API GetVersionId(uint32 InFrameHeader);			// 0=MPEG2.5, 1=reserved, 2=MPEG2 (ISO/IEC 13818-3), 3=MPEG1 (ISO/IEC 11172-3)
			int32 ELECTRADECODERS_API GetLayerIndex(uint32 InFrameHeader);			// 0=reserved, 1=Layer III, 2=Layer II, 3=Layer I
			int32 ELECTRADECODERS_API GetBitrateIndex(uint32 InFrameHeader);		// 0-15
			int32 ELECTRADECODERS_API GetSamplingRateIndex(uint32 InFrameHeader);	// 0-3
			int32 ELECTRADECODERS_API GetChannelMode(uint32 InFrameHeader);			// 0-3
			int32 ELECTRADECODERS_API GetNumPaddingBytes(uint32 InFrameHeader);		// 0 or 1


			// Convenience methods
			int32 ELECTRADECODERS_API GetVersion(uint32 InFrameHeader);				// 1=MPEG 1, 2=MPEG2, 3=MPEG2.5, 0=reserved
			int32 ELECTRADECODERS_API GetLayer(uint32 InFrameHeader);				// 0=reserved, 1=Layer I, 2=Layer II, 3=Layer III
			int32 ELECTRADECODERS_API GetBitrate(uint32 InFrameHeader);				// Bitrate in kbps, -1=invalid
			int32 ELECTRADECODERS_API GetSamplingRate(uint32 InFrameHeader);		// Sampling rate, -1=invalid
			int32 ELECTRADECODERS_API GetChannelCount(uint32 InFrameHeader);		// 1=mono, 2=stereo

			int32 ELECTRADECODERS_API GetSamplesPerFrame(uint32 InFrameHeader);		// number of samples encoded in the frame
			int32 ELECTRADECODERS_API GetFrameSize(uint32 InFrameHeader, int32 InForcedPadding=-1);		// number of bytes in the packet if CBR encoded, 0=could not calculate
		}

	} // namespace MPEG

} // namespace ElectraDecodersUtil

#undef UE_API
