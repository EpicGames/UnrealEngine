// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>

#include "ElectraDecodersUtils.h"

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		struct FSEIMessage
		{
			enum EPayloadType
			{
				PT_pic_timing = 1,
				PT_user_data_registered_itu_t_t35 = 4,
				// PT_tone_mapping_info = 23,							// not expected to be used (application custom)
				PT_time_code = 136,
				PT_mastering_display_colour_volume = 137,
				// PT_chroma_resampling_filter_hint = 140,				// not expected to be used
				// PT_knee_function_info = 141,							// not expected to be used (application custom)
				// PT_colour_remapping_info = 142,						// not expected to be used (application custom)
				PT_content_light_level_info = 144,
				PT_alternative_transfer_characteristics = 147,
				PT_ambient_viewing_environment = 148,
				PT_content_colour_volume = 149
			};
			uint32 PayloadType = ~uint32(0);
			TArray<uint8> Message;
		};
		enum class ESEIStreamType
		{
			H264,
			H265
		};
		bool ELECTRADECODERS_API ExtractSEIMessages(TArray<FSEIMessage>& OutMessages, const void* InBitstream, uint64 InBitstreamLength, ESEIStreamType StreamType, bool bIsPrefixSEI);


		struct FSEImastering_display_colour_volume
		{
			uint16 display_primaries_x[3] { 0 };
			uint16 display_primaries_y[3] { 0 };
			uint16 white_point_x = 0;
			uint16 white_point_y = 0;
			uint32 max_display_mastering_luminance = 0;
			uint32 min_display_mastering_luminance = 0;
			bool operator==(const FSEImastering_display_colour_volume& rhs) const
			{
				return display_primaries_x[0] == rhs.display_primaries_x[0] &&
					   display_primaries_y[0] == rhs.display_primaries_y[0] &&
					   display_primaries_x[1] == rhs.display_primaries_x[1] &&
					   display_primaries_y[1] == rhs.display_primaries_y[1] &&
					   display_primaries_x[2] == rhs.display_primaries_x[2] &&
					   display_primaries_y[2] == rhs.display_primaries_y[2] &&
					   white_point_x == rhs.white_point_x &&
					   white_point_y == rhs.white_point_y &&
					   max_display_mastering_luminance == rhs.max_display_mastering_luminance &&
					   min_display_mastering_luminance == rhs.min_display_mastering_luminance;
			}
			bool operator!=(const FSEImastering_display_colour_volume& rhs) const
			{
				return !operator==(rhs);
			}
		};
		bool ELECTRADECODERS_API ParseFromSEIMessage(FSEImastering_display_colour_volume& OutMDCV, const FSEIMessage& InMessage);
		bool ELECTRADECODERS_API ParseFromMDCVBox(FSEImastering_display_colour_volume& OutMDCV, const TConstArrayView<uint8>& InMDCVBox);
		TArray<uint8> ELECTRADECODERS_API BuildMDCVBox(const FSEImastering_display_colour_volume& InFromDisplayColorVolume);

		struct FSEIcontent_light_level_info
		{
			uint16 max_content_light_level = 0;
			uint16 max_pic_average_light_level = 0;
			bool operator==(const FSEIcontent_light_level_info& rhs) const
			{
				return max_content_light_level == rhs.max_content_light_level && max_pic_average_light_level == rhs.max_pic_average_light_level;
			}
			bool operator!=(const FSEIcontent_light_level_info& rhs) const
			{
				return !operator==(rhs);
			}
		};
		bool ELECTRADECODERS_API ParseFromSEIMessage(FSEIcontent_light_level_info& OutCLLI, const FSEIMessage& InMessage);
		bool ELECTRADECODERS_API ParseFromCOLLBox(FSEIcontent_light_level_info& OutCLLI, const TConstArrayView<uint8>& InCOLLBox);
		bool ELECTRADECODERS_API ParseFromCLLIBox(FSEIcontent_light_level_info& OutCLLI, const TConstArrayView<uint8>& InCLLIBox);
		TArray<uint8> ELECTRADECODERS_API BuildCLLIBox(const FSEIcontent_light_level_info& InFromContentLightLevel);

		struct FSEIalternative_transfer_characteristics
		{
			uint8 preferred_transfer_characteristics = 0;
		};
		bool ELECTRADECODERS_API ParseFromSEIMessage(FSEIalternative_transfer_characteristics& OutATC, const FSEIMessage& InMessage);

		struct FSEIambient_viewing_environment
		{
			uint32 ambient_illuminance = 0;
			uint16 ambient_light_x = 0;
			uint16 ambient_light_y = 0;
		};

		struct FSEIcontent_colour_volume
		{
			uint8 ccv_cancel_flag = 0;
			uint8 ccv_persistence_flag = 0;
			uint8 ccv_primaries_present_flag = 0;
			uint8 ccv_min_luminance_value_present_flag = 0;
			uint8 ccv_max_luminance_value_present_flag = 0;
			uint8 ccv_avg_luminance_value_present_flag = 0;
			int32 ccv_primaries_x[3] { 0 };
			int32 ccv_primaries_y[3] { 0 };
			uint32 ccv_min_luminance_value = 0;
			uint32 ccv_max_luminance_value = 0;
			uint32 ccv_avg_luminance_value = 0;
		};


		struct FCommonColorimetry
		{
			uint8 colour_primaries = 2;
			uint8 transfer_characteristics = 2;
			uint8 matrix_coeffs = 2;
			uint8 video_full_range_flag = 0;
			uint8 video_format = 5;
		};
		bool ELECTRADECODERS_API ParseFromCOLRBox(FCommonColorimetry& OutColorimetry, const TConstArrayView<uint8>& InCOLRBox);
		TArray<uint8> ELECTRADECODERS_API BuildCOLRBox(const FCommonColorimetry& InFromColorimetry);


		struct FCommonPictureTiming
		{
			// Calculated value as per:
			//   clockTimestamp = ( ( hH * 60 + mM ) * 60 + sS ) * time_scale + nFrames * ( num_units_in_tick * ( 1 + nuit_field_based_flag ) ) + tOffset,
			// can only be valid when there is timing information.
			int64 clockTimestamp = 0;
			// Values from a pic_timing() SEI in H.264 or from a time_code() SEI in H.265
			uint32 num_units_in_tick = 0;			// from the SPS
			uint32 time_scale = 0;					// from the SPS
			int32 time_offset = 0;
			uint16 n_frames = 0;
			uint8 timing_info_present_flag = 0;		// from the SPS
			uint8 clock_timestamp_flag = 0;
			uint8 ct_type = 0;
			uint8 nuit_field_based_flag = 0;
			uint8 counting_type = 0;
			uint8 full_timestamp_flag = 0;
			uint8 discontinuity_flag = 0;
			uint8 cnt_dropped_flag = 0;
			uint8 seconds_value = 0;
			uint8 minutes_value = 0;
			uint8 hours_value = 0;
			uint8 FromH26x = 0;		// last digit of the codec this comes from. 4=H.264, 5=H.265, etc. In case values need different interpretation.
		};

	} // namespace MPEG

} // namespace ElectraDecodersUtil
