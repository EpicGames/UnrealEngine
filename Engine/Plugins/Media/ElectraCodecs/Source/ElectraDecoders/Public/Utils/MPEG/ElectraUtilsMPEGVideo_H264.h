// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "Misc/Timespan.h"
#include "Utilities/ElectraBitstream.h"
#include "ElectraDecodersTypes.h"

#define UE_API ELECTRADECODERS_API

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		namespace H264
		{
			struct FNaluInfo
			{
				uint64 Offset;		// Offset into the bitstream where the startcode begins
				uint64 Size;		// Number of bytes, including the nal unit type, that follow the startcode
				uint8 Type;			// Nal unit type
				uint8 RefIdc;		// RefIdc
				uint8 UnitLength;	// Length of the start code in bytes (3 or 4)
			};
			bool ELECTRADECODERS_API ParseBitstreamForNALUs(TArray<FNaluInfo>& OutNALUs, const uint8* InBitstream, uint64 InBitstreamLength);

			struct FRBSP
			{
				FRBSP(uint64 InDataSize)
				{
					Data.SetNumUninitialized((int32)InDataSize);
				}
				void TrimToBitstream(const Electra::FBitstreamReader& InBSR)
				{
					if (InBSR.IsByteAligned())
					{
						Data.SetNum((int32) InBSR.GetBytePosition());
					}
					else
					{
						const uint8 Mask = 0xffU << (8 - InBSR.GetBitPosition());
						Data.SetNum((int32) InBSR.GetBytePosition() + 1);
						Data.Last() &= Mask;
					}
				}
				TArray<uint8> Data;
				TArray<uint64> RemovedAtSrc;
			};
			uint64 ELECTRADECODERS_API EBSPtoRBSP(TArray<uint64>& OutRemovedInBufPos, uint8* OutBuf, const uint8* InBuf, uint64 InNumBytes);
			TUniquePtr<FRBSP> ELECTRADECODERS_API MakeRBSP(const uint8* InData, uint64 InDataSize);

			class FBitstreamReaderH264 : public Electra::FBitstreamReader
			{
			public:
				FBitstreamReaderH264() = default;
				FBitstreamReaderH264(const uint8* InData, uint64 InDataSize)
					: Electra::FBitstreamReader(InData, InDataSize)
				{ }

				uint32 ue_v()
				{
					int32 lz = -1;
					for(uint32 b=0; b==0 && GetRemainingBits(); ++lz)
					{
						b = GetBits(1);
					}
					if (lz >= 0 && GetRemainingBits() >= lz)
					{
						return ((1 << lz) | GetBits(lz)) - 1;
					}
					return 0;
				}
				int32 se_v()
				{
					uint32 c = ue_v();
					return c & 1 ? int32((c + 1) >> 1) : -int32((c + 1) >> 1);
				}

				const uint8* GetData() const
				{
					return static_cast<const uint8*>(GetRemainingData());
				}

				bool more_rbsp_data()
				{
					// There is definitely more data available unless we are in the last byte.
					uint64 NumBitsRemaining = GetRemainingBits();
					if (NumBitsRemaining > 16 || PeekBits(1) == 0)
					{
						return true;
					}
					uint32 Remainder = PeekBits(NumBitsRemaining);
					// Find the position of the last set bit.
					uint64 Pos = 0;
					for(; Pos<NumBitsRemaining && (Remainder & 1) == 0; ++Pos)
					{
						Remainder >>= 1;
					}
					return NumBitsRemaining > Pos+1;
				}
			};

			struct FSequenceParameterSet
			{
				FSequenceParameterSet()
				{
					Reset();
				}
				void Reset()
				{
					ProcessedBitstream.Empty();
					// See default values in the comments of the respective members
					profile_idc = 0;
					constraint_set0_flag = 0;
					constraint_set1_flag = 0;
					constraint_set2_flag = 0;
					constraint_set3_flag = 0;
					constraint_set4_flag = 0;
					constraint_set5_flag = 0;
					level_idc = 0;
					seq_parameter_set_id = 0;
					chroma_format_idc = 1;
					separate_colour_plane_flag = 0;
					bit_depth_luma_minus8 = 0;
					bit_depth_chroma_minus8 = 0;
					qpprime_y_zero_transform_bypass_flag = 0;
					seq_scaling_matrix_present_flag = 0;
					FMemory::Memzero(seq_scaling_list_present_flag);
					FMemory::Memset(ScalingList4x4, 16);
					FMemory::Memset(ScalingList8x8, 16);
					log2_max_frame_num_minus4 = 0;
					pic_order_cnt_type = 0;
					log2_max_pic_order_cnt_lsb_minus4 = 0;
					delta_pic_order_always_zero_flag = 0;
					offset_for_non_ref_pic = 0;
					offset_for_top_to_bottom_field = 0;
					num_ref_frames_in_pic_order_cnt_cycle = 0;
					FMemory::Memzero(offset_for_ref_frame);
					max_num_ref_frames = 0;
					gaps_in_frame_num_value_allowed_flag = 0;
					pic_width_in_mbs_minus1 = 0;
					pic_height_in_map_units_minus1 = 0;
					frame_mbs_only_flag = 0;
					mb_adaptive_frame_field_flag = 0;
					direct_8x8_inference_flag = 0;
					frame_cropping_flag = 0;
					frame_crop_left_offset = 0;
					frame_crop_right_offset = 0;
					frame_crop_top_offset = 0;
					frame_crop_bottom_offset = 0;
					vui_parameters_present_flag = 0;
					aspect_ratio_info_present_flag = 0;
					aspect_ratio_idc = 0;
					sar_width = 0;
					sar_height = 0;
					overscan_info_present_flag = 0;
					overscan_appropriate_flag = 0;
					video_signal_type_present_flag = 0;
					video_format = 5;
					video_full_range_flag = 0;
					colour_description_present_flag = 0;
					colour_primaries = 2;
					transfer_characteristics = 2;
					matrix_coefficients = 2;
					chroma_loc_info_present_flag = 0;
					chroma_sample_loc_type_top_field = 0;
					chroma_sample_loc_type_bottom_field = 0;
					timing_info_present_flag = 0;
					num_units_in_tick = 0;
					time_scale = 0;
					fixed_frame_rate_flag = 0;
					nal_hrd_parameters_present_flag = 0;
					vcl_hrd_parameters_present_flag = 0;
					low_delay_hrd_flag = 1;
					pic_struct_present_flag = 0;
					bitstream_restriction_flag = 0;
					motion_vectors_over_pic_boundaries_flag = 1;
					max_bytes_per_pic_denom = 2;
					max_bits_per_mb_denom = 1;
					log2_max_mv_length_horizontal = 15;
					log2_max_mv_length_vertical = 15;
					max_num_reorder_frames = 0;
					max_dec_frame_buffering = 0;
					nal_hrd_parameters.Reset();
					vcl_hrd_parameters.Reset();

					ExpectedDeltaPerPicOrderCntCycle = 0;
				}
				ELECTRADECODERS_API int32 GetMaxDPBSize() const;
				ELECTRADECODERS_API int32 GetDPBSize() const;
				ELECTRADECODERS_API int32 GetWidth() const;
				ELECTRADECODERS_API int32 GetHeight() const;
				ELECTRADECODERS_API void GetDisplaySize(int32& OutWidth, int32& OutHeight) const;
				ELECTRADECODERS_API void GetDisplaySize(uint32& OutWidth, uint32& OutHeight) const;
				ELECTRADECODERS_API void GetCrop(int32& OutLeft, int32& OutRight, int32& OutTop, int32& OutBottom) const;
				ELECTRADECODERS_API void GetAspect(int32& OutSarW, int32& OutSarH) const;
				ELECTRADECODERS_API FFractionalValue GetTiming() const;
				ELECTRADECODERS_API FString GetFormatInfo() const;
				ELECTRADECODERS_API static FString GetFormatInfo(uint32 InProfileIDC, uint32 InLevelIDC, uint32 InConstraintFlags);

				TArray<uint8> ProcessedBitstream;

				uint32 profile_idc;											// u(8)
				uint8 constraint_set0_flag;									// u(1)
				uint8 constraint_set1_flag;									// u(1)
				uint8 constraint_set2_flag;									// u(1)
				uint8 constraint_set3_flag;									// u(1)
				uint8 constraint_set4_flag;									// u(1)
				uint8 constraint_set5_flag;									// u(1)
				uint32 level_idc;											// u(8)
				uint32 seq_parameter_set_id;								// ue(v), 0-31
				uint32 chroma_format_idc;									// ue(v), 0-3, default 1
				uint8 separate_colour_plane_flag;							// u(1)
				uint32 bit_depth_luma_minus8;								// ue(v), 0-6
				uint32 bit_depth_chroma_minus8;								// ue(v), 0-6
				uint8 qpprime_y_zero_transform_bypass_flag;					// u(1)
				uint8 seq_scaling_matrix_present_flag;						// u(1)
				uint8 seq_scaling_list_present_flag[12];					// u(1)
				uint8 ScalingList4x4[6][16];
				uint8 ScalingList8x8[6][64];
				uint32 log2_max_frame_num_minus4;							// ue(v), 0-12
				uint32 pic_order_cnt_type;									// ue(v), 0-2
				uint32 log2_max_pic_order_cnt_lsb_minus4;					// ue(v), 0-12
				uint8 delta_pic_order_always_zero_flag;						// u(1)
				int32 offset_for_non_ref_pic;								// se(v)
				int32 offset_for_top_to_bottom_field;						// se(v)
				uint32 num_ref_frames_in_pic_order_cnt_cycle;				// ue(v), 0-255
				int32 offset_for_ref_frame[255];							// se(v)
				uint32 max_num_ref_frames;									// ue(v), 0-MaxDpbFrames
				uint8 gaps_in_frame_num_value_allowed_flag;					// u(1)
				uint32 pic_width_in_mbs_minus1;								// ue(v)
				uint32 pic_height_in_map_units_minus1;						// ue(v)
				uint8 frame_mbs_only_flag;									// u(1)
				uint8 mb_adaptive_frame_field_flag;							// u(1)
				uint8 direct_8x8_inference_flag;							// u(1)
				uint8 frame_cropping_flag;									// u(1)
				uint32 frame_crop_left_offset;								// ue(v), 0 - ( PicWidthInSamplesL / CropUnitX ) - ( frame_crop_right_offset + 1 )
				uint32 frame_crop_right_offset;								// ue(v)
				uint32 frame_crop_top_offset;								// ue(v), 0 - ( 16 * FrameHeightInMbs / CropUnitY ) - ( frame_crop_bottom_offset + 1 )
				uint32 frame_crop_bottom_offset;							// ue(v)
				uint8 vui_parameters_present_flag;							// u(1)
				// VUI Parameters
				uint8 aspect_ratio_info_present_flag;						// u(1)
				uint32 aspect_ratio_idc;									// u(8), 0-16, 255 (17-254 reserved)
				uint32 sar_width;											// u(16)
				uint32 sar_height;											// u(16)
				uint8 overscan_info_present_flag;							// u(1)
				uint8 overscan_appropriate_flag;							// u(1)
				uint8 video_signal_type_present_flag;						// u(1)
				uint32 video_format;										// u(3), 0-5 (6 & 7 reserved), default 5
				uint8 video_full_range_flag;								// u(1)
				uint8 colour_description_present_flag;						// u(1)
				uint32 colour_primaries;									// u(8), 0-12, 22 (13-21 & 23-255 reserved), default 2
				uint32 transfer_characteristics;							// u(8), 0-2, 4-18 (3 & 19-255 reserved), default 2
				uint32 matrix_coefficients;									// u(8), 0-2, 4-14 (3 & 15-255 reserved), default 2
				uint8 chroma_loc_info_present_flag;							// u(1)
				uint32 chroma_sample_loc_type_top_field;					// ue(v), 0-5, default 0
				uint32 chroma_sample_loc_type_bottom_field;					// ue(v), 0-5, default 0
				uint8 timing_info_present_flag;								// u(1)
				uint32 num_units_in_tick;									// u(32), >0
				uint32 time_scale;											// u(32), >0
				uint8 fixed_frame_rate_flag;								// u(1)
				uint8 nal_hrd_parameters_present_flag;						// u(1)
				uint8 vcl_hrd_parameters_present_flag;						// u(1)
				uint8 low_delay_hrd_flag;									// u(1), default 1, must be 0 if fixed_frame_rate is 1
				uint8 pic_struct_present_flag;								// u(1)
				uint8 bitstream_restriction_flag;							// u(1)
				uint8 motion_vectors_over_pic_boundaries_flag;				// u(1), default 1
				uint32 max_bytes_per_pic_denom;								// ue(v), 0-16, default 2
				uint32 max_bits_per_mb_denom;								// ue(v), 0-16, default 1
				uint32 log2_max_mv_length_horizontal;						// ue(v), 0-15, default 15
				uint32 log2_max_mv_length_vertical;							// ue(v), 0-15, default 15
				uint32 max_num_reorder_frames;								// ue(v), 0-max_dec_frame_buffering,
																			//	defaults to: if profile_idc in [44,86,100,110,122,144] and constraint_set3_flag==1 then 0
																			//               otherwise MaxDpbFrames.
				uint32 max_dec_frame_buffering;								// ue(v), >= max_num_ref_frames
				struct FHRDParameters
				{
					FHRDParameters()
					{
						Reset();
					}
					void Reset()
					{
						cpb_cnt_minus1 = 0;
						bit_rate_scale = 0;
						cpb_size_scale = 0;
						FMemory::Memzero(bit_rate_value_minus1);
						FMemory::Memzero(cpb_size_value_minus1);
						FMemory::Memzero(cbr_flag);
						initial_cpb_removal_delay_length_minus1 = 23;
						cpb_removal_delay_length_minus1 = 23;
						dpb_output_delay_length_minus1 = 23;
						time_offset_length = 24;
					}
					uint32 cpb_cnt_minus1;									// ue(v), 0-31, if low_delay_hrd_flag==1 then 0, default 0
					uint32 bit_rate_scale;									// u(4)
					uint32 cpb_size_scale;									// u(4)
					uint32 bit_rate_value_minus1[32];						// ue(v), 0-2^32-2
					uint32 cpb_size_value_minus1[32];						// ue(v), 0-2^32-2
					uint8 cbr_flag[32];										// u(1)
					uint32 initial_cpb_removal_delay_length_minus1;			// u(5), default 23
					uint32 cpb_removal_delay_length_minus1;					// u(5), default 23
					uint32 dpb_output_delay_length_minus1;					// u(5), default 23
					uint32 time_offset_length;								// u(5), default 24
				};
				FHRDParameters nal_hrd_parameters;
				FHRDParameters vcl_hrd_parameters;
				// Calculated values
				int32 ExpectedDeltaPerPicOrderCntCycle;
			};

			struct FPictureParameterSet
			{
				FPictureParameterSet()
				{
					Reset();
				}
				void Reset()
				{
					pic_parameter_set_id = 0;
					seq_parameter_set_id = 0;
					entropy_coding_mode_flag = 0;
					bottom_field_pic_order_in_frame_present_flag = 0;
					num_slice_groups_minus1 = 0;
					slice_group_map_type = 0;
					FMemory::Memzero(run_length_minus1);
					FMemory::Memzero(top_left);
					FMemory::Memzero(bottom_right);
					slice_group_change_direction_flag = 0;
					slice_group_change_rate_minus1 = 0;
					pic_size_in_map_units_minus1 = 0;
					slice_group_id.Empty();
					num_ref_idx_l0_default_active_minus1 = 0;
					num_ref_idx_l1_default_active_minus1 = 0;
					weighted_pred_flag = 0;
					weighted_bipred_idc = 0;
					pic_init_qp_minus26 = 0;
					pic_init_qs_minus26 = 0;
					chroma_qp_index_offset = 0;
					deblocking_filter_control_present_flag = 0;
					constrained_intra_pred_flag = 0;
					redundant_pic_cnt_present_flag = 0;
					transform_8x8_mode_flag = 0;
					pic_scaling_matrix_present_flag = 0;
					FMemory::Memzero(pic_scaling_list_present_flag);
					FMemory::Memzero(ScalingList4x4);
					FMemory::Memzero(ScalingList8x8);
					second_chroma_qp_index_offset = 0;
				}
				uint32 pic_parameter_set_id;							// ue(v), 0-255
				uint32 seq_parameter_set_id;							// ue(v), 0-31
				uint8 entropy_coding_mode_flag;							// u(1)
				uint8 bottom_field_pic_order_in_frame_present_flag;		// u(1)
				uint32 num_slice_groups_minus1;							// ue(v), 0-7 (specified in Annex A, G and J)
				uint32 slice_group_map_type;							// ue(v), 0-6
				uint32 run_length_minus1[8];							// ue(v), 0 - PicSizeInMapUnits-1
				uint32 top_left[8];										// ue(v), <= bottom_right[i]
				uint32 bottom_right[8];									// ue(v), < PicSizeInMapUnits
				uint8 slice_group_change_direction_flag;				// u(1)
				uint32 slice_group_change_rate_minus1;					// ue(v), 0 - PicSizeInMapUnits-1
				uint32 pic_size_in_map_units_minus1;					// ue(v), == PicSizeInMapUnits-1
				TArray<uint8> slice_group_id;							// u(v), 0 - num_slice_groups_minus1
				uint32 num_ref_idx_l0_default_active_minus1;			// ue(v), 0-31
				uint32 num_ref_idx_l1_default_active_minus1;			// ue(v), 0-31
				uint8 weighted_pred_flag;								// u(1)
				uint32 weighted_bipred_idc;								// u(2), 0-2
				int32 pic_init_qp_minus26;								// se(v), -(26 + QpBdOffsetY) to +25
				int32 pic_init_qs_minus26;								// se(v), -26 - +25
				int32 chroma_qp_index_offset;							// se(v), -12 - +12
				uint8 deblocking_filter_control_present_flag;			// u(1)
				uint8 constrained_intra_pred_flag;						// u(1)
				uint8 redundant_pic_cnt_present_flag;					// u(1)
				uint8 transform_8x8_mode_flag;							// u(1)
				uint8 pic_scaling_matrix_present_flag;					// u(1)
				uint8 pic_scaling_list_present_flag[12];				// u(1)
				uint8 ScalingList4x4[6][16];
				uint8 ScalingList8x8[6][64];
				int32 second_chroma_qp_index_offset;					// se(v), -12 - +12
			};

			struct FSliceHeader
			{
				uint32 first_mb_in_slice = 0;							// ue(v)
				uint32 slice_type = 0;									// ue(v), 0-9
				uint32 pic_parameter_set_id = 0;						// ue(v)
				uint32 colour_plane_id = 0;								// u(2)
				uint32 frame_num = 0;									// u(v)
				uint8 field_pic_flag = 0;								// u(1)
				uint8 bottom_field_flag = 0;							// u(1)
				uint32 idr_pic_id = 0;									// ue(v)
				uint32 pic_order_cnt_lsb = 0;							// u(v)
				int32 delta_pic_order_cnt_bottom = 0;					// se(v), default 0
				int32 delta_pic_order_cnt[2] {0,0};						// se(v), default 0 for both
				uint32 redundant_pic_cnt = 0;							// ue(v), 0-127
				uint8 direct_spatial_mv_pred_flag = 0;					// u(1)
				uint8 num_ref_idx_active_override_flag = 0;				// u(1)
				uint32 num_ref_idx_l0_active_minus1 = 0;				// ue(v), 0-15 for frame coded pictures
				uint32 num_ref_idx_l1_active_minus1 = 0;				// ue(v), 0-15 for frame coded pictures
				uint32 cabac_init_idc = 0;								// ue(v), 0-2
				int32 slice_qp_delta = 0;								// se(v)
				uint8 sp_for_switch_flag = 0;							// u(1)
				int32 slice_qs_delta = 0;								// se(v)
				uint32 disable_deblocking_filter_idc = 0;				// ue(v), 0-2
				int32 slice_alpha_c0_offset_div2 = 0;					// se(v), -6 - +6, default 0
				int32 slice_beta_offset_div2 = 0;						// se(v), -6 - +6, default 0
				uint32 slice_group_change_cycle = 0;					// u(v)

				// ref_pic_list_modification
				struct FRefPicModification
				{
					uint32 modification_of_pic_nums_idc = 0;			// ue(v), 0-3
					uint32 abs_diff_pic_num_minus1 = 0;					// ue(v)
					uint32 long_term_pic_num = 0;						// ue(v)
				};
				uint8 ref_pic_list_modification_flag_l0 = 0;			// u(1)
				uint8 ref_pic_list_modification_flag_l1 = 0;			// u(1)
				TArray<FRefPicModification> RefPicListModifications[2];

				// pred_weight_table
				struct PredWeightTable_t
				{
					TArray<int32> luma_weight_l;						// se(v), -128 - + 127
					TArray<int32> luma_offset_l;						// se(v), -128 - + 127
					TArray<int32> chroma_weight_l[2];					// se(v), -128 - + 127
					TArray<int32> chroma_offset_l[2];					// se(v), -128 - + 127
				};
				uint32 luma_log2_weight_denom = 0;						// ue(v), 0-7
				uint32 chroma_log2_weight_denom = 0;					// ue(v), 0-7
				PredWeightTable_t PredWeightTable0;
				PredWeightTable_t PredWeightTable1;

				// dec_ref_pic_marking
				uint8 no_output_of_prior_pic_flag = 0;					// u(1)
				uint8 long_term_reference_flag = 0;						// u(1)
				uint8 adaptive_ref_pic_marking_mode_flag = 0;			// u(1)
				struct MemoryManagementControl_t
				{
					uint32 memory_management_control_operation = 0;		// ue(v)
					uint32 difference_of_pic_nums_minus1 = 0;			// ue(v)
					uint32 long_term_pic_num = 0;						// ue(v)
					uint32 long_term_frame_idx = 0;						// ue(v)
					uint32 max_long_term_frame_idx_plus1 = 0;			// ue(v)
				};
				TArray<MemoryManagementControl_t> MemoryManagementControl;
			};

			struct FOutputFrameInfo
			{
				int32 IndexInBuffer = -1;
				FTimespan PTS;
				uint64 UserValue0 = 0;
				uint64 UserValue1 = 0;
				bool bDoNotOutput = false;
			};

			class FSlicePOCVars
			{
			public:
				struct FReferenceFrameListEntry
				{
					FOutputFrameInfo UserFrameInfo;
					int32 FramePOC = 0;
					int32 TopPOC = 0;
					int32 BottomPOC = 0;
					int32 DPBIndex = -1;
					uint32 FrameNum = 0;
					int32 PicNum = 0;
					int32 LongTermFrameIndex = 0;
					int32 LongTermPicNum = 0;
					bool bIsLongTerm = false;
					bool bIsNonExisting = false;
					bool operator == (const FReferenceFrameListEntry& rhs) const
					{ return DPBIndex == rhs.DPBIndex && bIsLongTerm == rhs.bIsLongTerm; }
					bool operator != (const FReferenceFrameListEntry& rhs) const
					{ return ! operator == (rhs); }
				};

				FSlicePOCVars()
				{
					Reset();
				}

				// Call this to get the current short- and long-term reference frames.
				UE_API void GetCurrentReferenceFrames(TArray<FReferenceFrameListEntry>& OutCurrentReferenceFrames);

				// Call this per slice to get the list of references needed for decoding the slice.
				UE_API bool GetReferenceFrameLists(TArray<FReferenceFrameListEntry>& OutReferenceFrameList0, TArray<FReferenceFrameListEntry>& OutReferenceFrameList1, const FSliceHeader& InSliceHeader);

				// Call this on the first slice of an image to be decoded.
				UE_API bool BeginFrame(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet, const FPictureParameterSet& InPictureParameterSet);
				// Call this next to see if there are any missing frames between this and the last.
				UE_API bool HandleMissingFrames(TArray<FOutputFrameInfo>& OutOutputFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos, uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet);
				// Update the POC values for the current frame.
				UE_API bool UpdatePOC(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet);
				// Undo the POC value updates in case the decoding process could not run.
				UE_API void UndoPOCUpdate();

				int32 GetFramePOC() const
				{ return CurrentPOC.FramePOC; }
				int32 GetTopPOC() const
				{ return CurrentPOC.TopPOC; }
				int32 GetBottomPOC() const
				{ return CurrentPOC.BottomPOC; }

				// Call this when decoding of the entire frame (all slices) is done.
				UE_API bool EndFrame(TArray<FOutputFrameInfo>& OutOutputFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos, const FOutputFrameInfo& InOutputFrameInfo, uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, bool bIsNonExisting);

				// Call this to get all pending frame infos and clear the structure.
				UE_API void Flush(TArray<FOutputFrameInfo>& OutRemainingFrameInfos, TArray<FOutputFrameInfo>& OutUnrefFrameInfos);

				// Call this to reset this structure.
				UE_API void Reset();

				FString GetLastError() const
				{
					return LastErrorMsg;
				}
			private:
				struct FSmallestPOC
				{
					int32 Pos = -1;
					int32 POC = 0x7fffffff;
				};

				struct FFrameInDPBInfo : public FReferenceFrameListEntry
				{
					int32 FrameNumWrap = 0;
					bool bIsUsedForReference = false;
					bool bIsLongTermReference = false;
					bool bHasBeenOutput = false;
				};

				UE_API void UpdatePOCInternal(uint8 InNalUnitType, uint8 InNalRefIdc, const FSliceHeader& InSliceHeader, const FSequenceParameterSet& InSequenceParameterSet);
				UE_API void UpdateRefLists();
				UE_API FSmallestPOC GetSmallestPOC();
				UE_API bool ReorderRefPicList(TArray<FReferenceFrameListEntry>& InOutReferenceFrameList, const FSliceHeader& InSliceHeader, int32 InListNum);
				UE_API void FlushDPB(TArray<FOutputFrameInfo>& InOutInfos, TArray<FOutputFrameInfo>& InUnrefInfos);
				UE_API bool SetLastError(const FString& InLastErrorMsg);

				bool bLastHadMMCO5 = false;

				struct FPOCValues
				{
					FPOCValues()
					{
						Reset();
					}
					void Reset()
					{
						pic_order_cnt_msb = 0;
						prev_pic_order_cnt_msb = 0;
						prev_pic_order_cnt_lsb = 0;
						FrameNumOffset = 0;
						PrevFrameNumOffset = 0;
						FramePOC = 0;
						TopPOC = 0;
						BottomPOC = 0;
						prev_frame_num = 0;
					}
					// POC mode 0
					int32 pic_order_cnt_msb = 0;
					int32 prev_pic_order_cnt_msb = 0;
					uint32 prev_pic_order_cnt_lsb = 0;

					// POC mode 1 & 2
					uint32 FrameNumOffset = 0;
					uint32 PrevFrameNumOffset = 0;

					int32 FramePOC = 0;
					int32 TopPOC = 0;
					int32 BottomPOC = 0;

					uint32 prev_frame_num = 0;
				};
				FPOCValues CurrentPOC;
				FPOCValues PreviousPOC;

				// DPB
				int32 MaxDPBSize = 0;
				int32 max_num_ref_frames = 0;
				int32 max_frame_num = 0;
				int32 max_long_term_pic_index = -1;

				TArray<TSharedPtr<FFrameInDPBInfo>> FrameDPBInfos;
				TArray<TSharedPtr<FFrameInDPBInfo>> ShortTermRefs;
				TArray<TSharedPtr<FFrameInDPBInfo>> LongTermRefs;

				FString LastErrorMsg;
			};


			bool ELECTRADECODERS_API ParseSequenceParameterSet(TMap<uint32, FSequenceParameterSet>& InOutSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			bool ELECTRADECODERS_API ParsePictureParameterSet(TMap<uint32, FPictureParameterSet>& InOutPictureParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			bool ELECTRADECODERS_API ParseSliceHeader(TUniquePtr<FRBSP>& OutRBSP, FBitstreamReaderH264& OutRBSPReader, FSliceHeader& OutSlice, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const TMap<uint32, FPictureParameterSet>& InPictureParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);

			bool ELECTRADECODERS_API ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			inline bool ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const TArray<uint8>& InData)
			{ return ParseSequenceParameterSet(OutSequenceParameterSet, InData.GetData(), InData.Num()); }


			int32 ELECTRADECODERS_API CalcNumBytesInNAL(int32 InProfileLevel, int32 InFrameWidth, int32 InFrameHeight, double InFrameRate);



			/**
			 * ISO/IEC 14496-15:2022 - 5.3.2 Decoder configuration information
			 */
			class FAVCDecoderConfigurationRecord
			{
			public:
				ELECTRADECODERS_API bool Parse(const TConstArrayView<uint8>& InDCR);
				ELECTRADECODERS_API bool CreateFromCodecSpecificData(const TConstArrayView<uint8>& InCSD);

				inline const TArray<uint8>& GetRawData() const
				{ return RawData; }
				inline const TArray<uint8>& GetCodecSpecificData() const
				{ return CSD; }
				inline const TArray<TArray<uint8>>& GetSequenceParameterSets() const
				{ return SequenceParameterSets; }
				inline const TArray<TArray<uint8>>& GetPictureParameterSets() const
				{ return PictureParameterSets; }

				inline uint8 GetAVCProfileIndication() const
				{ return AVCProfileIndication; }
				inline uint8 GetProfileCompatibility() const
				{ return ProfileCompatibility; }
				inline uint8 GetAVCLevelIndication() const
				{ return AVCLevelIndication; }

				inline uint8 GetChromaFormat() const
				{ return ChromaFormat; }
				inline uint8 GetBitDepthLuma() const
				{ return 8 + BitDepthLumaMinus8; }
				inline uint8 GetBitDepthChroma() const
				{ return 8+ BitDepthChromaMinus8; }

				inline FString GetCodecSpecifierRFC6381(const TCHAR* InPrefix) const
				{ return FString::Printf(TEXT("%s.%02x%02x%02x"), InPrefix, GetAVCProfileIndication(), GetProfileCompatibility(), GetAVCLevelIndication()); }

				ELECTRADECODERS_API FString GetFormatInfo() const;

			private:
				TArray<TArray<uint8>> SequenceParameterSets;
				TArray<TArray<uint8>> PictureParameterSets;
				TArray<TArray<uint8>> SequenceParameterSetsExt;
				TArray<uint8> CSD;
				TArray<uint8> RawData;
				uint8 ConfigurationVersion = 0;
				uint8 AVCProfileIndication = 0;
				uint8 ProfileCompatibility = 0;
				uint8 AVCLevelIndication = 0;
				uint8 NALUnitLength = 0;
				uint8 ChromaFormat = 0;
				uint8 BitDepthLumaMinus8 = 0;
				uint8 BitDepthChromaMinus8 = 0;
				bool bHaveAdditionalProfileIndication = false;
			};




		} // namespace H264
	} // namespace MPEG
} // namespace ElectraDecodersUtil

#undef UE_API
