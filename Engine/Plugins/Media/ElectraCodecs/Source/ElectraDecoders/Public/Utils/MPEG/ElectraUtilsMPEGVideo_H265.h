// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"
#include "Misc/Timespan.h"
#include "Utilities/ElectraBitstream.h"
#include "ElectraDecodersTypes.h"

#define UE_API ELECTRADECODERS_API

namespace ElectraDecodersUtil
{
	namespace MPEG
	{
		namespace H265
		{
			struct FNaluInfo
			{
				uint64 Offset;				// Offset into the bitstream where the startcode begins
				uint64 Size;				// Number of bytes, including the nal unit type, that follow the startcode
				uint8 Type;					// Nal unit type
				uint8 LayerId;				// Nuh layer id
				uint8 TemporalIdPlusOne;	// Nuh temporal id +1
				uint8 UnitLength;			// Length of the start code in bytes (3 or 4)
			};
			bool ELECTRADECODERS_API ParseBitstreamForNALUs(TArray<FNaluInfo>& OutNALUs, const uint8* InBitstream, uint64 InBitstreamLength);

			struct FRBSP
			{
				FRBSP(uint64 InDataSize)
				{
					Size = InDataSize;
					Data = (uint8*)FMemory::Malloc(InDataSize);
				}
				~FRBSP()
				{
					FMemory::Free(Data);
				}
				uint8* Data = nullptr;
				uint64 Size = 0;
				TArray<uint64> RemovedAtSrc;
			};
			uint64 ELECTRADECODERS_API EBSPtoRBSP(TArray<uint64>& OutRemovedInBufPos, uint8* OutBuf, const uint8* InBuf, uint64 InNumBytes);
			TUniquePtr<FRBSP> ELECTRADECODERS_API MakeRBSP(const uint8* InData, uint64 InDataSize);

			class FBitstreamReaderH265 : public Electra::FBitstreamReader
			{
			public:
				FBitstreamReaderH265() = default;
				FBitstreamReaderH265(const uint8* InData, uint64 InDataSize)
					: Electra::FBitstreamReader(InData, InDataSize)
				{ }

				uint32 ue_v()
				{
					int32 lz = -1;
					for(uint32 b=0; b==0 && GetRemainingBits(); ++lz)
					{
						b = GetBits(1);
					}
					if (lz >= 0 && GetRemainingBits() >= lz)
					{
						return ((1 << lz) | GetBits(lz)) - 1;
					}
					return 0;
				}
				int32 se_v()
				{
					uint32 c = ue_v();
					return c & 1 ? int32((c + 1) >> 1) : -int32((c + 1) >> 1);
				}

				const uint8* GetData() const
				{
					return static_cast<const uint8*>(GetRemainingData());
				}

				bool more_rbsp_data()
				{
					// There is definitely more data available unless we are in the last byte.
					uint64 NumBitsRemaining = GetRemainingBits();
					if (NumBitsRemaining > 16 || PeekBits(1) == 0)
					{
						return true;
					}
					uint32 Remainder = PeekBits(NumBitsRemaining);
					// Find the position of the last set bit.
					uint64 Pos = 0;
					for(; Pos<NumBitsRemaining && (Remainder & 1) == 0; ++Pos)
					{
						Remainder >>= 1;
					}
					return NumBitsRemaining > Pos+1;
				}

				bool rbsp_trailing_bits()
				{
					if (GetBits(1) != 1)
					{
						return false;
					}
					while(!IsByteAligned())
					{
						if (GetBits(1) != 0)
						{
							return false;
						}
					}
					return true;
				}
				bool byte_alignment()
				{
					// this is exactly the same as rbsp_trailing_bits, so use that one.
					return rbsp_trailing_bits();
				}
			};

			struct FProfileTierLevel
			{
				FProfileTierLevel()
				{
					Reset();
				}
				UE_API bool Parse(FBitstreamReaderH265& br, bool profilePresentFlag, uint32 maxNumSubLayersMinus1);
				void Reset()
				{
					general_profile_space = 0;
					general_tier_flag = 0;
					general_profile_idc = 0;
					FMemory::Memzero(general_profile_compatibility_flag);
					general_profile_compatibility_flags = 0;
					general_constraint_indicator_flags = 0;
					general_progressive_source_flag = 0;
					general_interlaced_source_flag = 0;
					general_non_packed_constraint_flag = 0;
					general_frame_only_constraint_flag = 0;
					general_max_12bit_constraint_flag = 0;
					general_max_10bit_constraint_flag = 0;
					general_max_8bit_constraint_flag = 0;
					general_max_422chroma_constraint_flag = 0;
					general_max_420chroma_constraint_flag = 0;
					general_max_monochrome_constraint_flag = 0;
					general_intra_constraint_flag = 0;
					general_one_picture_only_constraint_flag = 0;
					general_lower_bit_rate_constraint_flag = 0;
					general_max_14bit_constraint_flag = 0;
					general_inbld_flag = 0;
					general_level_idc = 0;
					FMemory::Memzero(sub_layer_profile_present_flag);
					FMemory::Memzero(sub_layer_level_present_flag);
					for(int32 i=0; i<UE_ARRAY_COUNT(sub_layers); ++i)
					{
						sub_layers[i].Reset();
					}
				}

				struct FSubLayer
				{
					FSubLayer()
					{
						Reset();
					}
					void Reset()
					{
						sub_layer_profile_space = 0;
						sub_layer_tier_flag = 0;
						sub_layer_profile_idc = 0;
						FMemory::Memzero(sub_layer_profile_compatibility_flag);
						sub_layer_progressive_source_flag = 0;
						sub_layer_interlaced_source_flag = 0;
						sub_layer_non_packed_constraint_flag = 0;
						sub_layer_frame_only_constraint_flag = 0;
						sub_layer_max_12bit_constraint_flag = 0;
						sub_layer_max_10bit_constraint_flag = 0;
						sub_layer_max_8bit_constraint_flag = 0;
						sub_layer_max_422chroma_constraint_flag = 0;
						sub_layer_max_420chroma_constraint_flag = 0;
						sub_layer_max_monochrome_constraint_flag = 0;
						sub_layer_intra_constraint_flag = 0;
						sub_layer_one_picture_only_constraint_flag = 0;
						sub_layer_lower_bit_rate_constraint_flag = 0;
						sub_layer_max_14bit_constraint_flag = 0;
						sub_layer_inbld_flag = 0;
						sub_layer_level_idc = 0;
					}
					uint8 sub_layer_profile_space;							// u(2)
					uint8 sub_layer_tier_flag;								// u(1)
					uint8 sub_layer_profile_idc;							// u(5)
					uint8 sub_layer_profile_compatibility_flag[32];			// u(1)
					uint8 sub_layer_progressive_source_flag;				// u(1)
					uint8 sub_layer_interlaced_source_flag;					// u(1)
					uint8 sub_layer_non_packed_constraint_flag;				// u(1)
					uint8 sub_layer_frame_only_constraint_flag;				// u(1)
					uint8 sub_layer_max_12bit_constraint_flag;				// u(1)
					uint8 sub_layer_max_10bit_constraint_flag;				// u(1)
					uint8 sub_layer_max_8bit_constraint_flag;				// u(1)
					uint8 sub_layer_max_422chroma_constraint_flag;			// u(1)
					uint8 sub_layer_max_420chroma_constraint_flag;			// u(1)
					uint8 sub_layer_max_monochrome_constraint_flag;			// u(1)
					uint8 sub_layer_intra_constraint_flag;					// u(1)
					uint8 sub_layer_one_picture_only_constraint_flag;		// u(1)
					uint8 sub_layer_lower_bit_rate_constraint_flag;			// u(1)
					uint8 sub_layer_max_14bit_constraint_flag;				// u(1)
					uint8 sub_layer_inbld_flag;								// u(1)
					uint8 sub_layer_level_idc;								// u(8)
				};

				uint8 general_profile_space;							// u(2)
				uint8 general_tier_flag;								// u(1)
				uint8 general_profile_idc;								// u(5)
				uint8 general_profile_compatibility_flag[32];			// u(1)
				uint32 general_profile_compatibility_flags;				// u(32), same as general_profile_compatibility_flag[32] but in a single word
				uint64 general_constraint_indicator_flags;				// u(48), bits starting with `general_progressive_source_flag` and extending through the 44 constraint bits
				uint8 general_progressive_source_flag;					// u(1)
				uint8 general_interlaced_source_flag;					// u(1)
				uint8 general_non_packed_constraint_flag;				// u(1)
				uint8 general_frame_only_constraint_flag;				// u(1)
				uint8 general_max_12bit_constraint_flag;				// u(1)
				uint8 general_max_10bit_constraint_flag;				// u(1)
				uint8 general_max_8bit_constraint_flag;					// u(1)
				uint8 general_max_422chroma_constraint_flag;			// u(1)
				uint8 general_max_420chroma_constraint_flag;			// u(1)
				uint8 general_max_monochrome_constraint_flag;			// u(1)
				uint8 general_intra_constraint_flag;					// u(1)
				uint8 general_one_picture_only_constraint_flag;			// u(1)
				uint8 general_lower_bit_rate_constraint_flag;			// u(1)
				uint8 general_max_14bit_constraint_flag;				// u(1)
				uint8 general_inbld_flag;								// u(1)
				uint8 general_level_idc;								// u(8)
				uint8 sub_layer_profile_present_flag[7];				// u(1)
				uint8 sub_layer_level_present_flag[7];					// u(1)
				FSubLayer sub_layers[7];
			};

			struct FHRDParameters
			{
				UE_API bool Parse(FBitstreamReaderH265& br, bool commonInfPresentFlag, uint32 maxNumSubLayersMinus1);
				void Reset()
				{
					nal_hrd_parameters_present_flag = 0;
					vcl_hrd_parameters_present_flag = 0;
					sub_pic_hrd_params_present_flag = 0;
					tick_divisor_minus2 = 0;
					du_cpb_removal_delay_increment_length_minus1 = 0;
					sub_pic_cpb_params_in_pic_timing_sei_flag = 0;
					dpb_output_delay_du_length_minus1 = 0;
					bit_rate_scale = 0;
					cpb_size_scale = 0;
					cpb_size_du_scale = 0;
					initial_cpb_removal_delay_length_minus1 = 0;
					au_cpb_removal_delay_length_minus1 = 0;
					dpb_output_delay_length_minus1 = 0;
					SubLayers.Empty();
				}

				struct FSubLayer
				{
					struct FSubLayerParameters
					{
						struct FCpbEntry
						{
							uint32 bit_rate_value_minus1 = 0;			// ue(v)
							uint32 cpb_size_value_minus1 = 0;			// ue(v)
							uint32 cpb_size_du_value_minus1 = 0;		// ue(v)
							uint32 bit_rate_du_value_minus1 = 0;		// ue(v)
							uint8 cbr_flag = 0;							// u(1)
						};
						TArray<FCpbEntry> cpb;
					};
					uint8 fixed_pic_rate_general_flag = 0;				// u(1)
					uint8 fixed_pic_rate_within_cvs_flag = 0;			// u(1)
					uint32 elemental_duration_in_tc_minus1 = 0;			// ue(v)
					uint8 low_delay_hrd_flag = 0;						// u(1)
					uint32 cpb_cnt_minus1 = 0;							// ue(v)
					FSubLayerParameters nal_sub_layer_hrd_parameters;
					FSubLayerParameters vcl_sub_layer_hrd_parameters;
				};

				uint8 nal_hrd_parameters_present_flag = 0;				// u(1)
				uint8 vcl_hrd_parameters_present_flag = 0;				// u(1)
				uint8 sub_pic_hrd_params_present_flag = 0;				// u(1)
				uint8 tick_divisor_minus2 = 0;							// u(8)
				uint8 du_cpb_removal_delay_increment_length_minus1 = 0;	// u(5)
				uint8 sub_pic_cpb_params_in_pic_timing_sei_flag = 0;	// u(1)
				uint8 dpb_output_delay_du_length_minus1 = 0;			// u(5)
				uint8 bit_rate_scale = 0;								// u(4)
				uint8 cpb_size_scale = 0;								// u(4)
				uint8 cpb_size_du_scale = 0;							// u(4)
				uint8 initial_cpb_removal_delay_length_minus1 = 0;		// u(5)
				uint8 au_cpb_removal_delay_length_minus1 = 0;			// u(5)
				uint8 dpb_output_delay_length_minus1 = 0;				// u(5)
				TArray<FSubLayer> SubLayers;
			};

			struct FVUIParameters
			{
				FVUIParameters()
				{
					Reset();
				}
				UE_API bool Parse(FBitstreamReaderH265& br, uint32 In_sps_max_sub_layers_minus1);
				void Reset()
				{
					aspect_ratio_info_present_flag = 0;
					aspect_ratio_idc = 0;
					sar_width = 0;
					sar_height = 0;
					overscan_info_present_flag = 0;
					overscan_appropriate_flag = 0;
					video_signal_type_present_flag = 0;
					video_format = 5;
					video_full_range_flag = 0;
					colour_description_present_flag = 0;
					colour_primaries = 2;
					transfer_characteristics = 2;
					matrix_coeffs = 2;
					chroma_loc_info_present_flag = 0;
					chroma_sample_loc_type_top_field = 0;
					chroma_sample_loc_type_bottom_field = 0;
					neutral_chroma_indication_flag = 0;
					field_seq_flag = 0;
					frame_field_info_present_flag = 0;
					default_display_window_flag = 0;
					def_disp_win_left_offset = 0;
					def_disp_win_right_offset = 0;
					def_disp_win_top_offset = 0;
					def_disp_win_bottom_offset = 0;
					vui_timing_info_present_flag = 0;
					vui_num_units_in_tick = 0;
					vui_time_scale = 0;
					vui_poc_proportional_to_timing_flag = 0;
					vui_num_ticks_poc_diff_one_minus1 = 0;
					vui_hrd_parameters_present_flag = 0;
					hrd_parameters.Reset();
					bitstream_restriction_flag = 0;
					tiles_fixed_structure_flag = 0;
					motion_vectors_over_pic_boundaries_flag = 0;
					restricted_ref_pic_lists_flag = 0;
					min_spatial_segmentation_idc = 0;
					max_bytes_per_pic_denom = 0;
					max_bits_per_min_cu_denom = 0;
					log2_max_mv_length_horizontal = 0;
					log2_max_mv_length_vertical = 0;
				}
				uint8 aspect_ratio_info_present_flag;					// u(1)
				uint8 aspect_ratio_idc;									// u(8)
				uint16 sar_width;										// u(16)
				uint16 sar_height;										// u(16)
				uint8 overscan_info_present_flag;						// u(1)
				uint8 overscan_appropriate_flag;						// u(1)
				uint8 video_signal_type_present_flag;					// u(1)
				uint8 video_format;										// u(3), default 5
				uint8 video_full_range_flag;							// u(1)
				uint8 colour_description_present_flag;					// u(1)
				uint8 colour_primaries;									// u(8), default 2
				uint8 transfer_characteristics;							// u(8), default 2
				uint8 matrix_coeffs;									// u(8), default 2
				uint8 chroma_loc_info_present_flag;						// u(1)
				uint32 chroma_sample_loc_type_top_field;				// ue(v), 0 to 5, default 0
				uint32 chroma_sample_loc_type_bottom_field;				// ue(v), 0 to 5, default 0
				uint8 neutral_chroma_indication_flag;					// u(1)
				uint8 field_seq_flag;									// u(1)
				uint8 frame_field_info_present_flag;					// u(1)
				uint8 default_display_window_flag;						// u(1)
				uint32 def_disp_win_left_offset;						// ue(v), default 0
				uint32 def_disp_win_right_offset;						// ue(v), default 0
				uint32 def_disp_win_top_offset;							// ue(v), default 0
				uint32 def_disp_win_bottom_offset;						// ue(v), default 0
				uint8 vui_timing_info_present_flag;						// u(1)
				uint32 vui_num_units_in_tick;							// u(32)
				uint32 vui_time_scale;									// u(32)
				uint8 vui_poc_proportional_to_timing_flag;				// u(1)
				uint32 vui_num_ticks_poc_diff_one_minus1;				// ue(v)
				uint8 vui_hrd_parameters_present_flag;					// u(1)
				FHRDParameters hrd_parameters;
				uint8 bitstream_restriction_flag;						// u(1)
				uint8 tiles_fixed_structure_flag;						// u(1)
				uint8 motion_vectors_over_pic_boundaries_flag;			// u(1)
				uint8 restricted_ref_pic_lists_flag;					// u(1)
				uint32 min_spatial_segmentation_idc;					// ue(v), 0 to 4095
				uint32 max_bytes_per_pic_denom;							// ue(v), 0 to 16
				uint32 max_bits_per_min_cu_denom;						// ue(v), 0 to 16
				uint32 log2_max_mv_length_horizontal;					// ue(v), 0 to 15
				uint32 log2_max_mv_length_vertical;						// ue(v), 0 to 15
			};

			struct FScalingListData
			{
				UE_API bool Parse(FBitstreamReaderH265& br);
				UE_API void SetDefaults();
				void Reset()
				{
					FMemory::Memzero(scaling_list_dc);
					for(int32 i=0; i<4; ++i)
					{
						for(int32 j=0; j<6; ++j)
						{
							scaling_list[i][j].Empty();
							scaling_list[i][j].SetNum(i==0 ? 16 : 64);
						}
					}
				}
				uint8 scaling_list_dc[4][6];
				TArray<uint8> scaling_list[4][6];
			};

			struct FStRefPicSet
			{
				UE_API bool Parse(FBitstreamReaderH265& br, int32 stRpsIdx, const TArray<FStRefPicSet>& InReferencePicSetList, int32 InReferencePicSetListNum);
				void Reset()
				{
					inter_ref_pic_set_prediction_flag = 0;
					delta_idx_minus1 = 0;
					delta_rps_sign = 0;
					abs_delta_rps_minus1 = 0;
					num_negative_pics = 0;
					num_positive_pics = 0;
					raw_values.Reset();
					FMemory::Memzero(delta_poc);
					FMemory::Memzero(used_by_curr_pic_flag);
					NumDeltaPocsInSliceReferencedSet = 0;
				}

				uint8 inter_ref_pic_set_prediction_flag = 0;			// u(1)
				uint32 delta_idx_minus1 = 0;							// ue(v)
				uint8 delta_rps_sign = 0;								// u(1)
				uint32 abs_delta_rps_minus1 = 0;						// ue(v)
				uint32 num_negative_pics = 0;							// ue(v)
				uint32 num_positive_pics = 0;							// ue(v)

				// Raw values as read from the bitstream.
				// These are kept in case some type of hardware accelerator needs them.
				struct FRawValues
				{
					void Reset()
					{
						FMemory::Memzero(used_by_curr_pic_flag);
						FMemory::Memzero(use_delta_flag);
						FMemory::Memzero(delta_poc_s0_minus1);
						FMemory::Memzero(used_by_curr_pic_s0_flag);
						FMemory::Memzero(delta_poc_s1_minus1);
						FMemory::Memzero(used_by_curr_pic_s1_flag);
					}
					uint8 used_by_curr_pic_flag[32] {};					// u(1)
					uint8 use_delta_flag[32] {};						// u(1)
					uint32 delta_poc_s0_minus1[16] {};					// ue(v)
					uint8 used_by_curr_pic_s0_flag[16] {};				// u(1)
					uint32 delta_poc_s1_minus1[16] {};					// ue(v)
					uint8 used_by_curr_pic_s1_flag[16] {};				// u(1)
				};
				FRawValues raw_values;

				// Calculated and sorted values as they are commonly used.
				int32 delta_poc[32] {};
				uint8 used_by_curr_pic_flag[32] {};
				// When parsing this from a slice header we note the number of
				// delta_poc's in the referenced set from the SPS. This is needed by some accelerators.
				uint32 NumDeltaPocsInSliceReferencedSet = 0;

				uint32 NumNegativePics() const
				{ return num_negative_pics; }
				uint32 NumPositivePics() const
				{ return num_positive_pics; }
				uint32 NumDeltaPocs() const
				{ return NumNegativePics() + NumPositivePics(); }
				int32 GetDeltaPOC(uint32 InIndex) const
				{ return delta_poc[InIndex]; }
				bool IsUsed(uint32 InIndex) const
				{ return used_by_curr_pic_flag[InIndex] != 0; }
				UE_API void SortDeltaPOC();
			};

			struct FVideoParameterSet
			{
				FVideoParameterSet()
				{
					Reset();
				}
				void Reset()
				{
					vps_video_parameter_set_id = 0;
					vps_base_layer_internal_flag = 0;
					vps_base_layer_available_flag = 0;
					vps_max_layers_minus1 = 0;
					vps_max_sub_layers_minus1 = 0;
					vps_temporal_id_nesting_flag = 0;
					profile_tier_level.Reset();
					vps_sub_layer_ordering_info_present_flag = 0;
					FMemory::Memzero(vps_max_dec_pic_buffering_minus1);
					FMemory::Memzero(vps_max_num_reorder_pics);
					FMemory::Memzero(vps_max_latency_increase_plus1);
					vps_max_layer_id = 0;
					vps_num_layer_sets_minus1 = 0;
					layer_id_included_flag.Empty();
					vps_timing_info_present_flag = 0;
					vps_num_units_in_tick = 0;
					vps_time_scale = 0;
					vps_poc_proportional_to_timing_flag = 0;
					vps_num_ticks_poc_diff_one_minus1 = 0;
					ListOf_hrd_parameters.Empty();
					vps_extension_flag = 0;
				}

				struct FVPSHRD
				{
					uint32 hrd_layer_set_idx = 0;							// ue(v)
					uint8 cprms_present_flag = 0;							// u(1)
					FHRDParameters hrd_parameter;
				};

				uint8 vps_video_parameter_set_id;							// u(4)
				uint8 vps_base_layer_internal_flag;							// u(1)
				uint8 vps_base_layer_available_flag;						// u(1)
				uint8 vps_max_layers_minus1;								// u(6), 0-63 (63 is reserved)
				uint8 vps_max_sub_layers_minus1;							// u(3), 0-6
				uint8 vps_temporal_id_nesting_flag;							// u(1)
				FProfileTierLevel profile_tier_level;
				uint8 vps_sub_layer_ordering_info_present_flag;				// u(1)
				uint32 vps_max_dec_pic_buffering_minus1[8];					// ue(v), 0 to MaxDpbSize-1
				uint32 vps_max_num_reorder_pics[8];							// ue(v), 0 to vps_max_dec_pic_buffering_minus1[i]
				uint32 vps_max_latency_increase_plus1[8];					// ue(v)
				uint8 vps_max_layer_id;										// u(6), 0-63 (63 is reserved)
				uint32 vps_num_layer_sets_minus1;							// ue(v), 0-1023
				TArray<TArray<uint8>> layer_id_included_flag;				// u(1)
				uint8 vps_timing_info_present_flag;							// u(1)
				uint32 vps_num_units_in_tick;								// u(32)
				uint32 vps_time_scale;										// u(32)
				uint8 vps_poc_proportional_to_timing_flag;					// u(1)
				uint32 vps_num_ticks_poc_diff_one_minus1;					// ue(v)
				TArray<FVPSHRD> ListOf_hrd_parameters;
				uint8 vps_extension_flag;									// u(1)
			};

			struct FSequenceParameterSet
			{
				FSequenceParameterSet()
				{
					Reset();
				}
				void Reset()
				{
					sps_video_parameter_set_id = 0;
					sps_max_sub_layers_minus1 = 0;
					sps_temporal_id_nesting_flag = 0;
					profile_tier_level.Reset();
					sps_seq_parameter_set_id = 0;
					chroma_format_idc = 0;
					separate_colour_plane_flag = 0;
					pic_width_in_luma_samples = 0;
					pic_height_in_luma_samples = 0;
					conformance_window_flag = 0;
					conf_win_left_offset = 0;
					conf_win_right_offset = 0;
					conf_win_top_offset = 0;
					conf_win_bottom_offset = 0;
					bit_depth_luma_minus8 = 0;
					bit_depth_chroma_minus8 = 0;
					log2_max_pic_order_cnt_lsb_minus4 = 0;
					sps_sub_layer_ordering_info_present_flag = 0;
					FMemory::Memzero(sps_max_dec_pic_buffering_minus1);
					FMemory::Memzero(sps_max_num_reorder_pics);
					FMemory::Memzero(sps_max_latency_increase_plus1);
					log2_min_luma_coding_block_size_minus3 = 0;
					log2_diff_max_min_luma_coding_block_size = 0;
					log2_min_luma_transform_block_size_minus2 = 0;
					log2_diff_max_min_luma_transform_block_size = 0;
					max_transform_hierarchy_depth_inter = 0;
					max_transform_hierarchy_depth_intra = 0;
					scaling_list_enabled_flag = 0;
					sps_scaling_list_data_present_flag = 0;
					scaling_list_data.Reset();
					amp_enabled_flag = 0;
					sample_adaptive_offset_enabled_flag = 0;
					pcm_enabled_flag = 0;
					pcm_sample_bit_depth_luma_minus1 = 0;
					pcm_sample_bit_depth_chroma_minus1 = 0;
					log2_min_pcm_luma_coding_block_size_minus3 = 0;
					log2_diff_max_min_pcm_luma_coding_block_size = 0;
					pcm_loop_filter_disabled_flag = 0;
					num_short_term_ref_pic_sets = 0;
					st_ref_pic_set.Empty();
					long_term_ref_pics_present_flag = 0;
					num_long_term_ref_pics_sps = 0;
					long_term_ref_pics.Empty();
					sps_temporal_mvp_enabled_flag = 0;
					strong_intra_smoothing_enabled_flag = 0;
					vui_parameters_present_flag = 0;
					vui_parameters.Reset();
					sps_extension_present_flag = 0;
					sps_range_extension_flag = 0;
					sps_multilayer_extension_flag = 0;
					sps_3d_extension_flag = 0;
					sps_scc_extension_flag = 0;
					sps_extension_4bits = 0;
					sps_range_extension.Reset();
					sps_multilayer_extension.Reset();
					sps_3d_extension.Reset();
					sps_scc_extension.Reset();

					// Calculated common values
					ChromaArrayType = 0;
					MinCbLog2SizeY = 0;
					CtbLog2SizeY = 0;
					MinCbSizeY = 0;
					CtbSizeY = 0;
					PicWidthInMinCbsY = 0;
					PicWidthInCtbsY = 0;
					PicHeightInMinCbsY = 0;
					PicHeightInCtbsY = 0;
					PicSizeInMinCbsY = 0;
					PicSizeInCtbsY = 0;
					PicSizeInSamplesY = 0;
					PicWidthInSamplesC = 0;
					PicHeightInSamplesC = 0;
					CtbWidthC = 0;
					CtbHeightC = 0;
					SubWidthC = 0;
					SubHeightC = 0;
				}
				UE_API int32 GetMaxDPBSize() const;
				UE_API int32 GetDPBSize() const;
				UE_API int32 GetMinCbSizeY() const;
				UE_API int32 GetWidth() const;
				UE_API int32 GetHeight() const;
				UE_API void GetDisplaySize(int32& OutWidth, int32& OutHeight) const;
				UE_API void GetDisplaySize(uint32& OutWidth, uint32& OutHeight) const;
				UE_API void GetCrop(int32& OutLeft, int32& OutRight, int32& OutTop, int32& OutBottom) const;
				UE_API void GetAspect(int32& OutSarW, int32& OutSarH) const;
				UE_API FFractionalValue GetTiming() const;
				UE_API uint64 GetConstraintFlags() const;
				UE_API FString GetRFC6381(const TCHAR* SampleTypePrefix) const;
				UE_API FString GetFormatInfo() const;
				static UE_API FString GetFormatInfo(uint32 InProfileIDC, uint32 InLevelIDC, uint64 InConstraintFlags);

				uint8 sps_video_parameter_set_id;							// u(4)
				uint8 sps_max_sub_layers_minus1;							// u(3), 0-6
				uint8 sps_temporal_id_nesting_flag;							// u(1)
				FProfileTierLevel profile_tier_level;
				uint32 sps_seq_parameter_set_id;							// ue(v), 0-15
				uint32 chroma_format_idc;									// ue(v), 0-3
				uint8 separate_colour_plane_flag;							// u(1)
				uint32 pic_width_in_luma_samples;							// ue(v)
				uint32 pic_height_in_luma_samples;							// ue(v)
				uint8 conformance_window_flag;								// u(1)
				uint32 conf_win_left_offset;								// ue(v)
				uint32 conf_win_right_offset;								// ue(v)
				uint32 conf_win_top_offset;									// ue(v)
				uint32 conf_win_bottom_offset;								// ue(v)
				uint32 bit_depth_luma_minus8;								// ue(v)
				uint32 bit_depth_chroma_minus8;								// ue(v)
				uint32 log2_max_pic_order_cnt_lsb_minus4;					// ue(v)
				uint8 sps_sub_layer_ordering_info_present_flag;				// u(1)
				uint32 sps_max_dec_pic_buffering_minus1[8];					// ue(v)
				uint32 sps_max_num_reorder_pics[8];							// ue(v)
				uint32 sps_max_latency_increase_plus1[8];					// ue(v)
				uint32 log2_min_luma_coding_block_size_minus3;				// ue(v)
				uint32 log2_diff_max_min_luma_coding_block_size;			// ue(v)
				uint32 log2_min_luma_transform_block_size_minus2;			// ue(v)
				uint32 log2_diff_max_min_luma_transform_block_size;			// ue(v)
				uint32 max_transform_hierarchy_depth_inter;					// ue(v)
				uint32 max_transform_hierarchy_depth_intra;					// ue(v)
				uint8 scaling_list_enabled_flag;							// u(1)
				uint8 sps_scaling_list_data_present_flag;					// u(1)
				FScalingListData scaling_list_data;
				uint8 amp_enabled_flag;										// u(1)
				uint8 sample_adaptive_offset_enabled_flag;					// u(1)
				uint8 pcm_enabled_flag;										// u(1)
				uint8 pcm_sample_bit_depth_luma_minus1;						// u(4)
				uint8 pcm_sample_bit_depth_chroma_minus1;					// u(4)
				uint32 log2_min_pcm_luma_coding_block_size_minus3;			// ue(v)
				uint32 log2_diff_max_min_pcm_luma_coding_block_size;		// ue(v)
				uint8 pcm_loop_filter_disabled_flag;						// u(1)
				uint32 num_short_term_ref_pic_sets;							// ue(v)
				TArray<FStRefPicSet> st_ref_pic_set;
				uint8 long_term_ref_pics_present_flag;						// u(1)
				uint32 num_long_term_ref_pics_sps;							// ue(v)
				struct FLongTermRefPic
				{
					uint32 lt_ref_pic_poc_lsb_sps = 0;						// u(v)
					uint8 used_by_curr_pic_lt_sps_flag = 0;					// u(1)
				};
				TArray<FLongTermRefPic> long_term_ref_pics;
				uint8 sps_temporal_mvp_enabled_flag;						// u(1)
				uint8 strong_intra_smoothing_enabled_flag;					// u(1)
				uint8 vui_parameters_present_flag;							// u(1)
				FVUIParameters vui_parameters;
				uint8 sps_extension_present_flag;							// u(1)
				uint8 sps_range_extension_flag;								// u(1)
				uint8 sps_multilayer_extension_flag;						// u(1)
				uint8 sps_3d_extension_flag;								// u(1)
				uint8 sps_scc_extension_flag;								// u(1)
				uint8 sps_extension_4bits;									// u(4)

				struct FSPSRangeExtension
				{
					void Reset()
					{
						transform_skip_rotation_enabled_flag = 0;
						transform_skip_context_enabled_flag = 0;
						implicit_rdpcm_enabled_flag = 0;
						explicit_rdpcm_enabled_flag = 0;
						extended_precision_processing_flag = 0;
						intra_smoothing_disabled_flag = 0;
						high_precision_offsets_enabled_flag = 0;
						persistent_rice_adaptation_enabled_flag = 0;
						cabac_bypass_alignment_enabled_flag = 0;
					}
					uint8 transform_skip_rotation_enabled_flag;				// u(1)
					uint8 transform_skip_context_enabled_flag;				// u(1)
					uint8 implicit_rdpcm_enabled_flag;						// u(1)
					uint8 explicit_rdpcm_enabled_flag;						// u(1)
					uint8 extended_precision_processing_flag;				// u(1)
					uint8 intra_smoothing_disabled_flag;					// u(1)
					uint8 high_precision_offsets_enabled_flag;				// u(1)
					uint8 persistent_rice_adaptation_enabled_flag;			// u(1)
					uint8 cabac_bypass_alignment_enabled_flag;				// u(1)
				};
				FSPSRangeExtension sps_range_extension;

				struct FSPSMultilayerExtension
				{
					void Reset()
					{
						inter_view_mv_vert_constraint_flag = 0;
					}
					uint8 inter_view_mv_vert_constraint_flag;				// u(1)
				};
				FSPSMultilayerExtension sps_multilayer_extension;

				struct FSPS3DExtension
				{
					void Reset()
					{
						FMemory::Memzero(iv_di_mc_enabled_flag);
						FMemory::Memzero(iv_mv_scal_enabled_flag);
						FMemory::Memzero(log2_ivmc_sub_pb_size_minus3);
						FMemory::Memzero(iv_res_pred_enabled_flag);
						FMemory::Memzero(depth_ref_enabled_flag);
						FMemory::Memzero(vsp_mc_enabled_flag);
						FMemory::Memzero(dbbp_enabled_flag);
						FMemory::Memzero(tex_mc_enabled_flag);
						FMemory::Memzero(log2_texmc_sub_pb_size_minus3);
						FMemory::Memzero(intra_contour_enabled_flag);
						FMemory::Memzero(intra_dc_only_wedge_enabled_flag);
						FMemory::Memzero(cqt_cu_part_pred_enabled_flag);
						FMemory::Memzero(inter_dc_only_enabled_flag);
						FMemory::Memzero(skip_intra_enabled_flag);
					}
					uint8 iv_di_mc_enabled_flag[2];							// u(1)
					uint8 iv_mv_scal_enabled_flag[2];						// u(1)
					uint32 log2_ivmc_sub_pb_size_minus3[2];					// ue(v)
					uint8 iv_res_pred_enabled_flag[2];						// u(1)
					uint8 depth_ref_enabled_flag[2];						// u(1)
					uint8 vsp_mc_enabled_flag[2];							// u(1)
					uint8 dbbp_enabled_flag[2];								// u(1)
					uint8 tex_mc_enabled_flag[2];							// u(1)
					uint32 log2_texmc_sub_pb_size_minus3[2];				// ue(v)
					uint8 intra_contour_enabled_flag[2];					// u(1)
					uint8 intra_dc_only_wedge_enabled_flag[2];				// u(1)
					uint8 cqt_cu_part_pred_enabled_flag[2];					// u(1)
					uint8 inter_dc_only_enabled_flag[2];					// u(1)
					uint8 skip_intra_enabled_flag[2];						// u(1)
				};
				FSPS3DExtension sps_3d_extension;

				struct FSPSSCCExtension
				{
					void Reset()
					{
						sps_curr_pic_ref_enabled_flag = 0;
						palette_mode_enabled_flag = 0;
						palette_max_size = 0;
						delta_palette_max_predictor_size = 0;
						sps_palette_predictor_initializers_present_flag = 0;
						sps_num_palette_predictor_initializers_minus1 = 0;
						for(int32 i=0; i<UE_ARRAY_COUNT(sps_palette_predictor_initializer); ++i)
						{
							sps_palette_predictor_initializer[i].Reset();
						}
						motion_vector_resolution_control_idc = 0;
						intra_boundary_filtering_disabled_flag = 0;
					}
					uint8 sps_curr_pic_ref_enabled_flag;					// u(1)
					uint8 palette_mode_enabled_flag;						// u(1)
					uint32 palette_max_size;								// ue(v)
					uint32 delta_palette_max_predictor_size;				// ue(v)
					uint8 sps_palette_predictor_initializers_present_flag;	// u(1)
					uint32 sps_num_palette_predictor_initializers_minus1;	// ue(v)
					TArray<uint32> sps_palette_predictor_initializer[3];
					uint8 motion_vector_resolution_control_idc;				// u(2)
					uint8 intra_boundary_filtering_disabled_flag;			// u(1)
				};
				FSPSSCCExtension sps_scc_extension;

				// Calculated common values
				uint32 ChromaArrayType = 0;
				uint32 MinCbLog2SizeY = 0;
				uint32 CtbLog2SizeY = 0;
				uint32 MinCbSizeY = 0;
				uint32 CtbSizeY = 0;
				uint32 PicWidthInMinCbsY = 0;
				uint32 PicWidthInCtbsY = 0;
				uint32 PicHeightInMinCbsY = 0;
				uint32 PicHeightInCtbsY = 0;
				uint32 PicSizeInMinCbsY = 0;
				uint32 PicSizeInCtbsY = 0;
				uint32 PicSizeInSamplesY = 0;
				uint32 PicWidthInSamplesC = 0;
				uint32 PicHeightInSamplesC = 0;
				uint32 CtbWidthC = 0;
				uint32 CtbHeightC = 0;
				uint32 SubWidthC = 0;
				uint32 SubHeightC = 0;
			};

			struct FPictureParameterSet
			{
				FPictureParameterSet()
				{
					Reset();
				}
				void Reset()
				{
					pps_pic_parameter_set_id = 0;
					pps_seq_parameter_set_id = 0;
					dependent_slice_segments_enabled_flag = 0;
					output_flag_present_flag = 0;
					num_extra_slice_header_bits = 0;
					sign_data_hiding_enabled_flag = 0;
					cabac_init_present_flag = 0;
					num_ref_idx_l0_default_active_minus1 = 0;
					num_ref_idx_l1_default_active_minus1 = 0;
					init_qp_minus26 = 0;
					constrained_intra_pred_flag = 0;
					transform_skip_enabled_flag = 0;
					cu_qp_delta_enabled_flag = 0;
					diff_cu_qp_delta_depth = 0;
					pps_cb_qp_offset = 0;
					pps_cr_qp_offset = 0;
					pps_slice_chroma_qp_offsets_present_flag = 0;
					weighted_pred_flag = 0;
					weighted_bipred_flag = 0;
					transquant_bypass_enabled_flag = 0;
					tiles_enabled_flag = 0;
					entropy_coding_sync_enabled_flag = 0;
					num_tile_columns_minus1 = 0;
					num_tile_rows_minus1 = 0;
					uniform_spacing_flag = 1;
					column_width_minus1.Empty();
					row_height_minus1.Empty();
					loop_filter_across_tiles_enabled_flag = 1;
					pps_loop_filter_across_slices_enabled_flag = 0;
					deblocking_filter_control_present_flag = 0;
					deblocking_filter_override_enabled_flag = 0;
					pps_deblocking_filter_disabled_flag = 0;
					pps_beta_offset_div2 = 0;
					pps_tc_offset_div2 = 0;
					pps_scaling_list_data_present_flag = 0;
					scaling_list_data.Reset();
					lists_modification_present_flag = 0;
					log2_parallel_merge_level_minus2 = 0;
					slice_segment_header_extension_present_flag = 0;
					pps_extension_present_flag = 0;
					pps_range_extension_flag = 0;
					pps_multilayer_extension_flag = 0;
					pps_3d_extension_flag = 0;
					pps_scc_extension_flag = 0;
					pps_extension_4bits = 0;
					pps_range_extension.Reset();
					pps_multilayer_extension.Reset();
					pps_3d_extension.Reset();
					pps_scc_extension.Reset();
				}

				uint32 pps_pic_parameter_set_id;						// ue(v), 0-63
				uint32 pps_seq_parameter_set_id;						// ue(v), 0-15
				uint8 dependent_slice_segments_enabled_flag;			// u(1)
				uint8 output_flag_present_flag;							// u(1)
				uint8 num_extra_slice_header_bits;						// u(3), 0-2
				uint8 sign_data_hiding_enabled_flag;					// u(1)
				uint8 cabac_init_present_flag;							// u(1)
				uint32 num_ref_idx_l0_default_active_minus1;			// ue(v)
				uint32 num_ref_idx_l1_default_active_minus1;			// ue(v)
				int32 init_qp_minus26;									// se(v), -26 to 25
				uint8 constrained_intra_pred_flag;						// u(1)
				uint8 transform_skip_enabled_flag;						// u(1)
				uint8 cu_qp_delta_enabled_flag;							// u(1)
				uint32 diff_cu_qp_delta_depth;							// ue(v), 0 to log2_diff_max_min_luma_coding_block_size, default 0
				int32 pps_cb_qp_offset;									// se(v), -12 to 12
				int32 pps_cr_qp_offset;									// se(v), -12 to 12
				uint8 pps_slice_chroma_qp_offsets_present_flag;			// u(1)
				uint8 weighted_pred_flag;								// u(1)
				uint8 weighted_bipred_flag;								// u(1)
				uint8 transquant_bypass_enabled_flag;					// u(1)
				uint8 tiles_enabled_flag;								// u(1)
				uint8 entropy_coding_sync_enabled_flag;					// u(1)
				uint32 num_tile_columns_minus1;							// ue(v), 0 to PicWidthInCtbsY-1
				uint32 num_tile_rows_minus1;							// ue(v), 0 to PicHeightInCtbsY-1
				uint8 uniform_spacing_flag;								// u(1), default 1
				TArray<uint32> column_width_minus1;						// ue(v)
				TArray<uint32> row_height_minus1;						// ue(v)
				uint8 loop_filter_across_tiles_enabled_flag;			// u(1), default 1
				uint8 pps_loop_filter_across_slices_enabled_flag;		// u(1)
				uint8 deblocking_filter_control_present_flag;			// u(1)
				uint8 deblocking_filter_override_enabled_flag;			// u(1), default 0
				uint8 pps_deblocking_filter_disabled_flag;				// u(1), default 0
				int32 pps_beta_offset_div2;								// se(v), -6 to 6, default 0
				int32 pps_tc_offset_div2;								// se(v), -6 to 6, default 0
				uint8 pps_scaling_list_data_present_flag;				// u(1)
				FScalingListData scaling_list_data;
				uint8 lists_modification_present_flag;					// u(1)
				uint32 log2_parallel_merge_level_minus2;				// ue(v), 0 to CtbLog2SizeY-2
				uint8 slice_segment_header_extension_present_flag;		// u(1)
				uint8 pps_extension_present_flag;						// u(1)
				uint8 pps_range_extension_flag;							// u(1), default 0
				uint8 pps_multilayer_extension_flag;					// u(1), default 0
				uint8 pps_3d_extension_flag;							// u(1), default 0
				uint8 pps_scc_extension_flag;							// u(1), default 0
				uint8 pps_extension_4bits;								// u(4), default 0

				struct FPPSRangeExtension
				{
					FPPSRangeExtension()
					{
						Reset();
					}
					bool Parse(FBitstreamReaderH265& br);
					void Reset()
					{
						log2_max_transform_skip_block_size_minus2 = 0;
						cross_component_prediction_enabled_flag = 0;
						chroma_qp_offset_list_enabled_flag = 0;
						diff_cu_chroma_qp_offset_depth = 0;
						chroma_qp_offset_list_len_minus1 = 0;
						cb_qp_offset_list.Reset();
						cr_qp_offset_list.Reset();
						log2_sao_offset_scale_luma = 0;
						log2_sao_offset_scale_chroma = 0;
					}
					uint32 log2_max_transform_skip_block_size_minus2;	// ue(v)
					uint8 cross_component_prediction_enabled_flag;		// u(1)
					uint8 chroma_qp_offset_list_enabled_flag;			// u(1)
					uint32 diff_cu_chroma_qp_offset_depth;				// ue(v)
					uint32 chroma_qp_offset_list_len_minus1;			// ue(v)
					TArray<int32> cb_qp_offset_list;					// se(v)
					TArray<int32> cr_qp_offset_list;					// se(v)
					uint32 log2_sao_offset_scale_luma;					// ue(v)
					uint32 log2_sao_offset_scale_chroma;				// ue(v)
				};

				struct FPPSMultilayerExtension
				{
					FPPSMultilayerExtension()
					{
						Reset();
					}
					bool Parse(FBitstreamReaderH265& br);
					void Reset()
					{
						poc_reset_info_present_flag = 0;
						pps_infer_scaling_list_flag = 0;
						pps_scaling_list_ref_layer_id = 0;
						num_ref_loc_offsets = 0;
						rlos.Empty();
						colour_mapping_enabled_flag = 0;
						colour_mapping_table.Reset();
					}
					struct FColourMappingTable
					{
						void Reset()
						{
							num_cm_ref_layers_minus1 = 0;
							FMemory::Memzero(cm_ref_layer_id);
							cm_octant_depth = 0;
							cm_y_part_num_log2 = 0;
							luma_bit_depth_cm_input_minus8 = 0;
							chroma_bit_depth_cm_input_minus8 = 0;
							luma_bit_depth_cm_output_minus8 = 0;
							chroma_bit_depth_cm_output_minus8 = 0;
							cm_res_quant_bits = 0;
							cm_delta_flc_bits_minus1 = 0;
							cm_adapt_threshold_u_delta = 0;
							cm_adapt_threshold_v_delta = 0;
						}
						uint32 num_cm_ref_layers_minus1;				// ue(v), 0-61
						uint8 cm_ref_layer_id[62];						// u(6)
						uint8 cm_octant_depth;							// u(2), 0-1
						uint8 cm_y_part_num_log2;						// u(2), (cm_y_part_num_log2 + cm_octant_depth) shall be in the range of 0 to 3
						uint32 luma_bit_depth_cm_input_minus8;			// ue(v)
						uint32 chroma_bit_depth_cm_input_minus8;		// ue(v)
						uint32 luma_bit_depth_cm_output_minus8;			// ue(v)
						uint32 chroma_bit_depth_cm_output_minus8;		// ue(v)
						uint8 cm_res_quant_bits;						// u(2)
						uint8 cm_delta_flc_bits_minus1;					// u(2)
						int32 cm_adapt_threshold_u_delta;				// se(v)
						int32 cm_adapt_threshold_v_delta;				// se(v)

					};
					struct FRefLocOffset
					{
						uint8 ref_loc_offset_layer_id = 0;				// u(6)
						uint8 scaled_ref_layer_offset_present_flag = 0;	// u(1)
						int32 scaled_ref_layer_left_offset = 0;			// se(v)
						int32 scaled_ref_layer_top_offset = 0;			// se(v)
						int32 scaled_ref_layer_right_offset = 0;		// se(v)
						int32 scaled_ref_layer_bottom_offset = 0;		// se(v)
						uint8 ref_region_offset_present_flag = 0;		// u(1)
						int32 ref_region_left_offset = 0;				// se(v)
						int32 ref_region_top_offset = 0;				// se(v)
						int32 ref_region_right_offset = 0;				// se(v)
						int32 ref_region_bottom_offset = 0;				// se(v)
						uint8 resample_phase_set_present_flag = 0;		// u(1)
						uint32 phase_hor_luma = 0;						// ue(v)
						uint32 phase_ver_luma = 0;						// ue(v)
						uint32 phase_hor_chroma_plus8 = 0;				// ue(v)
						uint32 phase_ver_chroma_plus8 = 0;				// ue(v)
					};
					uint8 poc_reset_info_present_flag;					// u(1)
					uint8 pps_infer_scaling_list_flag;					// u(1)
					uint8 pps_scaling_list_ref_layer_id;				// u(6), 0-62
					uint32 num_ref_loc_offsets;							// ue(v), 0 to vps_max_layers_minus1
					TArray<FRefLocOffset> rlos;
					uint8 colour_mapping_enabled_flag;					// u(1)
					FColourMappingTable colour_mapping_table;
				};

				struct FPPS3DExtension
				{
					bool Parse(FBitstreamReaderH265& br);
					void Reset()
					{
					}
				};

				struct FPPSSCCExtension
				{
					FPPSSCCExtension()
					{
						Reset();
					}
					bool Parse(FBitstreamReaderH265& br);
					void Reset()
					{
						pps_curr_pic_ref_enabled_flag = 0;
						residual_adaptive_colour_transform_enabled_flag = 0;
						pps_slice_act_qp_offsets_present_flag = 0;
						pps_act_y_qp_offset_plus5 = 0;
						pps_act_cb_qp_offset_plus5 = 0;
						pps_act_cr_qp_offset_plus3 = 0;
						pps_palette_predictor_initializers_present_flag = 0;
						pps_num_palette_predictor_initializers = 0;
						monochrome_palette_flag = 0;
						luma_bit_depth_entry_minus8 = 0;
						chroma_bit_depth_entry_minus8 = 0;
						pps_palette_predictor_initializer.Empty();
					}
					uint8 pps_curr_pic_ref_enabled_flag;				// u(1)
					uint8 residual_adaptive_colour_transform_enabled_flag;// u(1)
					uint8 pps_slice_act_qp_offsets_present_flag;		// u(1)
					int32 pps_act_y_qp_offset_plus5;					// se(v)
					int32 pps_act_cb_qp_offset_plus5;					// se(v)
					int32 pps_act_cr_qp_offset_plus3;					// se(v)
					uint8 pps_palette_predictor_initializers_present_flag;// u(1)
					uint32 pps_num_palette_predictor_initializers;		// ue(v)
					uint8 monochrome_palette_flag;						// u(1)
					uint32 luma_bit_depth_entry_minus8;					// ue(v)
					uint32 chroma_bit_depth_entry_minus8;				// ue(v)
					TArray<TArray<uint32>> pps_palette_predictor_initializer;// u(v)
				};

				FPPSRangeExtension pps_range_extension;
				FPPSMultilayerExtension pps_multilayer_extension;
				FPPS3DExtension pps_3d_extension;
				FPPSSCCExtension pps_scc_extension;
			};

			struct FSliceSegmentHeader
			{
				uint8 first_slice_segment_in_pic_flag = 0;				// u(1)
				uint8 no_output_of_prior_pics_flag = 0;					// u(1)
				uint32 slice_pic_parameter_set_id = 0;					// ue(v), 0-63
				uint8 dependent_slice_segment_flag = 0;					// u(1), default 0
				uint32 slice_segment_address = 0;						// u(v)
				uint32 slice_type;										// ue(v), 0-2
				uint8 pic_output_flag = 1;								// u(1)
				uint8 colour_plane_id = 0;								// u(2), 0-2
				uint32 slice_pic_order_cnt_lsb = 0;						// u(v)
				uint8 short_term_ref_pic_set_sps_flag = 0;				// u(1)
				uint32 short_term_ref_pic_set_idx = 0;					// u(v), default 0
				FStRefPicSet st_ref_pic_set;
				uint32 num_long_term_sps = 0;							// ue(v), default 0
				uint32 num_long_term_pics = 0;							// ue(v), default 0
				struct FLongTermRef
				{
					uint32 lt_idx_sps = 0;								// u(v)
					uint32 poc_lsb_lt = 0;								// u(v)
					uint8 used_by_curr_pic_lt_flag = 0;					// u(1)
					uint8 delta_poc_msb_present_flag = 0;				// u(1)
					uint32 delta_poc_msb_cycle_lt = 0;					// ue(v)
				};
				TArray<FLongTermRef> long_term_ref;
				uint8 slice_temporal_mvp_enabled_flag = 0;				// u(1)
				uint8 slice_sao_luma_flag = 0;							// u(1)
				uint8 slice_sao_chroma_flag = 0;						// u(1)

				uint8 num_ref_idx_active_override_flag = 0;				// u(1)
				uint32 num_ref_idx_l0_active_minus1 = 0;				// ue(v), 0-14, defaults to PPS's num_ref_idx_l0_default_active_minus1
				uint32 num_ref_idx_l1_active_minus1 = 0;				// ue(v), 0-14, defaults to PPS's num_ref_idx_l1_default_active_minus1
				struct FRefPicListModification
				{
					TArray<uint32> list_entry_l0;						// u(v)
					TArray<uint32> list_entry_l1;						// u(v)
				};
				FRefPicListModification ref_pic_lists_modification;

				uint8 mvd_l1_zero_flag = 0;								// u(1)
				uint8 cabac_init_flag = 0;								// u(1), default 0
				uint8 collocated_from_l0_flag = 1;						// u(1), default 1
				uint32 collocated_ref_idx = 0;							// ue(v), default 0

				struct FPredWeightTable
				{
					uint32 luma_log2_weight_denom = 0;					// ue(v), 0-7
					int32 delta_chroma_log2_weight_denom = 0;			// se(v), default 0
					uint8 luma_weight_lX_flag[2][16] {};				// u(1)
					uint8 chroma_weight_lX_flag[2][16] {};				// u(1)
					int32 delta_luma_weight_lX[2][16] {};				// se(v), -128 to 127, defaults to 1 << luma_log2_weight_denom
					int32 luma_offset_lX[2][16] {};						// se(v), default 0
					int32 delta_chroma_weight_lX[2][16][2] {};			// se(v), -127 to 128, defaults to 1 << (luma_log2_weight_denom + delta_chroma_log2_weight_denom)
					int32 delta_chroma_offset_lX[2][16][2] {};			// se(v), default 0
				};
				FPredWeightTable pred_weight_table;

				uint32 five_minus_max_num_merge_cand = 0;				// ue(v), 0-4
				uint8 use_integer_mv_flag = 0;							// u(1), defaults to motion_vector_resolution_control_idc

				int32 slice_qp_delta = 0;								// se(v), -QpBdOffsetY to +51
				int32 slice_cb_qp_offset = 0;							// se(v), -12 to +12, default 0
				int32 slice_cr_qp_offset = 0;							// se(v), -12 to +12, default 0
				int32 slice_act_y_qp_offset = 0;						// se(v), -12 to +12, default 0
				int32 slice_act_cb_qp_offset = 0;						// se(v), -12 to +12, default 0
				int32 slice_act_cr_qp_offset = 0;						// se(v), -12 to +12, default 0
				uint8 cu_chroma_qp_offset_enabled_flag = 0;				// u(1)
				uint8 deblocking_filter_override_flag = 0;				// u(1)
				uint8 slice_deblocking_filter_disabled_flag = 0;		// u(1), defaults to pps_deblocking_filter_disabled_flag
				int32 slice_beta_offset_div2 = 0;						// se(v), -6 to +6, defaults to pps_beta_offset_div2
				int32 slice_tc_offset_div2 = 0;							// se(v), -6 to +6, defaults to pps_tc_offset_div2
				uint8 slice_loop_filter_across_slices_enabled_flag = 0;	// u(1), defaults to pps_loop_filter_across_slices_enabled_flag
				uint32 num_entry_point_offsets = 0;						// ue(v)
				uint32 offset_len_minus1 = 0;							// ue(v)
				TArray<uint32> entry_point_offset_minus1;				// u(v)

				// Convenience values used for parsing that do not come from the slice header itself.
				int32 NumBitsForPOCValues = 0;
				uint32 NalUnitType = 0;
				uint32 NuhLayerId = 0;
				uint32 NuhTemporalIdPlus1 = 0;
				bool bIsIDR = false;
				bool bIsCRA = false;
				bool bIsIRAP = false;
				bool bIsBLA = false;
				bool bIsReferenceNalu = false;
				bool bIsSLNR = false;
				uint32 CurrRpsIdx = 0;
				uint8 HighestTid = 0;									// sps_max_sub_layers_minus1
				uint32 sps_max_num_reorder_pics_HighestTid = 0;
				uint32 sps_max_latency_increase_plus1_HighestTid = 0;
				uint32 sps_max_dec_pic_buffering_minus1_HighestTid = 0;
				bool SPSLongTermRefPicsPresentFlag = false;
				TArray<FSequenceParameterSet::FLongTermRefPic> SPSLongTermRefPics;
				uint32 NumBitsForShortTermRefs = 0;
				uint32 NumBitsForLongTermRefs = 0;
			};

			class FSliceSegmentHeaderPOCVars
			{
			public:
				FSliceSegmentHeaderPOCVars()
				{
					Reset();
				}
				void Reset()
				{
					rps.Reset();
					prevTid0POC = 0;
					SlicePOC = 0;
					MaxPocLsb = 0;
					PocStCurrBefore.Empty();
					PocStCurrAfter.Empty();
					PocStFoll.Empty();
					PocLtCurr.Empty();
					PocLtFoll.Empty();
					CurrDeltaPocMsbPresentFlag.Empty();
					FollDeltaPocMsbPresentFlag.Empty();
				}
				UE_API void Update(const FSliceSegmentHeader& InFromSliceHeader);
				int32 GetSlicePOC() const
				{ return SlicePOC; }
				int32 GetMaxPocLsb() const
				{ return MaxPocLsb; }

				const TArray<int32>& GetPocStCurrBefore() const
				{ return PocStCurrBefore; }
				const TArray<int32>& GetPocStCurrAfter() const
				{ return PocStCurrAfter; }
				const TArray<int32>& GetPocStFoll() const
				{ return PocStFoll; }
				const TArray<int32>& GetPocLtCurr() const
				{ return PocLtCurr; }
				const TArray<int32>& GetPocLtFoll() const
				{ return PocLtFoll; }
				const TArray<bool>& GetCurrDeltaPocMsbPresentFlag() const
				{ return CurrDeltaPocMsbPresentFlag; }
				const TArray<bool>& GetFollDeltaPocMsbPresentFlag() const
				{ return FollDeltaPocMsbPresentFlag; }

			private:
				struct FRefPicSet : public FStRefPicSet
				{
					void Reset()
					{
						FStRefPicSet::Reset();
						FMemory::Memzero(poc_lt);
						FMemory::Memzero(used_by_curr_pic_lt_flag);
						FMemory::Memzero(delta_poc_msb_present_lt_flag);
						num_long_term_pics = 0;
					}
					FRefPicSet& operator = (const FStRefPicSet& rhs)
					{
						FStRefPicSet::operator=(rhs);
						FMemory::Memzero(poc_lt);
						FMemory::Memzero(used_by_curr_pic_lt_flag);
						FMemory::Memzero(delta_poc_msb_present_lt_flag);
						num_long_term_pics = 0;
						return *this;
					}
					int32 poc_lt[32] {};
					uint8 used_by_curr_pic_lt_flag[32] {};
					uint8 delta_poc_msb_present_lt_flag[32] {};
					uint32 num_long_term_pics = 0;

					uint32 GetNumLongTermPics() const
					{ return num_long_term_pics; }
					int32 GetLongTermPOC(uint32 InIndex) const
					{ return poc_lt[InIndex]; }
					bool IsUsedLongTerm(uint32 InIndex) const
					{ return used_by_curr_pic_lt_flag[InIndex] != 0; }
					bool IsLongTermPOCMSBPresent(uint32 InIndex) const
					{ return delta_poc_msb_present_lt_flag[InIndex] != 0; }
				};

				FRefPicSet rps;
				int32 prevTid0POC = 0;
				int32 SlicePOC = 0;
				int32 MaxPocLsb = 0;
				TArray<int32> PocStCurrBefore;
				TArray<int32> PocStCurrAfter;
				TArray<int32> PocStFoll;
				TArray<int32> PocLtCurr;
				TArray<int32> PocLtFoll;
				TArray<bool> CurrDeltaPocMsbPresentFlag;
				TArray<bool> FollDeltaPocMsbPresentFlag;
			};

			struct FOutputFrameInfo
			{
				int32 IndexInBuffer = -1;
				FTimespan PTS;
				uint64 UserValue0 = 0;
				uint64 UserValue1 = 0;
				bool bDoNotOutput = false;
			};

			class FDecodedPictureBuffer : public FSliceSegmentHeaderPOCVars
			{
			public:
				struct FReferenceFrameListEntry
				{
					void Reset()
					{
						POC = 0;
						DPBIndex = -1;
						bIsShortTermReference = false;
						bIsLongTermReference = false;
						bNeededForOutput = false;
						bWasAlreadyOutput = false;
						bIsMissing = false;
						PicLatencyCount = 0;
						UserFrameInfo = FOutputFrameInfo();
					}
					FOutputFrameInfo UserFrameInfo;
					int32 POC = 0;
					int32 DPBIndex = -1;
					bool bIsShortTermReference = false;
					bool bIsLongTermReference = false;
					bool bNeededForOutput = false;
					bool bWasAlreadyOutput = false;
					bool bIsMissing = false;		// set to true when a "missing frame" reference frame had to be inserted.
					int32 PicLatencyCount = 0;
				};

				FDecodedPictureBuffer()
				{
					Reset();
				}


				struct FDPBOutputFrame
				{
					FOutputFrameInfo UserFrameInfo;
					bool bDoNotDisplay = true;
				};

				UE_API void ProcessFirstSliceOfFrame(TArray<FDPBOutputFrame>& OutFrames, const FSliceSegmentHeader& InFromSliceHeader, bool bIsFirstInSequence);
				UE_API bool AddDecodedFrame(TArray<FDPBOutputFrame>& OutFrames, const FOutputFrameInfo& InNewDecodedFrame, const FSliceSegmentHeader& InFromSliceHeader);

				enum EList
				{
					eStCurrBefore,
					eStCurrAfter,
					eStFoll,
					eLtCurr,
					eLtFoll,
					eMAX
				};
				UE_API void GetReferenceFramesFromDPB(TArray<FReferenceFrameListEntry>& OutReferenceFrames, TArray<int32> OutDPBIndexLists[eMAX]);
				UE_API const FReferenceFrameListEntry* GetDPBEntryAtIndex(int32 InIndex) const;

				// Call this to get all pending frame infos and clear the structure.
				UE_API void Flush(TArray<FDPBOutputFrame>& OutRemainingFrameInfos);

				// Call this to reset this structure.
				UE_API void Reset();

				FString GetLastError() const
				{
					return LastErrorMsg;
				}
			private:
				UE_API bool UpdatePOCandRPS(const FSliceSegmentHeader& InFromSliceHeader);
				UE_API FReferenceFrameListEntry* GenerateMissingFrame(int32 InPOC, bool bIsLongterm);
				UE_API void EmitNoLongerNeeded(TArray<FDPBOutputFrame>& OutFrames);
				UE_API void EmitAllRemainingAndEmpty(TArray<FDPBOutputFrame>& OutFrames);
				UE_API void BumpPictures(TArray<FDPBOutputFrame>& OutFrames, const FSliceSegmentHeader& InFromSliceHeader, bool bAdditionalBumping);

				UE_API bool SetLastError(const FString& InLastErrorMsg);

				bool bNoRaslOutputFlag = false;
				bool bCRANoRaslOutputFlag = false;

				TArray<FReferenceFrameListEntry> DPBEntries;
				TArray<int32> DPBIdxStCurrBefore;
				TArray<int32> DPBIdxStCurrAfter;
				TArray<int32> DPBIdxStFoll;
				TArray<int32> DPBIdxLtCurr;
				TArray<int32> DPBIdxLtFoll;

				FString LastErrorMsg;
			};


			bool ELECTRADECODERS_API ParseVideoParameterSet(TMap<uint32, FVideoParameterSet>& InOutVideoParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			bool ELECTRADECODERS_API ParseSequenceParameterSet(TMap<uint32, FSequenceParameterSet>& InOutSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			bool ELECTRADECODERS_API ParsePictureParameterSet(TMap<uint32, FPictureParameterSet>& InOutPictureParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			bool ELECTRADECODERS_API ParseSliceHeader(TUniquePtr<FRBSP>& OutRBSP, FBitstreamReaderH265& OutRBSPReader, FSliceSegmentHeader& OutSlice, const TMap<uint32, FVideoParameterSet>& InVideoParameterSets, const TMap<uint32, FSequenceParameterSet>& InSequenceParameterSets, const TMap<uint32, FPictureParameterSet>& InPictureParameterSets, const uint8* InBitstream, uint64 InBitstreamLenInBytes);

			bool ELECTRADECODERS_API ParseVideoParameterSet(FVideoParameterSet& OutSequenceParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			inline bool ParseVideoParameterSet(FVideoParameterSet& OutSequenceParameterSet, const TArray<uint8>& InData)
			{ return ParseVideoParameterSet(OutSequenceParameterSet, InData.GetData(), InData.Num()); }

			bool ELECTRADECODERS_API ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const uint8* InBitstream, uint64 InBitstreamLenInBytes);
			inline bool ParseSequenceParameterSet(FSequenceParameterSet& OutSequenceParameterSet, const TArray<uint8>& InData)
			{ return ParseSequenceParameterSet(OutSequenceParameterSet, InData.GetData(), InData.Num()); }



			class FHEVCDecoderConfigurationRecord
			{
			public:
				ELECTRADECODERS_API bool Parse(const TConstArrayView<uint8>& InDCR);
				ELECTRADECODERS_API bool CreateFromCodecSpecificData(const TConstArrayView<uint8>& InCSD);

				inline const TArray<uint8>& GetRawData() const
				{ return RawData; }
				inline const TArray<uint8>& GetCodecSpecificData() const
				{ return CSD; }

				inline const TArray<TArray<uint8>>& GetVideoParameterSets() const
				{ return VideoParameterSets; }
				inline const TArray<TArray<uint8>>& GetSequenceParameterSets() const
				{ return SequenceParameterSets; }
				inline const TArray<FSequenceParameterSet>& GetParsedSequenceParameterSets() const
				{ return ParsedSequenceParameterSets; }
				inline uint8 GetGeneralProfileSpace() const
				{ return GeneralProfileSpace; }
				inline uint8 GetGeneralTierFlag() const
				{ return GeneralTierFlag; }
				inline uint8 GetGeneralProfileIDC() const
				{ return GeneralProfileIDC; }
				inline uint32 GetGeneralProfileCompatibilityFlags() const
				{ return GeneralProfileCompatibilityFlags; }
				inline uint64 GetGeneralConstraintIndicatorFlags() const
				{ return GeneralConstraintIndicatorFlags; }
				inline uint8 GetGeneralLevelIDC() const
				{ return GeneralLevelIDC; }
				inline uint8 GetChromaFormat() const
				{ return ChromaFormat; }
				inline uint8 GetBitDepthLumaMinus8() const
				{ return BitDepthLumaMinus8; }
				inline uint8 GetBitDepthChromaMinus8() const
				{ return BitDepthChromaMinus8; }
				inline const TArray<TArray<uint8>>& GetPrefixNUTs() const
				{ return PrefixNUTs; }
				inline const TArray<TArray<uint8>>& GetSuffixNUTs() const
				{ return SuffixNUTs; }

				ELECTRADECODERS_API FString GetCodecSpecifierRFC6381(const TCHAR* InPrefix) const;
				ELECTRADECODERS_API FString GetFormatInfo() const;

			private:
				struct FArray
				{
					TArray<TArray<uint8>> NALUs;
					uint8 Completeness = 0;
					uint8 NALUnitType = 0;
				};
				TArray<FArray> Arrays;
				TArray<TArray<uint8>> VideoParameterSets;
				TArray<TArray<uint8>> SequenceParameterSets;
				TArray<TArray<uint8>> PrefixNUTs;
				TArray<TArray<uint8>> SuffixNUTs;
				TArray<FVideoParameterSet> ParsedVideoParameterSets;
				TArray<FSequenceParameterSet> ParsedSequenceParameterSets;
				TArray<uint8> RawData;
				TArray<uint8> CSD;
				uint64 GeneralConstraintIndicatorFlags = 0;
				uint32 GeneralProfileCompatibilityFlags = 0;
				uint16 MinSpatialSegmentationIDC = 0;
				uint16 AverageFrameRate = 0;
				uint8 ConfigurationVersion = 0;
				uint8 GeneralProfileSpace = 0;
				uint8 GeneralTierFlag = 0;
				uint8 GeneralProfileIDC = 0;
				uint8 GeneralLevelIDC = 0;
				uint8 ParallelismType = 0;
				uint8 ChromaFormat = 0;
				uint8 BitDepthLumaMinus8 = 0;
				uint8 BitDepthChromaMinus8 = 0;
				uint8 ConstantFrameRate = 0;
				uint8 NumTemporalLayers = 0;
				uint8 TemporalIdNested = 0;
				uint8 NALUnitLengthMinus1 = 0;
			};

		} // namespace H265
	} // namespace MPEG
} // namespace ElectraDecodersUtil

#undef UE_API
