// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraDecoder.h"


class FElectraDecoderBitstreamProcessorAPV : public IElectraDecoderBitstreamProcessor
{
public:
	static TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> Create(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams)
	{ return MakeShareable<>(new FElectraDecoderBitstreamProcessorAPV(InDecoderParams, InInitialCodecFormat, InAdditionalFormatParams)); }

	virtual ~FElectraDecoderBitstreamProcessorAPV();
	bool WillModifyBitstreamInPlace() override;
	void Clear() override;
	EProcessResult ProcessInputForDecoding(TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe>& OutBSI, FElectraDecoderInputAccessUnit& InOutAccessUnit, const TMap<FString, FVariant>& InAccessUnitSidebandData) override;
	void SetPropertiesOnOutput(TMap<FString, FVariant>& InOutProperties, TSharedPtr<IElectraDecoderBitstreamInfo, ESPMode::ThreadSafe> InBSI) override;
	FString GetLastError() override;

private:
	ELECTRADECODERS_API FElectraDecoderBitstreamProcessorAPV(const TMap<FString, FVariant>& InDecoderParams, const Electra::FCodecTypeFormat& InInitialCodecFormat, const TMap<FString, FVariant>& InAdditionalFormatParams);
	class FImpl;
	TPimplPtr<FImpl> Impl;
};
