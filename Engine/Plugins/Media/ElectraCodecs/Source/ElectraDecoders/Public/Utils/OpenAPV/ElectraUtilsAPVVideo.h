// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Utilities/ElectraBitstream.h"

namespace ElectraDecodersUtil
{
	namespace APVVideo
	{
		struct FColorInfo
		{
			uint8 ColorPrimaries = 2;
			uint8 TransferCharacteristics = 2;
			uint8 MatrixCoefficients = 2;
			uint8 VideoFullRangeFlag = 0;
		};
		struct FFrameInfo
		{
			uint32 FrameWidth = 0;
			uint32 FrameHeight = 0;
			uint8 ProfileIDC = 0;
			uint8 LevelIDC = 0;
			uint8 BandIDC = 0;
			uint8 ChromaFormatIDC = 0;
			uint8 BitDepthMinus8 = 0;
			uint8 CaptureTimeDistance = 0;
			bool bHaveColorDescription = false;
			bool bCaptureTimeDistanceIgnored = true;
			FColorInfo ColorInfo;
		};
		ELECTRADECODERS_API bool ParseFrameHeader(FFrameInfo& OutFrameInfo, const TConstArrayView<uint8> InFramePBU);

		/**
		 * APV Codec ISO Media File Format Binding
		 *
		 * see: https://github.com/AcademySoftwareFoundation/openapv/blob/main/readme/apv_isobmff.md#apv-decoder-configuration-record-apvdecoderconfigurationrecord
		 */
		class FAPVDecoderConfigurationRecord
		{
		public:
			struct FConfigurationEntry
			{
				uint8 PBUType = 0;
				TArray<FFrameInfo> FrameInfos;
			};

			ELECTRADECODERS_API bool Parse(const TConstArrayView<uint8> InDCR);
			ELECTRADECODERS_API bool ParseFromAU(const TConstArrayView64<uint8> InAccessUnit);

			inline const TArray<uint8>& GetRawData() const
			{ return RawData; }

			inline int32 GetNumEncodedFrameVariants() const
			{ return NumEncodedFrameVariants; }

			inline uint32 GetWidth() const
			{ return Width; }
			inline uint32 GetHeight() const
			{ return Height; }
			inline uint8 GetProfile() const
			{ return Profile; }
			inline uint8 GetLevel() const
			{ return Level; }
			inline uint8 GetBand() const
			{ return Band; }
			inline uint8 GetBitDepth() const
			{ return BitDepth; }
			inline const FColorInfo& GetColorInfo() const
			{ return ColorInfo; }
			inline const TArray<FConfigurationEntry>& GetConfigurationEntries() const
			{ return ConfigurationEntries; }

			inline FString GetCodecSpecifierRFC6381() const
			{
				FString rfc(FString::Printf(TEXT("apv1.apvf%d.apvl%d.apvb%d"), LargestProfile, LargestLevel, LargestBand));
				return rfc;
			}

			ELECTRADECODERS_API FString GetFormatInfo();

			ELECTRADECODERS_API static FString GetFormatInfo(uint32 InProfile, uint32 InLevel);
		private:
			TArray<FConfigurationEntry> ConfigurationEntries;
			TArray<uint8> RawData;
			FColorInfo ColorInfo;
			uint32 Width = 0;
			uint32 Height = 0;
			uint8 Profile = 0;
			uint8 Level = 0;
			uint8 Band = 0;
			uint8 BitDepth = 0;
			uint8 LargestProfile = 0;
			uint8 LargestLevel = 0;
			uint8 LargestBand = 0;
			int32 NumEncodedFrameVariants = 0;
			bool bIsNonConformant = false;
		};


		struct FFramePBUInfo
		{
			// Offset to where the size of the PBU is located
			int64 PBUOffset = 0;
			// Size of the PBU including the size field.
			uint32 PBUSize = 0;
			bool bIsPrimaryFrame = false;
		};
		ELECTRADECODERS_API void ParseAUIntoFramePBUSubsamples(TArray<FFramePBUInfo>& OutPBUInfos, const TConstArrayView64<uint8> InAccessUnit);

	} // namespace APVVideo
} // namespace ElectraDecodersUtil
