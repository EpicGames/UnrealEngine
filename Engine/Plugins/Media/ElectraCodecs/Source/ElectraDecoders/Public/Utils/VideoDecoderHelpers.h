// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"
#include "Utils/MPEG/ElectraUtilsMPEGVideo.h"
#include "MediaVideoDecoderOutput.h"
#include "ParameterDictionary.h"

namespace Electra
{

namespace MPEG
{

class FVideoDecoderTimecode : public IVideoDecoderTimecode
{
public:
	virtual ~FVideoDecoderTimecode() = default;
	virtual FMPEGDefinition const* GetMPEGDefinition() const override
	{ return &DecoderTimecode; }
	void Update(const IVideoDecoderTimecode::FMPEGDefinition& InTimecode)
	{ DecoderTimecode = InTimecode; }
	void UpdateWith(const ElectraDecodersUtil::MPEG::FCommonPictureTiming& InTiming)
	{
		DecoderTimecode.clockTimestamp = InTiming.clockTimestamp;
		DecoderTimecode.num_units_in_tick = InTiming.num_units_in_tick;
		DecoderTimecode.time_scale = InTiming.time_scale;
		DecoderTimecode.time_offset = InTiming.time_offset;
		DecoderTimecode.n_frames = InTiming.n_frames;
		DecoderTimecode.timing_info_present_flag = InTiming.timing_info_present_flag;
		DecoderTimecode.clock_timestamp_flag = InTiming.clock_timestamp_flag;
		DecoderTimecode.ct_type = InTiming.ct_type;
		DecoderTimecode.nuit_field_based_flag = InTiming.nuit_field_based_flag;
		DecoderTimecode.counting_type = InTiming.counting_type;
		DecoderTimecode.full_timestamp_flag = InTiming.full_timestamp_flag;
		DecoderTimecode.discontinuity_flag = InTiming.discontinuity_flag;
		DecoderTimecode.cnt_dropped_flag = InTiming.cnt_dropped_flag;
		DecoderTimecode.seconds_value = InTiming.seconds_value;
		DecoderTimecode.minutes_value = InTiming.minutes_value;
		DecoderTimecode.hours_value = InTiming.hours_value;
		DecoderTimecode.FromH26x = InTiming.FromH26x;
	}
private:
	IVideoDecoderTimecode::FMPEGDefinition DecoderTimecode;
};



class FColorimetryHelper
{
public:
	ELECTRADECODERS_API void Reset();
	ELECTRADECODERS_API void Update(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format);
	ELECTRADECODERS_API void Update(const TArray<uint8>& InFromCOLRBox);
	ELECTRADECODERS_API void UpdateParamDict(FParamDict& InOutDictionary);
	ELECTRADECODERS_API bool GetCurrentValues(uint8& colour_primaries, uint8& transfer_characteristics, uint8& matrix_coeffs) const;

private:
	class FVideoDecoderColorimetry : public IVideoDecoderColorimetry
	{
	public:
		FVideoDecoderColorimetry(uint8 colour_primaries, uint8 transfer_characteristics, uint8 matrix_coeffs, uint8 video_full_range_flag, uint8 video_format)
		{
			Colorimetry.ColourPrimaries = colour_primaries;
			Colorimetry.TransferCharacteristics = transfer_characteristics;
			Colorimetry.MatrixCoefficients = matrix_coeffs;
			Colorimetry.VideoFullRangeFlag = video_full_range_flag;
			Colorimetry.VideoFormat = video_format;
		}
		virtual ~FVideoDecoderColorimetry() = default;
		IVideoDecoderColorimetry::FMPEGDefinition const* GetMPEGDefinition() const override
		{ return &Colorimetry; }
		IVideoDecoderColorimetry::FMPEGDefinition Colorimetry;
	};

	TSharedPtr<FVideoDecoderColorimetry, ESPMode::ThreadSafe> CurrentColorimetry;
};



class FHDRHelper
{
public:
	ELECTRADECODERS_API void Reset();
	ELECTRADECODERS_API void Update(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& InGlobalPrefixSEIs, const TArray<ElectraDecodersUtil::MPEG::FSEIMessage>& InLocalPrefixSEIs, bool bIsNewCLVS);
	ELECTRADECODERS_API void UpdateFromMPEGBoxes(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TArray<uint8>& InMDCVBox, const TArray<uint8>& InCLLIBox);
	ELECTRADECODERS_API void Update(int32 BitDepth, const FColorimetryHelper& InColorimetry, const TOptional<FVideoDecoderHDRMetadata_mastering_display_colour_volume>& InMDCV, const TOptional<FVideoDecoderHDRMetadata_content_light_level_info>& InCLLI);
	ELECTRADECODERS_API void UpdateWith(const ElectraDecodersUtil::MPEG::FSEImastering_display_colour_volume& InSEI);
	ELECTRADECODERS_API void UpdateWith(const ElectraDecodersUtil::MPEG::FSEIcontent_light_level_info& InSEI);
	ELECTRADECODERS_API void UpdateWith(const ElectraDecodersUtil::MPEG::FSEIalternative_transfer_characteristics& InSEI);
	ELECTRADECODERS_API void SetHDRType(int32 BitDepth, const FColorimetryHelper& InColorimetry);
	ELECTRADECODERS_API void UpdateParamDict(FParamDict& InOutDictionary);
private:
	class FVideoDecoderHDRInformation : public IVideoDecoderHDRInformation
	{
	public:
		virtual ~FVideoDecoderHDRInformation() = default;
		IVideoDecoderHDRInformation::EType GetHDRType() const override
		{ return HDRType; }
		const FVideoDecoderHDRMetadata_mastering_display_colour_volume* GetMasteringDisplayColourVolume() const override
		{ return MasteringDisplayColourVolume.GetPtrOrNull(); }
		const FVideoDecoderHDRMetadata_content_light_level_info* GetContentLightLevelInfo() const override
		{ return ContentLightLevelInfo.GetPtrOrNull(); }
		void SetHDRType(IVideoDecoderHDRInformation::EType InHDRType)
		{ HDRType = InHDRType; }
		void SetMasteringDisplayColourVolume(const FVideoDecoderHDRMetadata_mastering_display_colour_volume& In)
		{ MasteringDisplayColourVolume = In; }
		void SetContentLightLevelInfo(const FVideoDecoderHDRMetadata_content_light_level_info& In)
		{ ContentLightLevelInfo = In; }
	private:
		IVideoDecoderHDRInformation::EType HDRType = IVideoDecoderHDRInformation::EType::Unknown;
		TOptional<FVideoDecoderHDRMetadata_mastering_display_colour_volume> MasteringDisplayColourVolume;
		TOptional<FVideoDecoderHDRMetadata_content_light_level_info> ContentLightLevelInfo;
	};
	TSharedPtr<FVideoDecoderHDRInformation, ESPMode::ThreadSafe> CurrentHDRInfo;
	TOptional<ElectraDecodersUtil::MPEG::FSEIMessage> ActiveMasteringDisplayColourVolume;
	TOptional<ElectraDecodersUtil::MPEG::FSEIMessage> ActiveContentLightLevelInfo;
	TOptional<ElectraDecodersUtil::MPEG::FSEIMessage> ActiveAlternativeTransferCharacteristics;
	bool bIsFirst = true;
	int32 CurrentAlternativeTransferCharacteristics = -1;
};



} // namespace MPEG
} // namespace Electra

