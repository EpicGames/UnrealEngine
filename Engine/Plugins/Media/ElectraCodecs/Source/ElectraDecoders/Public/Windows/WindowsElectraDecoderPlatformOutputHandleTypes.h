// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ElectraDecodersPlatformResources.h"
#include "PixelFormat.h"
#include "IElectraTextureSampleEncoding.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "d3d12.h"
#include "mfobjects.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

enum class EElectraDecoderPlatformOutputHandleType
{
	MFSample,
	DXDevice,
	DXDeviceContext,
	ImageBuffers			// IElectraDecoderVideoOutputImageBuffers interface
};

class FElectraDecoderOutputSync final
{
public:
	FElectraDecoderOutputSync() : SyncValue(0), CopyDoneSyncValue(0) {}
	FElectraDecoderOutputSync(TRefCountPtr<IUnknown> InSync, uint64 InSyncValue, TRefCountPtr<IMFSample> InMFSample = nullptr)
		: Sync(InSync), SyncValue(InSyncValue), CopyDoneSyncValue(0), MFSample(InMFSample)
	{}
	FElectraDecoderOutputSync(TRefCountPtr<IUnknown> InSync, uint64 InSyncValue, TRefCountPtr<ID3D12Fence> InCopyDoneSync, uint64 InCopyDoneSyncValue)
		: Sync(InSync), SyncValue(InSyncValue), CopyDoneSync(InCopyDoneSync), CopyDoneSyncValue(InCopyDoneSyncValue)
	{}

	TRefCountPtr<IUnknown>		Sync;														// Optional IUnknown based sync primitive (fence, or DX12 specific WMF object)
	uint64						SyncValue;													// Optionally needed sync value

	TRefCountPtr<ID3D12Fence>	CopyDoneSync;												// Optional fence to signal end of copy from this buffer back to decoder
	uint64						CopyDoneSyncValue;											// Value to be used with CopyDoneSync fence

private:
	TRefCountPtr<IMFSample>		MFSample;													// The sample reference can ride along to ensure that the DX12 WMF interface properly keeps reference to the sample until we enqueue their sync primitives with the a D3D queue
};

class IElectraDecoderVideoOutputImageBuffers
{
public:
	// Return the 4cc of the codec.
	virtual uint32 GetCodec4CC() const = 0;

	// Returns the number of separate image buffers making up the frame.
	virtual int32 GetNumberOfBuffers() const = 0;

	// Returns the n'th image buffer as CPU data buffer.
	virtual TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer as GPU texture resource.
	virtual void* GetBufferTextureByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer's GPU sync object to signal texture data as being ready to be read
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const = 0;

	// Returns the n'th image buffer format
	virtual EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer encoding
	virtual EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const = 0;

	// Returns the n'th image buffer pitch
	virtual int32 GetBufferPitchByIndex(int32 InBufferIndex) const = 0;
};

#define ELECTRA_HAVE_IMAGEBUFFERS 1
