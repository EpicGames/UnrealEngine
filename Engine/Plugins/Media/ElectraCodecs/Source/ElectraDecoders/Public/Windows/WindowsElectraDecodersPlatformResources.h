// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#include "ParameterDictionary.h"

class IElectraDecoderVideoOutput;
class FElectraTextureSample;

class FElectraDecodersPlatformResourcesWindows
{
public:
	static ELECTRADECODERS_API bool GetD3DDeviceAndVersion(void** OutD3DDevice, int32* OutD3DVersion);

	class FAsyncConsecutiveTaskSync;
	static ELECTRADECODERS_API TSharedPtr<FAsyncConsecutiveTaskSync, ESPMode::ThreadSafe> CreateAsyncConsecutiveTaskSync();
	static ELECTRADECODERS_API bool RunCodeAsync(TFunction<void()>&& InCodeToRun, TSharedPtr<FAsyncConsecutiveTaskSync, ESPMode::ThreadSafe> InTaskSync);

	struct IDecoderPlatformResource;
	static ELECTRADECODERS_API IDecoderPlatformResource* CreatePlatformVideoResource(void* InOwnerHandle, const TMap<FString, FVariant> InOptions);
	static ELECTRADECODERS_API void ReleasePlatformVideoResource(void* InOwnerHandle, IDecoderPlatformResource* InHandleToRelease);

	static ELECTRADECODERS_API bool SetupOutputTextureSample(FString& OutErrorMessage, TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> InOutTextureSampleToSetup, TSharedPtr<IElectraDecoderVideoOutput, ESPMode::ThreadSafe> InDecoderOutput, TSharedPtr<Electra::FParamDict, ESPMode::ThreadSafe> InOutBufferPropertes, FElectraDecodersPlatformResourcesWindows::IDecoderPlatformResource* InPlatformResource);
};

using FElectraDecodersPlatformResources = FElectraDecodersPlatformResourcesWindows;
